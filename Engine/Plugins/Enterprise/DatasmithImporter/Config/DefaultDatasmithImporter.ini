[CoreRedirects]

+ClassRedirects=(OldName="/Script/DatasmithImporter.DatasmithBlutilityActor",NewName="/Script/Blutility.PlacedEditorUtilityBase")

[FileProducerTessellationOptions]
ChordTolerance=0.3
MaxEdgeLength=0.0
NormalTolerance=30.0
StitchingTechnique=StitchingSew

[FileProducerImportOptions]
SceneHandling=CurrentLevel
IncludeGeometry=True
IncludeMaterial=True
IncludeLight=True
IncludeCamera=True
IncludeAnimation=True
