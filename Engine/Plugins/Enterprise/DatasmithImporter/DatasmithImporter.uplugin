{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Datasmith Importer",
	"Description" : "Importer for Datasmith files.",
	"Category" : "Importers",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "https://docs.unrealengine.com/en-US/WorkingWithContent/Importing/Datasmith/",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules": [
		{
			"Name": "DatasmithExternalSource",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DatasmithImporter",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DatasmithTranslator",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DatasmithNativeTranslator",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DirectLinkExtension",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DirectLinkExtensionEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ExternalSource",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DirectLinkTest",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "DatasmithContent",
			"Enabled": true
		},
		{
			"Name": "EditorScriptingUtilities",
			"Enabled": true
		},
		{
			"Name": "DataPrepEditor",
			"Enabled": true,
			"TargetAllowList": [
				"Editor"
			]
		},
		{
			"Name": "InterchangeEditor",
			"Enabled": true,
			"TargetAllowList": [
				"Editor"
			]
		},
		{
			"Name": "VariantManagerContent",
			"Enabled": true
		},
		{
			"Name": "VariantManager",
			"Enabled": true
		},
		{
			"Name": "UdpMessaging",
			"Enabled": true
		}
	]
}