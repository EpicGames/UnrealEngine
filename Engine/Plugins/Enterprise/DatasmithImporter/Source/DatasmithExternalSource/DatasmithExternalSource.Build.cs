// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
    public class DatasmithExternalSource : ModuleRules
    {
        public DatasmithExternalSource(ReadOnlyTargetRules Target) : base(Target)
        {
            PublicDependencyModuleNames.AddRange(
                new string[]
                {
                    "Core",
					"DirectLinkExtension",
					"ExternalSource",
                    "DatasmithNativeTranslator",
                }
            );

            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                    "DatasmithCore",
                    "DatasmithTranslator",
					"Projects",
				}
			);

			if (Target.Type == TargetType.Editor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"DesktopPlatform",
						"DirectLinkExtensionEditor",
						"Slate",
						"UnrealEd",
					}
				);
			}
		}
    }
}
