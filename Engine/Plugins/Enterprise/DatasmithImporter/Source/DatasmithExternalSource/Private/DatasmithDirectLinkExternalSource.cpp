// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithDirectLinkExternalSource.h"

#include "DirectLink/DatasmithSceneReceiver.h"

#include "Interfaces/IPluginManager.h"

namespace UE::DatasmithImporter
{
	FDatasmithDirectLinkExternalSource::~FDatasmithDirectLinkExternalSource()
	{
		if (SceneReceiver)
		{
			SceneReceiver->SetChangeListener(nullptr);
		}
	}

	TSharedPtr<IDatasmithScene> FDatasmithDirectLinkExternalSource::GetDatasmithScene() const
	{
		if (SceneReceiver)
		{
			return SceneReceiver->GetScene();
		}

		return nullptr;
	}

	FString FDatasmithDirectLinkExternalSource::GetFallbackFilepath() const
	{
		//This Datasmith.directlink file is only used as temporarely until the Uri and ExternalSource system is fully integrated in the engine.
		static FString DummyDirectLinkSource = IPluginManager::Get().FindPlugin(TEXT("DatasmithImporter"))->GetBaseDir() / TEXT("Resources") / TEXT("Datasmith.directlink");
		return DummyDirectLinkSource;
	}

	TSharedPtr<class DirectLink::ISceneReceiver> FDatasmithDirectLinkExternalSource::GetSceneReceiverInternal(const DirectLink::IConnectionRequestHandler::FSourceInformation& Source)
	{
		if (!SceneReceiver)
		{
			// At the moment the ISceneChangeListener interface is not compatible with the fact that we might be importing multiple asset from the same ExternalSource.
			// So we simply create a FDatasmithSceneReceiver without assigning it a SceneChangeListener
			SceneReceiver = MakeShared<FDatasmithSceneReceiver>();
		}

		return SceneReceiver;
	}
}