// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DirectLink/DatasmithSceneReceiver.h"
#include "DirectLinkExternalSource.h"

class IDatasmithScene;
class FDatasmithSceneReceiver;

namespace UE::DatasmithImporter
{
	class FDatasmithDirectLinkExternalSource : public FDirectLinkExternalSource
	{
	public:
		explicit FDatasmithDirectLinkExternalSource(const FSourceUri& InSourceUri)
			: FDirectLinkExternalSource(InSourceUri)
		{}

		virtual ~FDatasmithDirectLinkExternalSource();

		// FExternalSource interface begin
		virtual TSharedPtr<IDatasmithScene> GetDatasmithScene() const;
		virtual FString GetFallbackFilepath() const override;
		// FExternalSource interface end

		// DirectLink::IConnectionRequestHandler interface begin
		virtual bool CanOpenNewConnection(const DirectLink::IConnectionRequestHandler::FSourceInformation& Source) override { /*there is currently no way to know the content of the DirectLink source, assume datasmith.*/ return true; }
		// DirectLink::IConnectionRequestHandler interface end

	protected:
		// UE::DatasmithImporter::FDirectLinkExternalSource interface begin
		virtual TSharedPtr<class DirectLink::ISceneReceiver> GetSceneReceiverInternal(const DirectLink::IConnectionRequestHandler::FSourceInformation& Source) override;
		// UE::DatasmithImporter::FDirectLinkExternalSource interface end

	private:

		/**
		 * The SceneReceiver used to parse the DirectLink snapshot into a DatasmithScene.
		 */
		TSharedPtr<FDatasmithSceneReceiver> SceneReceiver;
	};
}