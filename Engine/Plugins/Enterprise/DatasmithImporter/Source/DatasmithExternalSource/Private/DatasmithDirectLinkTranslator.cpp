// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithDirectLinkTranslator.h"
#include "IDatasmithSceneElements.h"

namespace UE::DatasmithImporter
{
	void FDatasmithDirectLinkTranslator::Initialize(FDatasmithTranslatorCapabilities& OutCapabilities)
	{
		OutCapabilities.SupportedFileFormats.Emplace(TEXT("directlink"), TEXT("DirectLink stream"));
		OutCapabilities.bParallelLoadStaticMeshSupported = true;
	}


	bool FDatasmithDirectLinkTranslator::LoadScene(TSharedRef<IDatasmithScene> OutScene)
	{
		// With DirectLink the scene is already translated by the SceneReceiver, don't override the scene.
		FString ResourcePath = OutScene->GetResourcePath();
		TArray<FString> ResourcePaths;
		ResourcePath.ParseIntoArray(ResourcePaths, TEXT(";"));
		ResolveSceneFilePaths(OutScene, ResourcePaths);
		return true;
	}
}