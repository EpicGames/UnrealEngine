// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DatasmithNativeTranslator.h"

namespace UE::DatasmithImporter
{
	/**
	 * Due to the way the Editor's import/reimport utilities currently work, we must provide a path to an existing file.
	 * The FDatasmithDirecTLinkTranslator is a workaround used to be directed to the DatasmithImportFactory when the
	 * source file has a ".directlink" extention. In that case, the actual scene is loaded using the FExternalSource
	 * generated from the directlink URI.
	 */
	class FDatasmithDirectLinkTranslator : public FDatasmithNativeTranslator
	{
	public:
		virtual FName GetFName() const override { return "DatasmithDirectLinkTranslator"; };

		virtual void Initialize(FDatasmithTranslatorCapabilities& OutCapabilities) override;

		virtual bool LoadScene(TSharedRef<IDatasmithScene> OutScene) override;
	};
}