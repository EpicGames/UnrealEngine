// Copyright Epic Games, Inc. All Rights Reserved.
#include "DatasmithExternalSourceModule.h"

#include "DatasmithDirectLinkExternalSource.h"
#include "DatasmithDirectLinkTranslator.h"
#include "DatasmithFileUriResolver.h"
#include "DatasmithNativeTranslator.h"
#include "DatasmithTranslator.h"

#include "DirectLinkExtensionModule.h"
#include "IDirectLinkManager.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"


#include "ExternalSourceModule.h"
#include "IUriManager.h"


IMPLEMENT_MODULE(FDatasmithExternalSourceModule, DatasmithExternalSource);

namespace UE::DatasmithExternalSourceModule
{
	const FName DatasmithDirectLinkExternalSourceName(TEXT("DatasmithDirectLinkExternalSource"));
	const FName FileUriResolverName(TEXT("FileUriResolver"));
}

void FDatasmithExternalSourceModule::StartupModule()
{
	using namespace UE::DatasmithImporter;

	Datasmith::RegisterTranslator<FDatasmithDirectLinkTranslator>();

	IDirectLinkManager& DirectLinkManager = IDirectLinkExtensionModule::Get().GetManager();
	DirectLinkManager.RegisterDirectLinkExternalSource<FDatasmithDirectLinkExternalSource>(UE::DatasmithExternalSourceModule::DatasmithDirectLinkExternalSourceName);

	IUriManager& UriManager = IExternalSourceModule::Get().GetManager();
	UriManager.RegisterResolver(UE::DatasmithExternalSourceModule::FileUriResolverName, MakeShared<FDatasmithFileUriResolver>());
}

void FDatasmithExternalSourceModule::ShutdownModule()
{
	using namespace UE::DatasmithImporter;

	if (IDirectLinkExtensionModule::IsAvailable())
	{
		IDirectLinkManager& DirectLinkManager = IDirectLinkExtensionModule::Get().GetManager();
		DirectLinkManager.UnregisterDirectLinkExternalSource(UE::DatasmithExternalSourceModule::DatasmithDirectLinkExternalSourceName);
	}

	if (IExternalSourceModule::IsAvailable())
	{
		IUriManager& UriManager = IExternalSourceModule::Get().GetManager();
		UriManager.RegisterResolver(UE::DatasmithExternalSourceModule::FileUriResolverName, MakeShared<FDatasmithFileUriResolver>());
	}

	Datasmith::UnregisterTranslator<FDatasmithDirectLinkTranslator>();
}
