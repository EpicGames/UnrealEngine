// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithFileExternalSource.h"

#include "DatasmithFileUriResolver.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithSceneSource.h"
#include "DatasmithTranslatableSource.h"
#include "DatasmithTranslator.h"
#include "DatasmithUtils.h"
#include "Logging/LogMacros.h"
#include "Misc/Paths.h"

DEFINE_LOG_CATEGORY(LogDatasmithFileExternalSource);

namespace UE::DatasmithImporter
{
	FString FDatasmithFileExternalSource::GetSourceName() const
	{
		return FDatasmithUtils::SanitizeObjectName(FPaths::GetBaseFilename(FilePath));
	}

	bool FDatasmithFileExternalSource::IsAvailable() const
	{
		return FPaths::FileExists(FilePath);
	}

	bool FDatasmithFileExternalSource::IsOutOfSync() const
	{
		return !IsAvailable() || CachedHash != FMD5Hash::HashFile(*FilePath);
	}

	FExternalSourceCapabilities FDatasmithFileExternalSource::GetCapabilities() const
	{
		FExternalSourceCapabilities Capabilities;
		Capabilities.bSupportSynchronousLoading = true;

		return Capabilities;
	}

	FString FDatasmithFileExternalSource::GetFallbackFilepath() const
	{
		return FilePath;
	}

	TSharedPtr<IDatasmithScene> FDatasmithFileExternalSource::LoadImpl()
	{
		TSharedRef<IDatasmithScene> LoadedScene = FDatasmithSceneFactory::CreateScene(*GetSceneName());
		if (!TranslatorLoadScene(LoadedScene))
		{
			UE_LOGF(LogDatasmithFileExternalSource, Warning, "Datasmith import error: Scene translation failure. Abort import.");
			return nullptr;
		}

		CachedHash = FMD5Hash::HashFile(*FilePath);
		DatasmithScene = LoadedScene;

		return LoadedScene;
	}
}