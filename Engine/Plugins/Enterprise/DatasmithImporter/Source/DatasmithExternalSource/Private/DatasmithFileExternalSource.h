// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ExternalSource.h"

#include "CoreMinimal.h"
#include "Misc/SecureHash.h"

class IDatasmithScene;

DECLARE_LOG_CATEGORY_EXTERN(LogDatasmithFileExternalSource, Log, All);

namespace UE::DatasmithImporter
{
	class FDatasmithFileExternalSource : public FExternalSource
	{
	public:
		explicit FDatasmithFileExternalSource(const FSourceUri& InSourceUri)
			: FExternalSource(InSourceUri)
		{
			FilePath = GetSourceUri().GetPath();
		}

		// UFileExternalSource interface begin
		virtual FString GetSourceName() const override;
		virtual bool IsAvailable() const override;
		virtual bool IsOutOfSync() const override;
		virtual FMD5Hash GetSourceHash() const override { return CachedHash; }
		virtual FExternalSourceCapabilities GetCapabilities() const override;
		virtual TSharedPtr<IDatasmithScene> GetDatasmithScene() const override { return DatasmithScene; }
		virtual FString GetFallbackFilepath() const override;
	protected:
		virtual TSharedPtr<IDatasmithScene> LoadImpl();
		virtual bool StartAsyncLoad() { checkNoEntry(); return false; }
		// UFileExternalSource interface begin

	private:
		FString FilePath;

		FMD5Hash CachedHash;

		TSharedPtr<IDatasmithScene> DatasmithScene;
	};
}