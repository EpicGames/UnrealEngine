// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithFileUriResolver.h"

#include "DatasmithFileExternalSource.h"
#include "DatasmithSceneSource.h"
#include "DatasmithTranslatorManager.h"

#if WITH_EDITOR
#include "DesktopPlatformModule.h"
#include "Framework/Application/SlateApplication.h"
#include "IDesktopPlatform.h"
#include "ObjectTools.h"
#endif //WITH_EDITOR

namespace UE::DatasmithImporter
{
	TSharedPtr<FExternalSource> FDatasmithFileUriResolver::GetOrCreateExternalSource(const FSourceUri& Uri) const
	{
		if (CanResolveUri(Uri))
		{
			return MakeShared<FDatasmithFileExternalSource>(Uri);
		}

		return nullptr;
	}

	bool FDatasmithFileUriResolver::CanResolveUri(const FSourceUri& Uri) const
	{
		if (Uri.HasScheme(FSourceUri::GetFileScheme()))
		{
			FDatasmithSceneSource DatasmithSceneSource;
			DatasmithSceneSource.SetSourceFile(FString(Uri.GetPath()));
			return FDatasmithTranslatorManager::Get().SelectFirstCompatible(DatasmithSceneSource).IsValid();
		}

		return false;
	}

	FName FDatasmithFileUriResolver::GetScheme() const
	{
		return FName(FSourceUri::GetFileScheme());
	}

#if WITH_EDITOR
	TSharedPtr<FExternalSource> FDatasmithFileUriResolver::BrowseExternalSource(const FSourceUri& DefaultUri) const
	{
		const TArray<FString> Formats = FDatasmithTranslatorManager::Get().GetSupportedFormats();
		FString FileTypes;
		FString Extensions;

		ObjectTools::AppendFormatsFileExtensions(Formats, FileTypes, Extensions);

		const FString FormatString = FString::Printf(TEXT("All Files (%s)|%s|%s"), *Extensions, *Extensions, *FileTypes);
		const FString Title = NSLOCTEXT("DatasmithFileUriResolver", "BrowseSourceDialogTitle", "Select Datasmith Source File").ToString();

		FString DefaultFolder;
		FString DefaultFile;
		if (DefaultUri.IsValid())
		{
			const FString DefaultFilePath(DefaultUri.GetPath());
			DefaultFolder = FPaths::GetPath(FString(DefaultFilePath));
			DefaultFile = FPaths::GetCleanFilename(DefaultFilePath);
		}

		TArray<FString> OutOpenFilenames;
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
		bool bOpened = false;
		if (DesktopPlatform)
		{
			bOpened = DesktopPlatform->OpenFileDialog(
				FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
				Title,
				DefaultFolder,
				DefaultFile,
				FormatString,
				EFileDialogFlags::None,
				OutOpenFilenames
			);
		}

		if (bOpened && OutOpenFilenames.Num() > 0)
		{
			const FSourceUri FileSourceUri = FSourceUri::FromFilePath(OutOpenFilenames[0]);
			return GetOrCreateExternalSource(FileSourceUri);
		}

		return nullptr;
	}
#endif //WITH_EDITOR
}