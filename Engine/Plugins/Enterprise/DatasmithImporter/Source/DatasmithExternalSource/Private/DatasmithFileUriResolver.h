// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IUriResolver.h"

namespace UE::DatasmithImporter
{
	class FDatasmithFileUriResolver : public IUriResolver
	{
	public:
		// IUriResolver interface begin
		virtual TSharedPtr<FExternalSource> GetOrCreateExternalSource(const FSourceUri& URI) const override;
		virtual bool CanResolveUri(const FSourceUri& URI) const override;
		virtual FName GetScheme() const override;
#if WITH_EDITOR
		virtual TSharedPtr<FExternalSource> BrowseExternalSource(const FSourceUri& DefaultUri) const override;
#endif //WITH_EDITOR
		// IUriResolver interface end
	};
}