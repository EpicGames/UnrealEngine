// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#define DATASMITHEXTERNALSOURCE_MODULE_NAME TEXT("DatasmithExternalSource")

/**
 * Module adding support for Datasmith DirectLink and file external sources.
 */
class FDatasmithExternalSourceModule : public IModuleInterface
{
public:

    static FDatasmithExternalSourceModule& Get()
    {
        return FModuleManager::LoadModuleChecked< FDatasmithExternalSourceModule >(DATASMITHEXTERNALSOURCE_MODULE_NAME);
    }

    static bool IsAvailable()
    {
        return FModuleManager::Get().IsModuleLoaded(DATASMITHEXTERNALSOURCE_MODULE_NAME);
    }

	virtual void StartupModule() override;

	virtual void ShutdownModule() override;
};
