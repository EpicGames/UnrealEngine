// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class DatasmithImporter : ModuleRules
	{
		public DatasmithImporter(ReadOnlyTargetRules Target)
			: base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Analytics",
					"ApplicationCore",
					"CinematicCamera",
					"ContentBrowser",
					"Core",
					"CoreUObject",
					"DesktopPlatform",
					"DirectLinkExtension",
					"EditorFramework",
					"EditorStyle",
					"EditorScriptingUtilities",
					"Engine",
					"Foliage",
					"FreeImage",
					"HTTP",
					"InputCore",
					"InterchangeCore",
					"InterchangeEngine",
					"InterchangeFactoryNodes",
					"InterchangeNodes",
					"Json",
					"Kismet",
					"Landscape",
					"LandscapeEditor",
					"LandscapeEditorUtilities",
					"LevelSequence",
					"MainFrame",
					"MaterialEditor",
					"MeshDescription",
					"MeshUtilities",
					"MeshUtilitiesCommon",
					"MessageLog",
					"MovieScene",
					"MovieSceneTracks",
					"PropertyEditor",
					"RenderCore",
					"RHI",
					"RawMesh",
					"Slate",
					"SlateCore",
					"SourceControl",
					"StaticMeshDescription",
					"ToolMenus",
					"UnrealEd",
					"VariantManager",
					"VariantManagerContent",
				}
			);

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"DataprepCore",
					"DatasmithContent",
					"DatasmithCore",
					"DatasmithTranslator",
					"DatasmithContentEditor",
					"DirectLinkExtensionEditor",
					"ExternalSource",
				}
			);
		}
	}
}