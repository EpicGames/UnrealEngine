// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorFactoryDatasmithScene.h"

#include "Components/StaticMeshComponent.h"
#include "DatasmithAssetImportData.h"
#include "DatasmithBlueprintLibrary.h"
#include "DatasmithImportContext.h"
#include "DatasmithImporter.h"
#include "DatasmithScene.h"
#include "DatasmithSceneActor.h"
#include "DatasmithSceneFactory.h"
#include "Engine/Level.h"
#include "Utility/DatasmithImporterUtils.h"

#include "AssetRegistry/AssetData.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/World.h"
#include "ExternalSourceModule.h"
#include "GameFramework/Actor.h"
#include "IUriManager.h"
#include "SourceUri.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define LOCTEXT_NAMESPACE "ActorFactoryDatasmithScene"

namespace UActorFactoryDatasmithSceneImpl
{
	/**
	 * Imports the actors for a DatasmithSceneImportData.
	 *
	 * @param DatasmithScene	The scene for which we will be spawning the actors.
	 * @param RootActor			An optional existing ADatasmithSceneActor for which we will spawn/update the actors. It must refer to DatasmithScene.
	 * @param InLevel			Optional level to spawn the actors, if not specified, will use Current Level
	 * @param Transform			The transform to use if no RootActor was specified.
	 * @param ObjectFlags		The flags to use when spawning new actors.
	 *
	 * @return					Returns either RootActor if it was supplied or the newly spawned ADatasmithSceneActor that refers to DatasmithScene.
	 */
	ADatasmithSceneActor* ImportActors( UDatasmithScene* DatasmithScene, ADatasmithSceneActor* RootActor, ULevel* InLevel, const FTransform& Transform, EObjectFlags ObjectFlags, bool bReimportDeletedActors )
	{
		using namespace UE::DatasmithImporter;
		// The root and scene must match in order to update an existing root from a scene
		ensure( !RootActor || RootActor->Scene == DatasmithScene );

		if ( !DatasmithScene || !DatasmithScene->AssetImportData )
		{
			return nullptr; // Can't import without the AssetImportData
		}

		TSharedPtr< IDatasmithScene > DatasmithSceneElement = FDatasmithImporterUtils::LoadDatasmithScene( DatasmithScene );

		if ( !DatasmithSceneElement.IsValid() )
		{
			return nullptr;
		}

		FName LoggerName = TEXT("DatasmithActorFactory");
		FText LoggerLabel = LOCTEXT("DatasmithSceneDisplayName", "Datasmith Scene");
		
		TSharedPtr<UE::DatasmithImporter::FExternalSource> ExternalSource;
		if (TObjectPtr<UDatasmithSceneImportData> DatasmithImportData = Cast<UDatasmithSceneImportData>(DatasmithScene->AssetImportData))
		{
			ExternalSource = IExternalSourceModule::Get().GetManager().TryGetExternalSourceFromImportData(*DatasmithImportData.Get());
		}
		
		const FString ImportPath = DatasmithScene->AssetImportData->BaseOptions.AssetOptions.PackagePath.ToString();
		FDatasmithImportContext ImportContext( ExternalSource, false, LoggerName, LoggerLabel );

		ImportContext.Options->BaseOptions = DatasmithScene->AssetImportData->BaseOptions;
		ImportContext.Options->BaseOptions.SceneHandling = EDatasmithImportScene::CurrentLevel;

		if (InLevel)
		{
			ImportContext.ActorsContext.ImportWorld = InLevel->GetWorld();
		}

		const bool bSilent = true;
		TSharedPtr<FJsonObject> JsonOptions;
		ImportContext.InitScene( DatasmithSceneElement.ToSharedRef() );
		if ( !ImportContext.Init( ImportPath, ObjectFlags, GWarn, JsonOptions, bSilent ) )
		{
			return nullptr;
		}

		if ( !RootActor )
		{
			RootActor = FDatasmithImporterUtils::CreateImportSceneActor( ImportContext, Transform );
		}

		if ( !RootActor )
		{
			return nullptr;
		}

		ImportContext.ActorsContext.FinalSceneActors.Add( RootActor );
		ImportContext.bIsAReimport = true;
		ImportContext.Options->ReimportOptions.bRespawnDeletedActors = bReimportDeletedActors;

		if ( bReimportDeletedActors )
		{
			if ( ImportContext.Options->BaseOptions.bIncludeGeometry )
			{
				ImportContext.Options->StaticMeshActorImportPolicy = EDatasmithImportActorPolicy::Full;
			}

			if ( ImportContext.Options->BaseOptions.bIncludeLight )
			{
				ImportContext.Options->LightImportPolicy = EDatasmithImportActorPolicy::Full;
			}

			if ( ImportContext.Options->BaseOptions.bIncludeCamera )
			{
				ImportContext.Options->CameraImportPolicy = EDatasmithImportActorPolicy::Full;
			}

			ImportContext.Options->OtherActorImportPolicy = EDatasmithImportActorPolicy::Full;
		}

		ImportContext.SceneAsset = DatasmithScene;

		// The actor might get deleted or become unreachable if the user cancel the import/finalize
		TWeakObjectPtr<ADatasmithSceneActor> RootObjectAsWeakPtr = RootActor;

		FDatasmithImporter::ImportActors( ImportContext );
		FDatasmithImporter::FinalizeActors( ImportContext, nullptr );

		RootActor = RootObjectAsWeakPtr.Get();

		if ( RootActor )
		{
			// If the root actor is still valid, ensure that it is in a world.
			if ( !RootActor->GetWorld() )
			{
				RootActor = nullptr;
			}
		}

		return RootActor;
	}
}

void UActorFactoryDatasmithScene::SpawnRelatedActors( ADatasmithSceneActor* DatasmithSceneActor, bool bReimportDeletedActors )
{
	UActorFactoryDatasmithSceneImpl::ImportActors( DatasmithSceneActor->Scene, DatasmithSceneActor, nullptr, DatasmithSceneActor->GetActorTransform(), DatasmithSceneActor->GetFlags(), bReimportDeletedActors );
}

UActorFactoryDatasmithScene::UActorFactoryDatasmithScene(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("DatasmithSceneDisplayName", "Datasmith Scene");
	NewActorClass = ADatasmithSceneActor::StaticClass();
	bUseSurfaceOrientation = false;
	bShowInEditorQuickMenu = false;
}

bool UActorFactoryDatasmithScene::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	if ( !AssetData.IsValid() || !AssetData.GetClass()->IsChildOf( UDatasmithScene::StaticClass() ) )
	{
		OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoDatasmithSceneSpecified", "No Datasmith scene was specified.");
		return false;
	}

	return true;
}

AActor* UActorFactoryDatasmithScene::SpawnActor( UObject* Asset, ULevel* InLevel, const FTransform& Transform, const FActorSpawnParameters& InSpawnParams)
{
	UDatasmithScene* DatasmithScene = Cast< UDatasmithScene >( Asset );

	if ( !DatasmithScene )
	{
		return nullptr;
	}

	AActor* ResultingActor = nullptr;

	if (InSpawnParams.ObjectFlags & RF_Transient)
	{
		// This is a hack for drag and drop so that we don't spawn all the actors for the preview actor since it gets deleted right after.

		FActorSpawnParameters SpawnInfo(InSpawnParams);
		SpawnInfo.OverrideLevel = InLevel;
		AStaticMeshActor* DragActor = Cast< AStaticMeshActor >( InLevel->GetWorld()->SpawnActor( AStaticMeshActor::StaticClass(), &Transform, SpawnInfo) );
		DragActor->GetStaticMeshComponent()->SetStaticMesh( Cast< UStaticMesh >( FSoftObjectPath( TEXT("StaticMesh'/Engine/EditorMeshes/EditorSphere.EditorSphere'") ).TryLoad() ) );
		DragActor->SetActorScale3D( FVector( 0.1f ) );

		ResultingActor = DragActor;
	}
	else
	{
		ResultingActor = UActorFactoryDatasmithSceneImpl::ImportActors( DatasmithScene, nullptr, InLevel, Transform, InSpawnParams.ObjectFlags, false );
	}

	return ResultingActor;
}

#undef LOCTEXT_NAMESPACE
