// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithActorImporter.h"

#include "DatasmithCameraImporter.h"
#include "DatasmithCloth.h"  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
#include "DatasmithImportContext.h"
#include "DatasmithImporterModule.h"
#include "DatasmithImportOptions.h"
#include "DatasmithLandscapeImporter.h"
#include "DatasmithLightImporter.h"
#include "DatasmithMaterialExpressions.h"
#include "DatasmithSceneActor.h"
#include "DatasmithSceneFactory.h"
#include "Engine/Blueprint.h"
#include "IDatasmithSceneElements.h"

#include "ObjectTemplates/DatasmithActorTemplate.h"
#include "ObjectTemplates/DatasmithDecalComponentTemplate.h"
#include "ObjectTemplates/DatasmithSceneComponentTemplate.h"
#include "ObjectTemplates/DatasmithStaticMeshComponentTemplate.h"
#include "Utility/DatasmithImporterImpl.h"
#include "Utility/DatasmithImporterUtils.h"
#include "Utility/DatasmithMeshHelper.h"

#include "ActorFactories/ActorFactoryDeferredDecal.h"
#include "CineCameraActor.h"
#include "CineCameraComponent.h"
#include "Components/DecalComponent.h"
#include "Components/HierarchicalInstancedStaticMeshComponent.h"
#include "DiffUtils.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/DecalActor.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/World.h"
#include "Features/IModularFeatures.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "Misc/Paths.h"
#include "Misc/UObjectToken.h"
#include "ObjectTools.h"
#include "UObject/PropertyPortFlags.h"


#define LOCTEXT_NAMESPACE "DatasmithActorImporter"

namespace UE::DatasmithActorImporter
{
	//
	// For base actors, Datasmith is using actor's label to name scene components.
	// If the label is too long, the scene component's name or the one of it is an outer to
	// may exceed the NAME_SIZE limit and cause a crash.
	// The deepest path would end up being:
	// /Game/[CONTENT_FOLDER_PATH]/[LevelName].[LevelName]:PersistentLevel.[SceneComponentName].DatasmithAssetUserData_0.DatasmithActorTemplate_0
	// To avoid such crash, the scene component's name will be truncated to allow the append of ".DatasmithAssetUserData_0.DatasmithActorTemplate_0"
	// In case a potential crash is identified,the scene component's name will be composed of at max
	// the first 50 characters of the desired name followed by a text indicating it has been truncated and terminated with
	// at max the 50 last characters of the desired name
	FString ComputeSceneComponentName( const UObject& Outer, const FString& DesiredName, FDatasmithImportContext& ImportContext )
	{
		FString PathFullName = Outer.GetFullName() / DesiredName;
		constexpr int32 ExpectedSuffixMaxLength = 51; // Length of ".DatasmithAssetUserData_0.DatasmithActorTemplate_0"
		int32 PathLength = PathFullName.Len() + ExpectedSuffixMaxLength;
		constexpr int32 PreventiveMaximumLength = 900;
		if ( PathLength < PreventiveMaximumLength)
		{
			return DesiredName;
		}

		// Known final path length = Outer's path length + 1 + length of ".DatasmithAssetUserData_0.DatasmithActorTemplate_0" + length of included warning text
		constexpr int32 WarningtextLength = 23; // Length of warning text "---NAME---TRUNCATED---" inserted in new name
		const int32 KnownPathLength = Outer.GetFullName().Len() + ExpectedSuffixMaxLength + WarningtextLength + 1;
		const int32 RemainingLength = FMath::Min(PreventiveMaximumLength - KnownPathLength, 100);

		FString ComputedName = FString::ConstructFromPtrSize(*DesiredName, RemainingLength / 2);
		ComputedName += TEXT("---NAME---TRUNCATED---");
		ComputedName += DesiredName.Right(RemainingLength / 2);

		ImportContext.LogWarning(FText::Format(LOCTEXT("ComputeSceneComponentName", "Detected potential long name, {0}, for scene component. Truncating..."), FText::FromString(DesiredName)));

		return ComputedName;
	}
}

AActor* FDatasmithActorImporter::ImportActor( UClass* ActorClass, const TSharedRef< IDatasmithActorElement >& ActorElement, FDatasmithImportContext& ImportContext, EDatasmithImportActorPolicy ImportActorPolicy,
	TFunction< void( AActor* ) > PostSpawnFunc )
{
	using namespace UE::DatasmithActorImporter;

	if ( ImportActorPolicy == EDatasmithImportActorPolicy::Ignore )
	{
		return nullptr;
	}

	if ( ActorElement->GetScale().IsNearlyZero() )
	{
		ImportContext.LogError( FText::Format( LOCTEXT("ImportActorFail_Scale", "Failed to import actor \"{0}\", scale is zero"), FText::FromString( ActorElement->GetLabel() ) ) );
		return nullptr;
	}

	check(ImportContext.ActorsContext.ImportWorld);

	AActor* ExistingActor = nullptr;

	ADatasmithSceneActor* DatasmithSceneActor = ImportContext.ActorsContext.ImportSceneActor;
	if ( DatasmithSceneActor )
	{
		TSoftObjectPtr< AActor >* RelatedActor = DatasmithSceneActor->RelatedActors.Find( ActorElement->GetName() );

		if ( RelatedActor )
		{
			ExistingActor = RelatedActor->Get();

			// If there's an entry in the RelatedActors maps but it's null, it means that it was deleted by the user. Skip the import unless we're in "full" import mode.
			if ( ExistingActor == nullptr && ImportActorPolicy != EDatasmithImportActorPolicy::Full )
			{
				return nullptr;
			}
		}
	}

	// Destroy the existing actor if it's not of the right class. Child classes are fine except child classes of AActor since it's too generic and might not have the proper component.
	if ( ExistingActor && ( ( ActorClass == AActor::StaticClass() && ExistingActor->GetClass() != ActorClass ) || !ExistingActor->GetClass()->IsChildOf( ActorClass ) ) )
	{
		FDatasmithImporterUtils::DeleteActor( *ExistingActor );
		ExistingActor = nullptr;
	}

	AActor* ImportedActor = ExistingActor;

	if ( ImportedActor == nullptr )
	{
		ImportedActor = ImportContext.ActorsContext.ImportWorld->SpawnActor( ActorClass, nullptr, nullptr );

		auto* ObjectTemplates = FDatasmithObjectTemplateUtils::FindOrCreateObjectTemplates( ImportedActor->GetRootComponent() );
		if ( ObjectTemplates )
		{
			ObjectTemplates->Reset();
		}

		if ( PostSpawnFunc )
		{
			PostSpawnFunc( ImportedActor );
		}

		ImportedActor->SetActorLabel( ActorElement->GetLabel() );
	}
	// Update label of actor if differs from the label of the existing one
	// We will sanitize it for duplicate labels only in FDatasmithImporter::FinalizeActor, as that
	// allows us to better handle reimport scenarios
	else if ( ExistingActor != nullptr && ExistingActor->GetActorLabel() != ActorElement->GetLabel() )
	{
		ImportedActor->SetActorLabel( ActorElement->GetLabel() );
	}

	if ( ImportedActor )
	{
		USceneComponent* RootComponent = ImportedActor->GetRootComponent();

		if ( !RootComponent )
		{
			FString ComponentName = ComputeSceneComponentName( *ImportedActor, ActorElement->GetLabel(), ImportContext );

			RootComponent = NewObject< USceneComponent >( ImportedActor, USceneComponent::StaticClass(), *ComponentName, RF_Transactional );

			ImportedActor->AddInstanceComponent( RootComponent );
			ImportedActor->SetRootComponent( RootComponent );
		}

		// We'll be returning the root component as unregistered since the import process is not complete at this point in most cases
		// Needs to be done after SetActorLabel because SetActorLabel is ambitious and registers the components
		if ( ImportedActor->GetRootComponent() && ImportedActor->GetRootComponent()->IsRegistered() )
		{
			ImportedActor->GetRootComponent()->UnregisterComponent();
		}

		SetupActorProperties( ImportedActor, ActorElement, ImportContext ) ;
		SetupSceneComponent( ImportedActor->GetRootComponent(), ActorElement, ImportContext.Hierarchy.Num() > 0 ? ImportContext.Hierarchy.Top() : nullptr );

		ImportContext.AddImportedActor( ImportedActor );

		if ( DatasmithSceneActor )
		{
			DatasmithSceneActor->RelatedActors.FindOrAdd( ActorElement->GetName() ) = ImportedActor;
		}
	}

	return ImportedActor;
}

USceneComponent* FDatasmithActorImporter::ImportSceneComponent( UClass* ComponentClass, const TSharedRef< IDatasmithActorElement >& ActorElement, FDatasmithImportContext& ImportContext, UObject* Outer, FDatasmithActorUniqueLabelProvider& UniqueNameProvider )
{
	using namespace UE::DatasmithActorImporter;

	if ( !ComponentClass->IsChildOf( USceneComponent::StaticClass() ) )
	{
		ensure( false );
		return nullptr;
	}

	if ( ActorElement->GetScale().IsNearlyZero() )
	{
		ImportContext.LogError( FText::Format( LOCTEXT("ImportActorFail_Scale", "Failed to import actor \"{0}\", scale is zero"), FText::FromString( ActorElement->GetLabel() ) ) );
		return nullptr;
	}

	AActor* Actor = Cast< AActor >( Outer );
	if (!ensure(Actor))
	{
		return nullptr;
	}

	// This is possibly the SceneComponent we are looking for as the existing component
	USceneComponent* SceneComponent = static_cast< USceneComponent* >( FindObjectWithOuter( Outer, ComponentClass, ActorElement->GetLabel() ) );

	// This is scene component that we are looking for as the existing component
	USceneComponent* ValidSceneComponent = nullptr;
	if ( SceneComponent )
	{

		// Validate that the scene component might be the existing component
		FName ComponentDatasmithId = FDatasmithImporterUtils::GetDatasmithElementId(SceneComponent);
		if ( ComponentDatasmithId.IsEqual( FName( ActorElement->GetName() ) ) )
		{
			ValidSceneComponent = SceneComponent;
		}

		// Look at components of the actor, we might find the scene component we are looking for.
		if ( !ValidSceneComponent && Actor )
		{
			for ( UActorComponent* Component : Actor->GetComponents() )
			{
				if (Component && Component->IsA(ComponentClass))
				{
					ComponentDatasmithId = FDatasmithImporterUtils::GetDatasmithElementId( Component );
					if ( ComponentDatasmithId.IsEqual( FName( ActorElement->GetName() ) ) )
					{
						ValidSceneComponent = static_cast< USceneComponent* >( Component );
						break;
					}
				}
			}
		}
	}

	if ( !ValidSceneComponent )
	{
		FName ComponentName( *ComputeSceneComponentName( *Outer, ActorElement->GetLabel(), ImportContext ) );
		if ( SceneComponent || FindObjectWithOuter( Outer, UObject::StaticClass(), ComponentName ) )
		{
			// There is already a object with this name inside the outer. Generate unique name.
			UniqueNameProvider.AddExistingName( ComponentName.ToString() );
			ComponentName = *UniqueNameProvider.GenerateUniqueName( ComponentName.ToString() );
		}
		ValidSceneComponent = NewObject< USceneComponent >( Outer, ComponentClass, ComponentName, RF_Transactional );
	}

	if ( !ValidSceneComponent )
	{
		return nullptr;
	}

	SetupSceneComponent( ValidSceneComponent, ActorElement, ImportContext.Hierarchy.Num() > 0 ? ImportContext.Hierarchy.Top() : nullptr );

	if ( Actor )
	{
		Actor->AddInstanceComponent( ValidSceneComponent );
	}

	return ValidSceneComponent;
}

AActor* FDatasmithActorImporter::ImportBaseActor( FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithActorElement >& ActorElement )
{
	AActor* Actor = ImportActor( AActor::StaticClass(), ActorElement, ImportContext, ImportContext.Options->OtherActorImportPolicy );

	if ( !Actor )
	{
		return nullptr;
	}

	Actor->SpriteScale = 0.1f;

	if ( USceneComponent* RootComponent = Actor->GetRootComponent() )
	{
		RootComponent->bVisualizeComponent = true;
		RootComponent->RegisterComponent();
	}

	return Actor;
}

USceneComponent* FDatasmithActorImporter::ImportBaseActorAsComponent(FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithActorElement >& ActorElement, UObject* Outer, FDatasmithActorUniqueLabelProvider& UniqueNameProvider)
{
	if ( ImportContext.Options->OtherActorImportPolicy == EDatasmithImportActorPolicy::Ignore )
	{
		return nullptr;
	}

	USceneComponent* SceneComponent = FDatasmithActorImporter::ImportSceneComponent( USceneComponent::StaticClass(), ActorElement, ImportContext, Outer, UniqueNameProvider );
	SceneComponent->RegisterComponent();

	ImportContext.AddSceneComponent(SceneComponent->GetName(), SceneComponent);

	return SceneComponent;
}

AStaticMeshActor* FDatasmithActorImporter::ImportStaticMeshActor(FDatasmithImportContext & ImportContext, const TSharedRef<IDatasmithMeshActorElement>& MeshActorElement)
{
	AActor* ImportedActor = ImportActor( AStaticMeshActor::StaticClass(), MeshActorElement, ImportContext, ImportContext.Options->StaticMeshActorImportPolicy );

	AStaticMeshActor* StaticMeshActor = Cast< AStaticMeshActor >(ImportedActor);

	if ( !StaticMeshActor )
	{
		return nullptr;
	}

	SetupStaticMeshComponent( ImportContext, StaticMeshActor->GetStaticMeshComponent(), MeshActorElement );

	return StaticMeshActor;
}

UStaticMeshComponent* FDatasmithActorImporter::ImportStaticMeshComponent( FDatasmithImportContext& ImportContext, const TSharedRef<IDatasmithMeshActorElement>& InMeshActor, UObject* Outer, FDatasmithActorUniqueLabelProvider& UniqueNameProvider )
{
	if ( ImportContext.Options->StaticMeshActorImportPolicy == EDatasmithImportActorPolicy::Ignore )
	{
		return nullptr;
	}

	USceneComponent* SceneComponent =  FDatasmithActorImporter::ImportSceneComponent( UStaticMeshComponent::StaticClass(), InMeshActor, ImportContext, Outer, UniqueNameProvider );
	UStaticMeshComponent* StaticMeshComponent = Cast< UStaticMeshComponent >( SceneComponent );

	SetupStaticMeshComponent( ImportContext, StaticMeshComponent, InMeshActor );

	return StaticMeshComponent;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
AActor* FDatasmithActorImporter::ImportClothActor(FDatasmithImportContext& ImportContext, const TSharedRef<IDatasmithClothActorElement>& ClothActorElement)  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
{
	AActor* ImportedActor = ImportActor(AActor::StaticClass(), ClothActorElement, ImportContext, ImportContext.Options->StaticMeshActorImportPolicy);

	if (ImportedActor)
	{
		// Find imported resource
		const TCHAR* ClothName = ClothActorElement->GetCloth();
		UObject* ClothAsset = nullptr;

		// search in imported resources
		for (const auto& ImportedAssetPair : ImportContext.ImportedClothes) // Algo::Find
		{
			if (FCString::Stricmp(ImportedAssetPair.Key->GetName(), ClothName) == 0)
			{
				ClothAsset = ImportedAssetPair.Value;
				break;
			}
		}

		// search in existing scene asset
		if (ClothAsset == nullptr)
		{
			check(ImportContext.SceneAsset);
			if (TSoftObjectPtr<UObject>* ClothAssetPtr = ImportContext.SceneAsset->Clothes.Find(FName(ClothName)))
			{
				ClothAsset = ClothAssetPtr->LoadSynchronous();
			}
		}

		// Locate a cloth factory provider
		const TArray<IDatasmithClothFactoryClassesProvider*> ClothFactoryClassesProviders = IModularFeatures::Get().GetModularFeatureImplementations<IDatasmithClothFactoryClassesProvider>(IDatasmithClothFactoryClassesProvider::FeatureName);
		const IDatasmithClothFactoryClassesProvider* const ClothFactoryClassesProvider = ClothFactoryClassesProviders.Num() ? ClothFactoryClassesProviders[0] : nullptr;
		const UDatasmithClothComponentFactory* const ClothComponentFactory = ClothFactoryClassesProvider ? ClothFactoryClassesProvider->GetClothComponentFactoryClass().GetDefaultObject() : nullptr;
		
		// Create a new cloth component
		if (USceneComponent* const ClothComponent = ClothComponentFactory ? ClothComponentFactory->CreateClothComponent(ImportedActor) : nullptr)
		{
			ClothComponentFactory->InitializeClothComponent(ClothComponent, ClothAsset, ImportedActor->GetRootComponent());

			ImportedActor->AddInstanceComponent(ClothComponent);

			ClothComponent->bVisualizeComponent = true;
			ClothComponent->RegisterComponent();
		}
		else
		{
			ImportContext.LogError(FText::Format(LOCTEXT("ClothComponentCreationFailure", "Cannot create cloth component for asset {0} required by actor {1}."), FText::FromString(ClothName), FText::FromString(ClothActorElement->GetLabel())));
		}

		if (ClothAsset == nullptr)
		{
			ImportContext.LogError(FText::Format(LOCTEXT("MissingClothAsset", "Cannot find cloth asset {0} for actor {1}."), FText::FromString(ClothName), FText::FromString(ClothActorElement->GetLabel())));
		}
	}
	return ImportedActor;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FDatasmithActorImporter::SetupStaticMeshComponent( FDatasmithImportContext& ImportContext, UStaticMeshComponent* StaticMeshComponent, const TSharedRef< IDatasmithMeshActorElement >& MeshActorElement )
{
	if ( !StaticMeshComponent )
	{
		ImportContext.LogError( FText::Format( LOCTEXT( "MissingStaticMeshComponent", "{0} has no Static Mesh Component."), FText::FromString( MeshActorElement->GetLabel() ) ) );
		return;
	}

	UStaticMesh* StaticMesh = FDatasmithImporterUtils::FindAsset< UStaticMesh >( ImportContext.AssetsContext, MeshActorElement->GetStaticMeshPathName() );

	if ( !StaticMesh )
	{
		FString OwnerLabel;

		if ( StaticMeshComponent->GetOwner() )
		{
			OwnerLabel = StaticMeshComponent->GetOwner()->GetActorLabel();
			if (OwnerLabel.IsEmpty())
			{
				OwnerLabel = StaticMeshComponent->GetOwner()->GetName();
			}
		}

		ImportContext.LogError( FText::Format( LOCTEXT( "FindStaticMesh", "Cannot find Static Mesh {0} for Static Mesh Actor {1}."),
			FText::FromString(MeshActorElement->GetLabel()),
			FText::FromString(OwnerLabel + TEXT(".") + StaticMeshComponent->GetName())));

		return;
	}

	UDatasmithStaticMeshComponentTemplate* StaticMeshComponentTemplate = NewObject< UDatasmithStaticMeshComponentTemplate >( StaticMeshComponent );
	StaticMeshComponentTemplate->StaticMesh = StaticMesh;

	OverrideStaticMeshActorMaterials( ImportContext, MeshActorElement, StaticMesh, StaticMeshComponentTemplate );

	StaticMeshComponentTemplate->Apply( StaticMeshComponent );
	StaticMeshComponent->RegisterComponent();
}

AActor* FDatasmithActorImporter::ImportCameraActor(FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithCameraActorElement >& InCameraActor)
{
	if (ImportContext.Options->CameraImportPolicy == EDatasmithImportActorPolicy::Ignore)
	{
		return nullptr;
	}

	return FDatasmithCameraImporter::ImportCameraActor(InCameraActor, ImportContext);
}

AActor* FDatasmithActorImporter::ImportCustomActor(FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithCustomActorElement >& InCustomActorElement, FDatasmithActorUniqueLabelProvider& UniqueNameProvider)
{
	if ( ImportContext.Options->OtherActorImportPolicy == EDatasmithImportActorPolicy::Ignore )
	{
		return nullptr;
	}

	UClass* ActorClass = nullptr;

	const bool bIsValidClassOrPathName = !FPackageName::IsShortPackageName( FString( InCustomActorElement->GetClassOrPathName() ) ); // FSoftObjectPath doesn't support short package names

	if ( bIsValidClassOrPathName )
	{
		UBlueprint* Blueprint = FDatasmithImporterUtils::FindObject< UBlueprint >( nullptr, InCustomActorElement->GetClassOrPathName() );

		if ( Blueprint )
		{
			ActorClass = Blueprint->GeneratedClass;
		}
	}

	if ( !ActorClass )
	{
		if ( bIsValidClassOrPathName )
		{
			FSoftClassPath ActorClassPath( InCustomActorElement->GetClassOrPathName() );
			ActorClass = ActorClassPath.TryLoadClass< AActor >();
		}

		if ( !ActorClass )
		{
			ImportContext.LogError( FText::Format(LOCTEXT("MissingClass", "Cannot find Class {0} to spawn actor {1}. An empty actor will be spawned instead."),
				FText::FromString( InCustomActorElement->GetClassOrPathName() ), FText::FromString( InCustomActorElement->GetName() )) );

			return ImportBaseActor( ImportContext, InCustomActorElement ); // If we couldn't find the Blueprint, import an empty actor as a place holder
		}
	}

	AActor* Actor = FDatasmithActorImporter::ImportActor( ActorClass, InCustomActorElement, ImportContext, ImportContext.Options->OtherActorImportPolicy );

	if ( !Actor )
	{
		return nullptr;
	}

	if ( !Actor->GetRootComponent() )
	{
		USceneComponent* RootComponent = FDatasmithActorImporter::ImportSceneComponent( USceneComponent::StaticClass(), InCustomActorElement, ImportContext, Actor, UniqueNameProvider );
		RootComponent->bVisualizeComponent = true;

		Actor->SetRootComponent( RootComponent );
	}

	for ( int32 i = 0; i < InCustomActorElement->GetPropertiesCount(); ++i )
	{
		const TSharedPtr< IDatasmithKeyValueProperty >& KeyValueProperty = InCustomActorElement->GetProperty( i );

		FString PropStr = KeyValueProperty->GetName();

		TArray<FString> PropNamesStr;
		PropStr.ParseIntoArray(PropNamesStr, TEXT("."));

		TArray<FName> PropNames;
		for (auto&& Str : PropNamesStr)
		{
			PropNames.Add(FName(*Str));
		}

		FPropertySoftPath PropertyPath(PropNames);
		FResolvedProperty ResolvedProperty = PropertyPath.Resolve(Actor);

		void* Container = const_cast<void*>(ResolvedProperty.Object);
		if (!ResolvedProperty.Property || !Container)
		{
			ImportContext.LogWarning(FText::Format(LOCTEXT("BadPropertyKey", "Cannot find Property '{0}' on actor {1}.")
				, FText::FromString(PropStr)
				, FText::FromString(InCustomActorElement->GetName() ))
			);
			continue;
		}

		const TCHAR* Remaining = ResolvedProperty.Property->ImportText_InContainer( KeyValueProperty->GetValue(), Container, nullptr, PPF_None );
		if (Remaining && Remaining[0] != 0)
		{
			ImportContext.LogWarning(FText::Format(LOCTEXT("BadPropertyValue", "Cannot properly load value '{0}' on property '{1}' for actor {2}.")
				, FText::FromStringView(KeyValueProperty->GetValue())
				, FText::FromString(PropStr)
				, FText::FromStringView(InCustomActorElement->GetName()))
			);
		}
	}

	if ( Actor->GetRootComponent() )
	{
		Actor->GetRootComponent()->RegisterComponent();
	}

	Actor->RerunConstructionScripts();

	return Actor;
}

AActor* FDatasmithActorImporter::ImportLandscapeActor(FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithLandscapeElement >& InLandscapeActorElement)
{
	if (ImportContext.Options->OtherActorImportPolicy == EDatasmithImportActorPolicy::Ignore)
	{
		return nullptr;
	}

	return FDatasmithLandscapeImporter::ImportLandscapeActor( InLandscapeActorElement, ImportContext, ImportContext.Options->OtherActorImportPolicy );
}

AActor* FDatasmithActorImporter::ImportLightActor(FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithLightActorElement >& InLightElement)
{
	if (ImportContext.Options->LightImportPolicy == EDatasmithImportActorPolicy::Ignore)
	{
		return nullptr;
	}

	return FDatasmithLightImporter::ImportLightActor( InLightElement, ImportContext );
}

AActor* FDatasmithActorImporter::ImportEnvironment(FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithEnvironmentElement >& InEnvironmentElement)
{
	AActor* Actor = nullptr;

	TSharedPtr< IDatasmithShaderElement > ShaderElement = FDatasmithSceneFactory::CreateShader(TEXT("ImageBasedEnvironmentMaterial"));

	if ( InEnvironmentElement->GetEnvironmentComp()->GetParamSurfacesCount() > 0 )
	{
		ShaderElement->SetEmitTexture(InEnvironmentElement->GetEnvironmentComp()->GetParamTexture(0));
		ShaderElement->SetEmitTextureSampler(InEnvironmentElement->GetEnvironmentComp()->GetParamTextureSampler(0));
	}

	if (InEnvironmentElement->GetIsIlluminationMap() == false)
	{
		InEnvironmentElement->SetScale( FVector(100.f, 100.f, 100.f) );
		InEnvironmentElement->SetRotation( FQuat::MakeFromEuler(FVector(0.f, 0.f, 360.f * ShaderElement->GetEmitTextureSampler().Rotation)) );

		Actor = FDatasmithActorImporter::ImportActor( AStaticMeshActor::StaticClass(), InEnvironmentElement, ImportContext, ImportContext.Options->LightImportPolicy );

		AStaticMeshActor* EnvironmentActor = Cast< AStaticMeshActor >( Actor );

		if ( !EnvironmentActor )
		{
			return nullptr;
		}

		FSoftObjectPath EditorSphereMeshPath( TEXT("StaticMesh'/Engine/EditorMeshes/EditorSphere.EditorSphere'") );

		UStaticMesh* EditorSphereMesh = Cast< UStaticMesh >( EditorSphereMeshPath.TryLoad() );
		if ( EditorSphereMesh != nullptr )
		{
			if ( EnvironmentActor->GetStaticMeshComponent() == nullptr )
			{
				ImportContext.LogError( FText::Format( LOCTEXT( "MissingStaticMeshComponent", "{0} has no Static Mesh Component." ), FText::FromName( EditorSphereMesh->GetFName() ) ) );
				return nullptr;
			}

			EnvironmentActor->GetStaticMeshComponent()->SetStaticMesh( EditorSphereMesh );
			EnvironmentActor->GetStaticMeshComponent()->bAffectDynamicIndirectLighting = false;
			EnvironmentActor->GetStaticMeshComponent()->bAffectDistanceFieldLighting = false;
			EnvironmentActor->GetStaticMeshComponent()->bCastDynamicShadow = false;
			EnvironmentActor->GetStaticMeshComponent()->bCastStaticShadow = false;

			UMaterial* ExistingMaterial = nullptr;
			if ( ImportContext.Options->SearchPackagePolicy == EDatasmithImportSearchPackagePolicy::All )
			{
				ExistingMaterial = FindFirstObject<UMaterial>(ShaderElement->GetName(), EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("FDatasmithActorImporter::ImportEnvironment"));
			}
			else
			{
				ExistingMaterial = FindObject<UMaterial>(ImportContext.AssetsContext.MaterialsFinalPackage.Get(), ShaderElement->GetName());
			}

			UPackage* Package = CreatePackage( *FPaths::Combine( ImportContext.AssetsContext.MaterialsFinalPackage.Get()->GetPathName(), ShaderElement->GetName() ) );
			UMaterialInterface* Material = FDatasmithMaterialExpressions::CreateDatasmithEnvironmentMaterial(Package, ShaderElement, ImportContext.AssetsContext, ExistingMaterial);
			if (Material)
			{
				EnvironmentActor->GetStaticMeshComponent()->SetMaterial(0, Material);
			}

			EnvironmentActor->MarkComponentsRenderStateDirty();
			EnvironmentActor->GetStaticMeshComponent()->RegisterComponent();
		}
		else
		{
			UE_LOGF(LogDatasmithImport, Warning, "Cannot load mesh StaticMesh'/Engine/EditorMeshes/EditorSphere.EditorSphere'");
		}
	}
	else
	{
		Actor = FDatasmithLightImporter::CreateHDRISkyLight(ShaderElement, ImportContext);
	}

	return Actor;
}

AActor* FDatasmithActorImporter::ImportHierarchicalInstancedStaticMeshAsActor(FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithHierarchicalInstancedStaticMeshActorElement >& HierarchicalInstancedStaticMeshActorElement, FDatasmithActorUniqueLabelProvider& UniqueNameProvider)
{
	AActor* Actor = ImportActor(AActor::StaticClass(), HierarchicalInstancedStaticMeshActorElement, ImportContext, ImportContext.Options->StaticMeshActorImportPolicy);

	if ( Actor )
	{
		FString OriginalLabel = HierarchicalInstancedStaticMeshActorElement->GetLabel();
		HierarchicalInstancedStaticMeshActorElement->SetLabel(TEXT("HierarchicalInstancedStaticMesh"));
		if (Actor->GetRootComponent())
		{
			ImportContext.Hierarchy.Push(Actor->GetRootComponent());
			ImportHierarchicalInstancedStaticMeshComponent(ImportContext, HierarchicalInstancedStaticMeshActorElement, Actor, UniqueNameProvider);
			ImportContext.Hierarchy.Pop();
		}
		else
		{
			ImportHierarchicalInstancedStaticMeshComponent(ImportContext, HierarchicalInstancedStaticMeshActorElement, Actor, UniqueNameProvider);
		}
		HierarchicalInstancedStaticMeshActorElement->SetLabel(*OriginalLabel);
	}

	return Actor;
}

UHierarchicalInstancedStaticMeshComponent* FDatasmithActorImporter::ImportHierarchicalInstancedStaticMeshComponent(FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithHierarchicalInstancedStaticMeshActorElement >& HierarchicalInstancedStatictMeshActorElement,
	UObject* Outer, FDatasmithActorUniqueLabelProvider& UniqueNameProvider)
{
	if ( ImportContext.Options->StaticMeshActorImportPolicy == EDatasmithImportActorPolicy::Ignore )
	{
		return nullptr;
	}

	USceneComponent* SeneComponent = FDatasmithActorImporter::ImportSceneComponent(UHierarchicalInstancedStaticMeshComponent::StaticClass(), HierarchicalInstancedStatictMeshActorElement, ImportContext, Outer, UniqueNameProvider);
	UHierarchicalInstancedStaticMeshComponent* HierarchilcalInstanciedStaticMeshComponent = Cast< UHierarchicalInstancedStaticMeshComponent >(SeneComponent);

	SetupHierarchicalInstancedStaticMeshComponent(ImportContext, HierarchilcalInstanciedStaticMeshComponent, HierarchicalInstancedStatictMeshActorElement);

	return HierarchilcalInstanciedStaticMeshComponent;
}

void FDatasmithActorImporter::SetupHierarchicalInstancedStaticMeshComponent(FDatasmithImportContext& ImportContext, UHierarchicalInstancedStaticMeshComponent* HierarchicalInstancedStaticMeshComponent,
	const TSharedRef< IDatasmithHierarchicalInstancedStaticMeshActorElement >& HierarchicalInstancedStatictMeshActorElement)
{
	if (!HierarchicalInstancedStaticMeshComponent)
	{
		ImportContext.LogError(FText::Format(LOCTEXT("MissingHierarchicalInstancedStaticMeshComponent", "{0} has no Hierarchical Instanced Static Mesh Component."), FText::FromString(HierarchicalInstancedStatictMeshActorElement->GetLabel())));
		return;
	}

	const bool bAutoRebuildTreeOnInstanceChanges = HierarchicalInstancedStaticMeshComponent->bAutoRebuildTreeOnInstanceChanges;
	HierarchicalInstancedStaticMeshComponent->bAutoRebuildTreeOnInstanceChanges = false;

	HierarchicalInstancedStaticMeshComponent->ClearInstances();

	int32 InstanceCount = HierarchicalInstancedStatictMeshActorElement->GetInstancesCount();
	bool bContainsInvertedMeshes = false;

	for (int32 i = 0; i < InstanceCount; i++)
	{
		FTransform Instance = HierarchicalInstancedStatictMeshActorElement->GetInstance(i);
		HierarchicalInstancedStaticMeshComponent->AddInstance(Instance);

		FVector InstanceScale = Instance.GetScale3D();
		if ((InstanceScale.X * InstanceScale.Y * InstanceScale.Z) < 0)
		{
			bContainsInvertedMeshes = true;
		}
	}

	if (bContainsInvertedMeshes)
	{
		ImportContext.LogWarning(FText::GetEmpty())
			->AddToken(FUObjectToken::Create(HierarchicalInstancedStaticMeshComponent))
			->AddToken(FTextToken::Create(FText::Format(LOCTEXT("HierarchicalInstancedStaticMeshComponentHasInvertedScale", "{0} has instances with negative scaling producing unsupported inverted meshes."), FText::FromString(HierarchicalInstancedStatictMeshActorElement->GetLabel()))));
	}

	SetupStaticMeshComponent(ImportContext, HierarchicalInstancedStaticMeshComponent, HierarchicalInstancedStatictMeshActorElement);

	HierarchicalInstancedStaticMeshComponent->bAutoRebuildTreeOnInstanceChanges = bAutoRebuildTreeOnInstanceChanges;
}

void FDatasmithActorImporter::SetupActorProperties(AActor* ImportedActor, const TSharedRef< IDatasmithActorElement >& ActorElement, FDatasmithImportContext& ImportContext)
{
	if (ImportedActor == nullptr || ImportedActor->GetRootComponent() == nullptr)
	{
		return;
	}
	UDatasmithActorTemplate* NewActorTemplate = NewObject<UDatasmithActorTemplate>(ImportedActor);

	// Import in template
	NewActorTemplate->Layers = ParseCsvLayers(ActorElement->GetLayer());
	int32 TagsCount = ActorElement->GetTagsCount();
	for (int32 i = 0; i < TagsCount; i++)
	{
		NewActorTemplate->Tags.Add(ActorElement->GetTag(i));
	}

	NewActorTemplate->Apply(ImportedActor);

	// Make sure all used layers exist
	FDatasmithImporterUtils::AddUniqueLayersToWorld( ImportContext.ActorsContext.ImportWorld , TSet< FName >( ImportedActor->Layers ) );
}

void FDatasmithActorImporter::SetupSceneComponent( USceneComponent* SceneComponent, const TSharedRef< IDatasmithActorElement >& ActorElement, USceneComponent* Parent )
{
	static_assert((uint8)EDatasmithActorMobilityType::Static == (uint8)EComponentMobility::Type::Static, "ENUM_VALUE_HAS_CHANGED");
	static_assert((uint8)EDatasmithActorMobilityType::Stationary == (uint8)EComponentMobility::Type::Stationary, "ENUM_VALUE_HAS_CHANGED");
	static_assert((uint8)EDatasmithActorMobilityType::Movable == (uint8)EComponentMobility::Type::Movable, "ENUM_VALUE_HAS_CHANGED");

	if ( !SceneComponent )
	{
		return;
	}

	UObject* Outer = SceneComponent->GetOwner(); // Actor is used as Outer because the RootSceneComponent was not properly serialized on some Blueprint instance (see UE-61561)
	UDatasmithSceneComponentTemplate* SceneComponentTemplate = NewObject< UDatasmithSceneComponentTemplate >(Outer);

	SceneComponentTemplate->RelativeTransform = ActorElement->GetRelativeTransform();
	SceneComponentTemplate->Mobility = (EComponentMobility::Type)ActorElement->GetMobility();
	SceneComponentTemplate->bVisible = ActorElement->GetVisibility();
	SceneComponentTemplate->bCastShadow = ActorElement->GetCastShadow();
	SceneComponentTemplate->AttachParent = Parent;

	// Add tags from ActorElement to SceneComponentTemplate
	int32 TagsCount = ActorElement->GetTagsCount();
	for (int32 i = 0; i < TagsCount; i++)
	{
		SceneComponentTemplate->Tags.Add( ActorElement->GetTag(i) );
	}

	SceneComponentTemplate->Apply( SceneComponent );
}

TSet<FName> FDatasmithActorImporter::ParseCsvLayers(const TCHAR* CsvLayersNames)
{
	TSet<FName> Names;

	FString CsvString(CsvLayersNames);
	if (!CsvString.IsEmpty() && CsvString != TEXT("0")) // For legacy reasons, "0" means no layer
	{
		TArray<FString> NameArray;
		CsvString.ParseIntoArray(NameArray, TEXT(","), true);
		for (const FString& Name : NameArray)
		{
			Names.Add(FName(*Name));
		}
	}

	return Names;
}

void FDatasmithActorImporter::OverrideStaticMeshActorMaterials( const FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithMeshActorElement >& MeshActorElement, const UStaticMesh* StaticMesh,
	UDatasmithStaticMeshComponentTemplate* StaticMeshComponentTemplate )
{
	for (int32 i = 0; i < MeshActorElement->GetMaterialOverridesCount(); ++i) //original sub material
	{
		const TSharedRef< IDatasmithMaterialIDElement >& OriginalSubMaterial = MeshActorElement->GetMaterialOverride(i).ToSharedRef();

		 // if the material id is < 0, apply the material to all the material slots
		if (OriginalSubMaterial->GetId() < 0)
		{
			for (int32 MeshSubMaterialIdx = 0; MeshSubMaterialIdx < StaticMesh->GetStaticMaterials().Num(); MeshSubMaterialIdx++)
			{
				OverrideStaticMeshActorMaterial( ImportContext, OriginalSubMaterial, StaticMeshComponentTemplate, MeshSubMaterialIdx );
			}
		}
		else
		{
			const FName SlotName = DatasmithMeshHelper::DefaultSlotName( OriginalSubMaterial->GetId() );

			int32 MeshSubMaterialIdx = StaticMesh->GetMaterialIndex( SlotName );

			// if we failed to find a material slot named SlotName, use material id as the material slot index
			if ( MeshSubMaterialIdx < 0 )
			{
				MeshSubMaterialIdx = OriginalSubMaterial->GetId();
			}

			// Override material only if index of partition does exist in static mesh
			if( ensure( StaticMesh->GetStaticMaterials().IsValidIndex( MeshSubMaterialIdx ) ) )
			{
				OverrideStaticMeshActorMaterial( ImportContext, OriginalSubMaterial, StaticMeshComponentTemplate, MeshSubMaterialIdx );
			}
		}
	}
}

void FDatasmithActorImporter::OverrideStaticMeshActorMaterial( const FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithMaterialIDElement >& SubMaterial,
	UDatasmithStaticMeshComponentTemplate* StaticMeshComponentTemplate, int32 MeshSubMaterialIdx )
{
	if ( MeshSubMaterialIdx >= 0 )
	{
		UMaterialInterface* Material = FDatasmithImporterUtils::FindAsset< UMaterialInterface >( ImportContext.AssetsContext, SubMaterial->GetName() );

		if ( Material )
		{
			if ( !StaticMeshComponentTemplate->OverrideMaterials.IsValidIndex( MeshSubMaterialIdx ) )
			{
				StaticMeshComponentTemplate->OverrideMaterials.AddDefaulted( ( MeshSubMaterialIdx - StaticMeshComponentTemplate->OverrideMaterials.Num() ) + 1 );
			}

			StaticMeshComponentTemplate->OverrideMaterials[ MeshSubMaterialIdx ] = Material;
		}
	}
}

AActor* FDatasmithActorImporter::ImportDecalActor(FDatasmithImportContext& ImportContext, const TSharedRef<IDatasmithDecalActorElement>& DecalActorElement, FDatasmithActorUniqueLabelProvider& UniqueNameProvider)
{
	if (ImportContext.Options->OtherActorImportPolicy == EDatasmithImportActorPolicy::Ignore)
	{
		return nullptr;
	}

	AActor* Actor = FDatasmithActorImporter::ImportActor( ADecalActor::StaticClass(), DecalActorElement, ImportContext, ImportContext.Options->OtherActorImportPolicy );

	if ( ADecalActor* DecalActor = Cast< ADecalActor >( Actor ) )
	{
		UDecalComponent* DecalComponent = Cast<UDecalComponent>(DecalActor->FindComponentByClass(UDecalComponent::StaticClass()));
		check(DecalComponent);

		DecalComponent->UnregisterComponent();

		FDatasmithAssetsImportContext& AssetsContext = ImportContext.AssetsContext;

		UDatasmithDecalComponentTemplate* DecalTemplate = NewObject< UDatasmithDecalComponentTemplate >( DecalComponent );

		DecalTemplate->SortOrder = DecalActorElement->GetSortOrder();
		DecalTemplate->DecalSize = DecalActorElement->GetDimensions();
		DecalTemplate->Material = FDatasmithImporterUtils::FindAsset< UMaterialInterface >( AssetsContext, DecalActorElement->GetDecalMaterialPathName() );

		DecalTemplate->Apply( DecalComponent );

		// Init Component
		DecalComponent->RegisterComponent();

		DecalActor->UpdateComponentTransforms();
		DecalActor->MarkPackageDirty();

		return DecalActor;
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE