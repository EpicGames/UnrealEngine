// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithBlueprintLibrary.h"

#include "Components/StaticMeshComponent.h"
#include "DatasmithImporterHelper.h"
#include "DatasmithImportFactory.h"
#include "DatasmithImportOptions.h"
#include "DatasmithSceneActor.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithSceneSource.h"
#include "DatasmithStaticMeshImporter.h"
#include "DatasmithTranslatableSource.h"
#include "DatasmithTranslatorManager.h"
#include "ObjectElements/DatasmithUSceneElement.h"
#include "StaticMeshResources.h"
#include "Utility/DatasmithImporterUtils.h"
#include "Utility/DatasmithImportFactoryHelper.h"
#include "Utility/DatasmithMeshHelper.h"

#include "Async/ParallelFor.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "DatasmithAssetImportData.h"
#include "DirectLinkExtensionModule.h"
#include "DirectLinkExternalSource.h"
#include "Editor.h"
#include "EditorAssetLibrary.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/World.h"
#include "ExternalSource.h"
#include "ExternalSourceModule.h"
#include "GameFramework/Actor.h"
#include "HAL/FileManager.h"
#include "IUriManager.h"
#include "MeshExport.h"
#include "Misc/FileHelper.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "PackageTools.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshCompiler.h"
#include "SourceUri.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Templates/Greater.h"
#include "Templates/SharedPointer.h"
#include "UObject/Package.h"
#include "UObject/StrongObjectPtr.h"
#include "UObject/UObjectGlobals.h"


#define LOCTEXT_NAMESPACE "DatasmithBlueprintLibrary"

DEFINE_LOG_CATEGORY_STATIC(LogSetupStaticLighting, Log, All);

namespace DatasmithStaticMeshBlueprintLibraryUtil
{
	void EnsureLightmapSourceUVsAreAvailable( UStaticMesh* StaticMesh )
	{
		if ( StaticMesh->GetNumSourceModels() > 0 && StaticMesh->GetSourceModel(0).BuildSettings.bGenerateLightmapUVs )
		{
			FDatasmithStaticMeshImporter::PreBuildStaticMesh( StaticMesh );
		}
	}

	float ParallelogramArea(FVector v0, FVector v1, FVector v2)
	{
		FVector TriangleNormal = (v1 - v0) ^ (v2 - v0);
		float Area = TriangleNormal.Size();

		return Area;
	}

	//Used for creating a mapping of StaticMeshes and the StaticMeshComponents that references them in the given list.
	TMap< UStaticMesh*, TSet< UStaticMeshComponent* > > GetStaticMeshComponentMap(const TArray< UObject* >& Objects)
	{
		TMap< UStaticMesh*, TSet< UStaticMeshComponent* > > StaticMeshMap;

		for (UObject* Object : Objects)
		{
			if (AActor* Actor = Cast< AActor >(Object))
			{
				TInlineComponentArray<UStaticMeshComponent*> StaticMeshComponents(Actor);
				for (UStaticMeshComponent* StaticMeshComponent : StaticMeshComponents)
				{
					if (StaticMeshComponent && StaticMeshComponent->GetStaticMesh())
					{
						TSet<UStaticMeshComponent*>& Components = StaticMeshMap.FindOrAdd(StaticMeshComponent->GetStaticMesh());
						Components.Add(StaticMeshComponent);
					}
				}
			}
			else if (UStaticMeshComponent* StaticMeshComponent = Cast< UStaticMeshComponent >(Object))
			{
				if (UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh())
				{
					TSet<UStaticMeshComponent*>& Components = StaticMeshMap.FindOrAdd(StaticMesh);
					Components.Add(StaticMeshComponent);
				}
			}
			else if (UStaticMesh* StaticMesh = Cast< UStaticMesh >(Object))
			{
				TSet<UStaticMeshComponent*>& Components = StaticMeshMap.FindOrAdd(StaticMesh);
				Components.Add(nullptr);
			}
		}

		return StaticMeshMap;
	}
}

namespace DatasmithBlueprintLibraryImpl
{
	static FName GetLoggerName() { return FName(TEXT("DatasmithLibrary")); }

	static FText GetDisplayName() { return NSLOCTEXT("DatasmithBlueprintLibrary", "LibraryName", "Datasmith Library"); }

	bool ValidatePackage(FString PackageName, UPackage*& OutPackage, const TCHAR* OutFailureReason)
	{
		OutFailureReason = TEXT("");
		OutPackage = nullptr;

		if (PackageName.IsEmpty())
		{
			OutFailureReason = TEXT("No destination Folder was provided.");
			return false;
		}

		PackageName.ReplaceCharInline(TEXT('\\'), TEXT('/'));
		while(PackageName.ReplaceInline(TEXT("//"), TEXT("/"), ESearchCase::CaseSensitive)) {}
		PackageName = UPackageTools::SanitizePackageName(PackageName);

		while (PackageName.EndsWith(TEXT("/")))
		{
			PackageName = PackageName.LeftChop(1);
		}

		FText Reason;
		if (!FPackageName::IsValidLongPackageName(PackageName, true, &Reason))
		{
			OutFailureReason = TEXT("Invalid long package mame.");
			return false;
		}

		OutPackage = CreatePackage( *PackageName );
		return OutPackage != nullptr;
	}
}

namespace DatasmithSceneElementUtil
{
	void SetDefaultTranslatorOptions(IDatasmithTranslator* Translator)
	{
		static FDatasmithTessellationOptions DefaultTessellationOptions(0.3f, 0.0f, 30.0f, EDatasmithCADStitchingTechnique::StitchingSew);

		TArray< TObjectPtr<UDatasmithOptionsBase> > Options;
		Translator->GetSceneImportOptions(Options);

		bool bUpdateOptions = false;
		for (TObjectPtr<UDatasmithOptionsBase>& ObjectPtr : Options)
		{
			if (UDatasmithCommonTessellationOptions* TessellationOption = Cast<UDatasmithCommonTessellationOptions>(ObjectPtr))
			{
				bUpdateOptions = true;
				TessellationOption->Options = DefaultTessellationOptions;
			}
		}

		if (bUpdateOptions == true)
		{
			Translator->SetSceneImportOptions(Options);
		}
	};

	bool ImportDatasmithSceneFromCADFiles(const FString& DestinationFolder, const TArray<FString>& FilePaths, FDatasmithImportFactoryCreateFileResult& Result, TArray<FString>& FilesNotProcessed)
	{
		using namespace DatasmithBlueprintLibraryImpl;
		using namespace UE::DatasmithImporter;

		// Make a unique filename to store file in the Saved folder for reimport
		FString SaveDir = FPaths::Combine(FPlatformMisc::ProjectDir(), TEXT("Saved"), TEXT("Datasmith"));
		if (!IFileManager::Get().DirectoryExists(*SaveDir))
		{
			IFileManager::Get().MakeDirectory(*SaveDir);
		}
		FString PlmXmlFileName = FPaths::CreateTempFilename(*SaveDir, TEXT("FromFiles"), TEXT(".plmxml"));

		TSet<FString> FilesToProcess(FilePaths);
		if (!FDatasmithImporterUtils::CreatePlmXmlSceneFromCADFiles(PlmXmlFileName, FilesToProcess, FilesNotProcessed))
		{
			return false;
		}

		TSharedPtr<FExternalSource> ExternalSource = IExternalSourceModule::GetOrCreateExternalSource(FSourceUri::FromFilePath(PlmXmlFileName));
		if (!ExternalSource)
		{
			UE_LOGF(LogDatasmithImport, Warning, "Datasmith import error: cannot resolve the external source from Source file or URI. Abort import");
			return false;
		}

		const bool bLoadConfig = false; // don't load values from ini files
		FDatasmithImportContext ImportContext(ExternalSource.ToSharedRef(), bLoadConfig, GetLoggerName(), GetDisplayName());

		UPackage* DestinationPackage;
		const TCHAR* OutFailureReason = TEXT("");
		if (!DatasmithBlueprintLibraryImpl::ValidatePackage(DestinationFolder, DestinationPackage, OutFailureReason))
		{
			UE_LOGF(LogDatasmithImport, Error, "Invalid Destination '%ls': %ls", *DestinationFolder, OutFailureReason);
			return false;
		}

		const bool bSilent = true; // don't pop options window
		ImportContext.InitOptions(nullptr, TOptional<FString>(), bSilent);

		if (TSharedPtr<IDatasmithScene> LoadedScene = ExternalSource->TryLoad())
		{
			ImportContext.InitScene(LoadedScene.ToSharedRef());
		}
		else
		{
			return false;
		}

		// Inlined bool FDatasmithImportContext::SetupDestination to pass DestinationFolder to AssetsContext.ReInit
		// so that all is imported directly into DestinationFolder
		ImportContext.Options->BaseOptions.AssetOptions.PackagePath = FName(*DestinationPackage->GetName());
		ImportContext.FeedbackContext = GWarn;
		ImportContext.FilteredScene = FDatasmithSceneFactory::DuplicateScene(ImportContext.Scene.ToSharedRef());
		ImportContext.SceneName = FDatasmithUtils::SanitizeObjectName(ImportContext.Scene->GetName());
		ImportContext.ObjectFlags = RF_Public | RF_Standalone | RF_Transactional;
		ImportContext.bUserCancelled = false;
		ImportContext.AssetsContext.ReInit(DestinationFolder);
		ImportContext.ActorsContext.ImportWorld = GWorld; // Make sure actors are imported(into the current world)

		if(!ImportContext.ActorsContext.Init())
		{
			return false;
		}

		bool bUserCancelled = false;
		Result.bImportSucceed = DatasmithImportFactoryImpl::ImportDatasmithScene(ImportContext, bUserCancelled);
		Result.bImportSucceed &= !bUserCancelled;

		if (Result.bImportSucceed)
		{
			Result.FillFromImportContext(ImportContext);
		}

		return true;
	}

	TArray<FDatasmithImportFactoryCreateFileResult> ImportDatasmithScenesFromFiles(const FString & DestinationFolder, const TArray<FString> & FilePaths)
	{
		using namespace DatasmithBlueprintLibraryImpl;
		using namespace UE::DatasmithImporter;
		TArray<FDatasmithImportFactoryCreateFileResult> Result;
		Result.SetNum(FilePaths.Num()); // Allocate result for each input file

		UPackage* DestinationPackage;
		const TCHAR* OutFailureReason = TEXT("");
		if (!DatasmithBlueprintLibraryImpl::ValidatePackage(DestinationFolder, DestinationPackage, OutFailureReason))
		{
			UE_LOGF(LogDatasmithImport, Error, "Invalid Destination '%ls': %ls", *DestinationFolder, OutFailureReason);
			return Result;
		}

		TMap<FString, int32> FileNameToFileIndex;
		TMap<FString, int32> ActorLabelInPlmXmlToFileIndex;
		for (int32 FileIndex = 0; FileIndex < FilePaths.Num(); ++FileIndex)
		{
			FString FileName = FilePaths[FileIndex];
			// This Label is expected to be on the corresponding top-level actor in the combined PlmXml scene
			// should be possible to replace with Name instead, since we control how the PlmXml is created and how PlmXml ids are set
			FString ActorLabel = FDatasmithUtils::SanitizeObjectName(FPaths::GetBaseFilename(FileName));
			ActorLabelInPlmXmlToFileIndex.Add(ActorLabel, FileIndex);
			FileNameToFileIndex.Add(FileName, FileIndex);
		}

		// Translate all files as part of PlmXml scene, making use of multiprocessing translation of CAD files(this is done in TranslateScene for PlmXml with help of DatasmithDispatcher)
		FString TempDir = FPaths::Combine(FPaths::ProjectIntermediateDir(), TEXT("DatasmithBlueprintLibraryTemp"));
		if (!IFileManager::Get().DirectoryExists(*TempDir))
		{
			IFileManager::Get().MakeDirectory(*TempDir);
		}
		FString PlmXmlFileName = FPaths::Combine(TempDir, TEXT("ImportScenesFromFiles.plmxml"));

		TSet<FString> FilesToProcess(FilePaths);
		TArray<FString> FilesNotProcessed;
		if (!FDatasmithImporterUtils::CreatePlmXmlSceneFromCADFiles(PlmXmlFileName, FilesToProcess, FilesNotProcessed))
		{
			FilesNotProcessed = FilePaths;
		}
		else
		{
			TSharedPtr<FExternalSource> ExternalSource = IExternalSourceModule::GetOrCreateExternalSource(FSourceUri::FromFilePath(PlmXmlFileName));
			if (!ExternalSource.IsValid() || !ExternalSource->GetAssetTranslator().IsValid())
			{
				// Process all files separately if PlmXml if not translatable
				FilesNotProcessed = FilePaths;
			}
			else
			{
				const TSharedPtr<IDatasmithTranslator> Translator = ExternalSource->GetAssetTranslator();
				SetDefaultTranslatorOptions(Translator.Get());

				const bool bLoadConfig = false; // don't load values from ini files
				// Context for importing assets - assets(static meshes etc) from all files will be imported using single context
				TUniquePtr<FDatasmithImportContext> PlmXmlImportContextPtr = MakeUnique<FDatasmithImportContext>(ExternalSource.ToSharedRef(), bLoadConfig, GetLoggerName(), GetDisplayName());

				PlmXmlImportContextPtr->InitOptions(nullptr, TOptional<FString>(), true);
				PlmXmlImportContextPtr->Options->BaseOptions.SceneHandling = EDatasmithImportScene::AssetsOnly;

				if (TSharedPtr<IDatasmithScene> LoadedScene = ExternalSource->TryLoad())
				{
					TSharedRef<IDatasmithScene> PlmXmlScene = LoadedScene.ToSharedRef();
					PlmXmlImportContextPtr->InitScene(PlmXmlScene);
					// Inlined bool FDatasmithImportContext::SetupDestination
					// Overriding RootPackagePath
					PlmXmlImportContextPtr->Options->BaseOptions.AssetOptions.PackagePath = FName(*DestinationPackage->GetName());
					PlmXmlImportContextPtr->FeedbackContext = GWarn;
					PlmXmlImportContextPtr->FilteredScene = FDatasmithSceneFactory::DuplicateScene(PlmXmlImportContextPtr->Scene.ToSharedRef());
					PlmXmlImportContextPtr->SceneName = FDatasmithUtils::SanitizeObjectName(PlmXmlImportContextPtr->Scene->GetName());
					PlmXmlImportContextPtr->ObjectFlags = RF_Public | RF_Standalone | RF_Transactional;
					PlmXmlImportContextPtr->bUserCancelled = false;
					PlmXmlImportContextPtr->AssetsContext.ReInit(DestinationFolder);

					// Import and build all the assets
					FDatasmithImportContext& AssetsImportContext = *PlmXmlImportContextPtr;
					FDatasmithImporter::ImportTextures(AssetsImportContext);
					FDatasmithImporter::ImportMaterials(AssetsImportContext);
					FDatasmithImporter::ImportStaticMeshes(AssetsImportContext);
					FDatasmithStaticMeshImporter::PreBuildStaticMeshes(AssetsImportContext);

					// Import into separate scene each of the root actors in translated PlmXml that have a corresponding input file
					TArray<TSharedPtr<FDatasmithImportContext>> ImportContexts;
					for (int32 ActorIndex = 0; ActorIndex < PlmXmlScene->GetActorsCount(); ++ActorIndex)
					{
						TSharedPtr<IDatasmithActorElement> RootChildActorElement = PlmXmlScene->GetActor(ActorIndex);
						if (RootChildActorElement->GetChildrenCount() == 0)
						{
							continue;
						}

						// PlmXml structure creates additional actor element, so take first(and only) child
						TSharedPtr<IDatasmithActorElement> ActorElement = RootChildActorElement->GetChild(0);

						FString Label = RootChildActorElement->GetLabel();
						int32* FileIndexPtr = ActorLabelInPlmXmlToFileIndex.Find(Label);
						if (!FileIndexPtr)
						{
							continue;
						}
						int32 FileIndex = *FileIndexPtr;

						// Create scene imitating import of a single file
						TSharedRef<IDatasmithScene> SceneForRootActor = FDatasmithSceneFactory::CreateScene(*Label);
						{
							// #ueent_todo: Besides Actors(that are imported) and Meshes, Metadata, Materials and Textures that Actors reference
							// Need to add
							// - Scene info(host etc)
							// - Level Sequence(not supported by PlmXml or CAD for the moment)
							// - Variant Sets(not supported by PlmXml or CAD for the moment; Additionally PlmXml creates its own, not related to linked CAD files)
							SceneForRootActor->AddActor(ActorElement);
						}

						// Make sure filename set to context is the same that would be used if scene was imported separately
						const FSourceUri SourceUri = FSourceUri::FromFilePath(FilePaths[FileIndex]);
						TSharedPtr<UE::DatasmithImporter::FExternalSource> ActorExternalSource = IExternalSourceModule::GetOrCreateExternalSource(SourceUri);
						if (!ActorExternalSource)
						{
							continue;
						}

						if (const TSharedPtr<IDatasmithTranslator>& TranslatorPtr = ActorExternalSource->GetAssetTranslator())
						{
							SetDefaultTranslatorOptions(TranslatorPtr.Get());
						}

						// Create ImportContext for each separate scene
						TSharedPtr<FDatasmithImportContext> ImportContextPtr = MakeShared<FDatasmithImportContext>(ActorExternalSource.ToSharedRef(), false, GetLoggerName(), GetDisplayName());
						FDatasmithImportContext& ImportContext = *ImportContextPtr;
						{
							ImportContexts.Add(ImportContextPtr);

							ImportContextPtr->InitScene(SceneForRootActor);

							const bool bSilent = true; // don't pop options window
							ImportContextPtr->InitOptions(nullptr, TOptional<FString>(), bSilent);

							const EObjectFlags NewObjectFlags = RF_Public | RF_Standalone | RF_Transactional;
							if (!ImportContext.SetupDestination(DestinationPackage->GetName(), NewObjectFlags, GWarn, bSilent))
							{
								continue;
							}

							// Copy AssetsContext - to assign static meshes to actors
							ImportContext.AssetsContext = AssetsImportContext.AssetsContext;
						}

						// CreateSceneAsset
						{
							FString AssetName = Label;
							AssetName = FDatasmithUtils::SanitizeObjectName(AssetName);

							FString PackageName = FPaths::Combine(DestinationFolder, Label);
							PackageName = UPackageTools::SanitizePackageName(PackageName);

							FText CreateAssetFailure = LOCTEXT("CreateSceneAsset_PackageFailure", "Failed to create the Datasmith Scene asset.");
							FText OutFailureReasonText;
							if (!FDatasmithImporterUtils::CanCreateAsset< UDatasmithScene >(PackageName + "." + AssetName, OutFailureReasonText))
							{
								ImportContext.LogError(OutFailureReasonText);
								ImportContext.LogError(CreateAssetFailure);
								continue;
							}

							UDatasmithScene* SceneAsset = FDatasmithImporterUtils::FindObject< UDatasmithScene >(nullptr, PackageName);
							if (!SceneAsset)
							{
								UPackage* Package = CreatePackage( *PackageName);
								if (!ensure(Package))
								{
									ImportContext.LogError(CreateAssetFailure);
									continue;
								}
								Package->FullyLoad();

								SceneAsset = NewObject< UDatasmithScene >(Package, FName(*AssetName), RF_Public | RF_Standalone);
							}

							UDatasmithTranslatedSceneImportData* ReImportSceneData = NewObject< UDatasmithTranslatedSceneImportData >(SceneAsset);
							SceneAsset->AssetImportData = ReImportSceneData;

							// Copy over the changes the user may have done on the options
							ReImportSceneData->BaseOptions = ImportContext.Options->BaseOptions;

							for (const TObjectPtr<UDatasmithOptionsBase>& Option : ImportContext.AdditionalImportOptions)
							{
								UDatasmithOptionsBase* OptionObj = Option.Get();
								OptionObj->Rename(nullptr, ReImportSceneData);
								ReImportSceneData->AdditionalOptions.Add(OptionObj);
							}
							ReImportSceneData->Update(ImportContext.Options->FilePath, ImportContext.FileHash.IsValid() ? &ImportContext.FileHash : nullptr);
							ReImportSceneData->DatasmithImportInfo = FDatasmithImportInfo(ImportContext.Options->SourceUri, ImportContext.Options->SourceHash);

							FAssetRegistryModule::AssetCreated(ReImportSceneData);

							ImportContext.SceneAsset = SceneAsset;

							FDatasmithImporterUtils::SaveDatasmithScene(ImportContext.Scene.ToSharedRef(), SceneAsset);
						}

						// Copy whole metadata map into each separate scene
						for (int32 MetadataIndex = 0; MetadataIndex < PlmXmlScene->GetMetaDataCount(); ++MetadataIndex)
						{
							SceneForRootActor->AddMetaData(PlmXmlScene->GetMetaData(MetadataIndex));
						}

						FDatasmithImporter::ImportActors(ImportContext);

						FDatasmithImportFactoryCreateFileResult& FileResult = Result[*FileIndexPtr];
						FileResult.Scene = ImportContext.SceneAsset;
						FileResult.bImportSucceed = true;

						if (FileResult.bImportSucceed)
						{
							FileResult.FillFromImportContext(ImportContext);
						}
					}

					FDatasmithImporter::FinalizeImport(AssetsImportContext, TSet<UObject*>());
					for (TSharedPtr<FDatasmithImportContext> ImportContext : ImportContexts)
					{
						FDatasmithImporter::FinalizeImport(*ImportContext, TSet<UObject*>());
					}
				}
				else
				{
					// Process all files separately if PlmXml if not translatable
					FilesNotProcessed = FilePaths;
				}
			}
		}

		// Try importing each file that wasn't imported with PlmXml separately
		for (int32 FileIndex = 0; FileIndex < FilePaths.Num(); ++FileIndex)
		{
			FDatasmithImportFactoryCreateFileResult& FileResult = Result[FileIndex];
			if (FileResult.bImportSucceed)
			{
				continue;
			}

			const FString& FileName = FilePaths[FileIndex];
			TSharedPtr<FExternalSource> ExternalSource = IExternalSourceModule::GetOrCreateExternalSource(FSourceUri::FromFilePath(FileName));

			if (!ExternalSource)
			{
				continue;
			}

			TUniquePtr<FDatasmithImportContext> ImportContextPtr = MakeUnique<FDatasmithImportContext>(ExternalSource.ToSharedRef(), false, GetLoggerName(), GetDisplayName());
			ImportContextPtr->InitOptions(nullptr, TOptional<FString>(), true);

			if (TSharedPtr<IDatasmithScene> LoadedScene = ExternalSource->TryLoad())
			{
				ImportContextPtr->InitScene(LoadedScene.ToSharedRef());
			}
			else
			{
				continue;
			}

			FDatasmithImportContext& ImportContext = *ImportContextPtr;

			const EObjectFlags NewObjectFlags = RF_Public | RF_Standalone | RF_Transactional;
			const bool bIsSilent = true;
			if (!ImportContext.SetupDestination(DestinationPackage->GetName(), NewObjectFlags, GWarn, bIsSilent))
			{
				continue;
			}

			// Inlined bool FDatasmithImportContext::SetupDestination
			// Overriding RootPackagePath
			ImportContext.Options->BaseOptions.AssetOptions.PackagePath = FName(*DestinationPackage->GetName());
			ImportContext.FeedbackContext = GWarn;
			ImportContext.FilteredScene = FDatasmithSceneFactory::DuplicateScene(ImportContext.Scene.ToSharedRef());
			ImportContext.SceneName = FDatasmithUtils::SanitizeObjectName(ImportContext.Scene->GetName());
			ImportContext.ObjectFlags = RF_Public | RF_Standalone | RF_Transactional;
			ImportContext.bUserCancelled = false;
			ImportContext.AssetsContext.ReInit(DestinationFolder);

			bool bUserCancelled = false;
			FileResult.bImportSucceed = DatasmithImportFactoryImpl::ImportDatasmithScene(ImportContext, bUserCancelled);
			FileResult.bImportSucceed &= !bUserCancelled;

			if (FileResult.bImportSucceed)
			{
				FileResult.FillFromImportContext(ImportContext);
			}

			FileResult.Scene = ImportContext.SceneAsset;
		}

		return MoveTemp(Result);
	}
}

UDatasmithSceneElement* UDatasmithSceneElement::ConstructDatasmithSceneFromFile(const FString& InFilename)
{
	using namespace UE::DatasmithImporter;

	const FSourceUri SourceUri = FSourceUri::FromFilePath(InFilename);
	return ConstructDatasmithSceneFromSourceUri(SourceUri.ToString());
}

UDatasmithSceneElement* UDatasmithSceneElement::ConstructDatasmithSceneFromSourceUri(const FString& SourceUriString)
{
	using namespace DatasmithBlueprintLibraryImpl;
	using namespace UE::DatasmithImporter;

	const FSourceUri SourceUri = FSourceUri(SourceUriString);
	TSharedPtr<FExternalSource> ExternalSource = IExternalSourceModule::GetOrCreateExternalSource(SourceUri);
	if (!ExternalSource)
	{
		UE_LOGF(LogDatasmithImport, Error, "Datasmith import error: source uri could not be resolved to an external source. Abort import.");
		return nullptr;
	}

	UDatasmithSceneElement* DatasmithSceneElement = NewObject<UDatasmithSceneElement>();
	DatasmithSceneElement->ExternalSourcePtr = ExternalSource;

	// Creating an empty scene so that UDatasmithSceneElement's functions can safely be called.
	// This scene is overriden when UDatasmithSceneElement::TranslateScene() is called.
	TSharedRef<IDatasmithScene> Scene = FDatasmithSceneFactory::CreateScene(*ExternalSource->GetSourceName());
	DatasmithSceneElement->SetDatasmithSceneElement(Scene);

	const bool bLoadConfig = false; // don't load values from ini files
	DatasmithSceneElement->ImportContextPtr.Reset(new FDatasmithImportContext(ExternalSource.ToSharedRef(), bLoadConfig, GetLoggerName(), GetDisplayName()));

	return DatasmithSceneElement;
}

TArray<FDatasmithImportFactoryCreateFileResult> UDatasmithSceneElement::ImportScenes(const FString& DestinationFolder)
{
	if(bMultifile)
	{
		return DatasmithSceneElementUtil::ImportDatasmithScenesFromFiles(DestinationFolder, FilePaths);
	}

	TArray<FDatasmithImportFactoryCreateFileResult> Result;
	Result.Add(ImportScene(DestinationFolder));
	return Result;
}

UDatasmithSceneElement* UDatasmithSceneElement::ConstructDatasmithSceneFromCADFiles(const TArray<FString>& FilePaths)
{
	UDatasmithSceneElement* DatasmithSceneElement = NewObject<UDatasmithSceneElement>();
	DatasmithSceneElement->bMultifile = true;
	DatasmithSceneElement->FilePaths = FilePaths;
	return DatasmithSceneElement;
}

UDatasmithSceneElement* UDatasmithSceneElement::GetExistingDatasmithScene(const FString& AssetPath)
{
	using namespace DatasmithBlueprintLibraryImpl;
	using namespace UE::DatasmithImporter;

	if (UDatasmithScene* SceneAsset = Cast<UDatasmithScene>(UEditorAssetLibrary::LoadAsset(AssetPath)))
	{
		if (!SceneAsset->AssetImportData)
		{
			UE_LOGF(LogDatasmithImport, Warning, "Datasmith ReimportScene error: no import data.");
			return nullptr;
		}

		UDatasmithSceneImportData& ReimportData = *SceneAsset->AssetImportData;

		const FSourceUri SourceUri = FSourceUri::FromFilePath(ReimportData.GetFirstFilename());
		TSharedPtr<FExternalSource> ExternalSource = IExternalSourceModule::GetOrCreateExternalSource(SourceUri);
		if (!ExternalSource)
		{
			UE_LOGF(LogDatasmithImport, Warning, "Datasmith import error: file name could not be resolved to an external source. Abort import.");
			return nullptr;
		}

		UDatasmithSceneElement* DatasmithSceneElement = NewObject<UDatasmithSceneElement>();
		DatasmithSceneElement->ExternalSourcePtr = ExternalSource;
		
		ExternalSource->SetSceneName(*SceneAsset->GetName()); // keep initial name

		// Creating an empty scene so that UDatasmithSceneElement's functions can safely be called.
		// This scene is overriden when UDatasmithSceneElement::TranslateScene() is called.
		TSharedRef<IDatasmithScene> Scene = FDatasmithSceneFactory::CreateScene(*SceneAsset->GetName());
		DatasmithSceneElement->SetDatasmithSceneElement(Scene);

		// Setup pipe for reimport
		const bool bLoadConfig = false;
		DatasmithSceneElement->ImportContextPtr.Reset(new FDatasmithImportContext(ExternalSource.ToSharedRef(), bLoadConfig, GetLoggerName(), GetDisplayName()));
		FDatasmithImportContext& ImportContext = *DatasmithSceneElement->ImportContextPtr;
		ImportContext.SceneAsset = SceneAsset;
		ImportContext.Options->BaseOptions = ReimportData.BaseOptions; // Restore options as used in original import
		ImportContext.bIsAReimport = true;

		FString ImportPath = ImportContext.Options->BaseOptions.AssetOptions.PackagePath.ToString();

		return DatasmithSceneElement;
	}

	return nullptr;
}

bool UDatasmithSceneElement::TranslateScene()
{
	if (!ImportContextPtr.IsValid() || !ExternalSourcePtr.IsValid())
	{
		UE_LOGF(LogDatasmithImport, Error, "Invalid State. Ensure ConstructDatasmithSceneFromFile has been called.");
		return false;
	}

	const bool bSilent = true; // don't pop options window
	ImportContextPtr->InitOptions(nullptr, TOptional<FString>(), bSilent);

	if (TSharedPtr<IDatasmithScene> LoadedScene = ExternalSourcePtr->TryLoad())
	{
		SetDatasmithSceneElement(LoadedScene);
		ImportContextPtr->InitScene(LoadedScene.ToSharedRef());
	}
	else
	{
		UE_LOGF(LogDatasmithImport, Error, "Datasmith import error: Scene translation failure. Abort import.");
		return false;
	}

	bTranslated = true;
	return true;
}

FDatasmithImportFactoryCreateFileResult UDatasmithSceneElement::ImportScene(const FString& DestinationFolder)
{
	FDatasmithImportFactoryCreateFileResult Result;

	if (bMultifile)
	{
		TArray<FString> FilesNotProcessed;
		if (!DatasmithSceneElementUtil::ImportDatasmithSceneFromCADFiles(DestinationFolder, FilePaths, Result, FilesNotProcessed))
		{
			FilesNotProcessed = FilePaths;
		}
		if (FilesNotProcessed.Num() > 0)
		{
			UE_LOGF(LogDatasmithImport, Warning, "ImportScene - not all files were imported into the scene(try construct_datasmith_scene_from_file to import each file separately):");
			for (FString FileName : FilesNotProcessed)
			{
				UE_LOGF(LogDatasmithImport, Warning, "  '%ls' wasn't imported", *FileName);
			}
		}

		return Result;
	}

	if (!ImportContextPtr.IsValid() || !ExternalSourcePtr.IsValid())
	{
		UE_LOGF(LogDatasmithImport, Error, "Invalid State. Ensure ConstructDatasmithSceneFromFile has been called.");
		return Result;
	}

	UPackage* DestinationPackage;
	const TCHAR* OutFailureReason = TEXT("");
	if (!DatasmithBlueprintLibraryImpl::ValidatePackage(DestinationFolder, DestinationPackage, OutFailureReason))
	{
		UE_LOGF(LogDatasmithImport, Error, "Invalid Destination '%ls': %ls", *DestinationFolder, OutFailureReason);
		return Result;
	}

	if (!bTranslated)
	{
		if (!TranslateScene())
		{
			return Result;
		}
	}

	FDatasmithImportContext& ImportContext = *ImportContextPtr;
	const EObjectFlags NewObjectFlags = RF_Public | RF_Standalone | RF_Transactional;
	const bool bIsSilent = true;
	if (!ImportContext.SetupDestination(DestinationPackage->GetName(), NewObjectFlags, GWarn, bIsSilent))
	{
		return Result;
	}
	bool bUserCancelled = false;
	Result.bImportSucceed = DatasmithImportFactoryImpl::ImportDatasmithScene(ImportContext, bUserCancelled);
	Result.bImportSucceed &= !bUserCancelled;

	if (Result.bImportSucceed)
	{
		Result.FillFromImportContext(ImportContext);
	}

	DestroyScene();

	return Result;
}

FDatasmithImportFactoryCreateFileResult UDatasmithSceneElement::ReimportScene()
{
	FDatasmithImportFactoryCreateFileResult Result;

	if (!ImportContextPtr.IsValid() || !ImportContextPtr->Options || !ExternalSourcePtr.IsValid())
	{
		UE_LOGF(LogDatasmithImport, Error, "Invalid State. Ensure GetExistingDatasmithScene has been called.");
		return Result;
	}

	FDatasmithImportContext& ImportContext = *ImportContextPtr;
	FString ImportPath = ImportContext.Options->BaseOptions.AssetOptions.PackagePath.ToString();

	UPackage* DestinationPackage;
	const TCHAR* OutFailureReason = TEXT("");
	if (!DatasmithBlueprintLibraryImpl::ValidatePackage(ImportPath, DestinationPackage, OutFailureReason))
	{
		UE_LOGF(LogDatasmithImport, Error, "Invalid Destination '%ls': %ls", *ImportPath, OutFailureReason);
		return Result;
	}

	EObjectFlags NewObjectFlags = RF_Public | RF_Standalone | RF_Transactional;
	TSharedPtr<FJsonObject> ImportSettingsJson;
	const bool bIsSilent = true;
	const FString DestinationPath = DestinationPackage->GetName();
	if (!ImportContext.Init(DestinationPath, NewObjectFlags, GWarn, ImportSettingsJson, bIsSilent))
	{
		return Result;
	}

	if (TSharedPtr<IDatasmithScene> LoadedScene = ExternalSourcePtr->TryLoad())
	{
		SetDatasmithSceneElement(LoadedScene);
		ImportContextPtr->InitScene(LoadedScene.ToSharedRef());
	}
	else
	{
		UE_LOGF(LogDatasmithImport, Error, "Datasmith import error: Scene translation failure. Abort import.");
		return Result;
	}

	bool bUserCancelled = false;
	Result.bImportSucceed = DatasmithImportFactoryImpl::ImportDatasmithScene(ImportContext, bUserCancelled);
	Result.bImportSucceed &= !bUserCancelled;

	if (Result.bImportSucceed)
	{
		Result.FillFromImportContext(ImportContext);
	}

	// Copy over the changes the user may have done on the options
	if (!ImportContext.SceneAsset->AssetImportData)
	{
		UE_LOGF(LogDatasmithImport, Warning, "Datasmith import error: Missing scene asset import data. Abort import.");
		return Result;
	}

	UDatasmithSceneImportData& NewReimportData = *ImportContext.SceneAsset->AssetImportData;
	NewReimportData.BaseOptions = ImportContext.Options->BaseOptions;

	NewReimportData.Modify();
	NewReimportData.PostEditChange();
	NewReimportData.MarkPackageDirty();

	DestroyScene();

	return Result;
}

UObject* UDatasmithSceneElement::GetOptions(UClass* OptionType)
{
	if (OptionType == nullptr)
	{
		OptionType = UDatasmithImportOptions::StaticClass();
	}

	if (ImportContextPtr.IsValid())
	{
		// Standard options from Datasmith
		if (ImportContextPtr->Options && ImportContextPtr->Options->GetClass()->IsChildOf(OptionType))
		{
			return ImportContextPtr->Options.Get();
		}

		// Additional options from specific translators
		for(const auto& AdditionalOption : ImportContextPtr->AdditionalImportOptions)
		{
			if (AdditionalOption->GetClass()->IsChildOf(OptionType))
			{
				return AdditionalOption.Get();
			}
		}
	}
	return nullptr;
}

TMap<UClass*, UObject*> UDatasmithSceneElement::GetAllOptions()
{
	TMap<UClass*, UObject*> AllOptions;

	auto Append = [&](UObject* Option)
	{
		if (Option)
		{
			AllOptions.Add(Option->GetClass(), Option);
		}
	};

	if (ImportContextPtr.IsValid())
	{
		// Standard options from Datasmith
		if (ImportContextPtr->Options)
		{
			Append(ImportContextPtr->Options);
		}

		// Additional options from specific translators
		for(const auto& AdditionalOption : ImportContextPtr->AdditionalImportOptions)
		{
			Append(AdditionalOption);
		}
	}

	return AllOptions;
}

UDatasmithImportOptions* UDatasmithSceneElement::GetImportOptions()
{
	return Cast<UDatasmithImportOptions>(GetOptions());
}

void UDatasmithSceneElement::DestroyScene()
{
	ImportContextPtr.Reset();
	ExternalSourcePtr.Reset();
	Reset();
}

void UDatasmithStaticMeshBlueprintLibrary::SetupStaticLighting(const TArray< UObject* >& Objects, bool bApplyChanges, bool bGenerateLightmapUVs, float LightmapResolutionIdealRatio)
{
	// Collect all the static meshes and static mesh components to compute lightmap resolution for
	TMap< UStaticMesh*, TSet< UStaticMeshComponent* > > StaticMeshMap(DatasmithStaticMeshBlueprintLibraryUtil::GetStaticMeshComponentMap(Objects));

	for (const auto& StaticMeshPair : StaticMeshMap)
	{
		UStaticMesh* StaticMesh = StaticMeshPair.Key;

		if (bApplyChanges)
		{
			StaticMesh->Modify();
		}

		for (int32 LODIndex = 0; LODIndex < StaticMesh->GetNumSourceModels(); ++LODIndex)
		{
			FStaticMeshSourceModel& SourceModel = StaticMesh->GetSourceModel(LODIndex);
			const bool bDidChangeSettings = SourceModel.BuildSettings.bGenerateLightmapUVs != bGenerateLightmapUVs;
			SourceModel.BuildSettings.bGenerateLightmapUVs = bGenerateLightmapUVs;

			if (LODIndex == 0)
			{
				int32 MaxBiggestUVChannel = Lightmass::MAX_TEXCOORDS;

				if (FMeshDescription* MeshDescription = SourceModel.GetCachedMeshDescription())
				{
					FStaticMeshConstAttributes Attributes(*MeshDescription);

					// 3 is the maximum that lightmass accept. Defined in MeshExport.h : MAX_TEXCOORDS .
					MaxBiggestUVChannel = FMath::Min(MaxBiggestUVChannel, Attributes.GetVertexInstanceUVs().GetNumChannels() - 1);
				}

				if (bGenerateLightmapUVs)
				{
					const int32 GeneratedLightmapChannel = SourceModel.BuildSettings.DstLightmapIndex;

					if (GeneratedLightmapChannel < Lightmass::MAX_TEXCOORDS)
					{
						StaticMesh->SetLightMapCoordinateIndex(GeneratedLightmapChannel);
					}
					else
					{
						UE_LOGF(LogSetupStaticLighting, Warning, "Could not complete the static lighting setup for static mesh %ls as the generated lightmap UV is set to be in channel #%i while the maximum lightmap channel is %i", *StaticMesh->GetName(), GeneratedLightmapChannel, Lightmass::MAX_TEXCOORDS);
						break;
					}
				}
				else if (StaticMesh->GetLightMapCoordinateIndex() > MaxBiggestUVChannel && bDidChangeSettings)
				{
					// If we are not generating the lightmap anymore make sure we are selecting a valid lightmap index.
					StaticMesh->SetLightMapCoordinateIndex(MaxBiggestUVChannel);
				}
			}
		}
	}

	// Compute the lightmap resolution, do not apply the changes so that the computation is done on multiple threads
	// We'll directly call PostEditChange() at the end of the function so that we also get the StaticLightingSetup changes.
	ComputeLightmapResolution(StaticMeshMap, /* bApplyChange */false, LightmapResolutionIdealRatio);

	for (const auto& StaticMeshPair : StaticMeshMap)
	{
		StaticMeshPair.Key->PostEditChange();
	}
}

void UDatasmithStaticMeshBlueprintLibrary::ComputeLightmapResolution(const TArray< UObject* >& Objects, bool bApplyChanges, float IdealRatio)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UDatasmithStaticMeshBlueprintLibrary::ComputeLightmapResolution)

	// Collect all the static meshes and static mesh components to compute lightmap resolution for
	TMap< UStaticMesh*, TSet< UStaticMeshComponent* > > StaticMeshMap(DatasmithStaticMeshBlueprintLibraryUtil::GetStaticMeshComponentMap(Objects));

	ComputeLightmapResolution(StaticMeshMap, bApplyChanges, IdealRatio);
}

void UDatasmithStaticMeshBlueprintLibrary::ComputeLightmapResolution(const TMap< UStaticMesh*, TSet< UStaticMeshComponent* > >& StaticMeshMap, bool bApplyChanges, float IdealRatio)
{
	// The actual work
	auto Compute = [&](UStaticMesh* StaticMesh, const TSet<UStaticMeshComponent*>& Components)
	{
		bool bComputeForComponents = true;

		// Compute light map resolution for static mesh asset if required
		if(Components.Contains(nullptr))
		{
			if( int32 LightMapResolution = ComputeLightmapResolution(StaticMesh, IdealRatio, FVector::OneVector) )
			{
				// Close the mesh editor to prevent crashing. If changes are applied, reopen it after the mesh has been built.
				bool bStaticMeshIsEdited = false;
				UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
				if (AssetEditorSubsystem && AssetEditorSubsystem->FindEditorForAsset(StaticMesh, false))
				{
					AssetEditorSubsystem->CloseAllEditorsForAsset(StaticMesh);
					bStaticMeshIsEdited = true;
				}

				if(bApplyChanges)
				{
					StaticMesh->Modify();
				}

				StaticMesh->SetLightMapResolution(LightMapResolution);

				if(bApplyChanges)
				{
					// Request re-building of mesh with new LODs
					StaticMesh->PostEditChange();

					// Reopen MeshEditor on this mesh if the MeshEditor was previously opened in it
					if (bStaticMeshIsEdited)
					{
						AssetEditorSubsystem->OpenEditorForAsset(StaticMesh);
					}
				}
			}
			else
			{
				bComputeForComponents = false;
			}
		}

		if(bComputeForComponents)
		{
			for (UStaticMeshComponent* StaticMeshComponent : Components)
			{
				if(StaticMeshComponent)
				{
					if( int32 LightMapResolution = ComputeLightmapResolution( StaticMesh, IdealRatio, StaticMeshComponent->GetComponentScale() ) )
					{
						StaticMeshComponent->bOverrideLightMapRes = true;
						StaticMeshComponent->OverriddenLightMapRes = LightMapResolution;
					}
				}
			}
		}
	};

	// If no need to notify changes, multi-thread the computing
	if(!bApplyChanges)
	{
		TArray< UStaticMesh* > StaticMeshes;
		StaticMeshMap.GenerateKeyArray( StaticMeshes );

		// Start with the biggest mesh first to help balancing tasks on threads
		Algo::SortBy(
			StaticMeshes,
			[](const UStaticMesh* Mesh){ return Mesh->IsMeshDescriptionValid(0) ? Mesh->GetMeshDescription(0)->Vertices().Num() : 0; },
			TGreater<>()
		);

		ParallelFor(StaticMeshes.Num(),
			[&](int32 Index)
			{
				// We need to ensure the source UVs for generated lightmaps are available before generating then in the UStaticMesh::BatchBuild().
				DatasmithStaticMeshBlueprintLibraryUtil::EnsureLightmapSourceUVsAreAvailable(StaticMeshes[Index]);
			},
			EParallelForFlags::Unbalanced
		);

		UStaticMesh::BatchBuild( StaticMeshes, true );

		// To avoid race condition, we need to wait for async mesh compilation to finish (if enabled) before proceeding to access mesh properties
		FStaticMeshCompilingManager::Get().FinishCompilation( StaticMeshes );

		ParallelFor( StaticMeshes.Num(),
			[&]( int32 Index )
			{
				Compute( StaticMeshes[Index], StaticMeshMap[StaticMeshes[Index]] );
			},
			EParallelForFlags::Unbalanced
		);
	}
	// Do not take any chance, compute sequentially
	else
	{
		for(const TPair< UStaticMesh*, TSet< UStaticMeshComponent* > >& Entry : StaticMeshMap)
		{
			Compute(  Entry.Key, Entry.Value );
		}
	}
}

int32 UDatasmithStaticMeshBlueprintLibrary::ComputeLightmapResolution(UStaticMesh* StaticMesh, float IdealRatio, const FVector& StaticMeshScale)
{
	if(StaticMesh == nullptr || !StaticMesh->HasValidRenderData())
	{
		return 0;
	}

	const FRawStaticIndexBuffer& IndexBuffer = StaticMesh->GetRenderData()->LODResources[0].IndexBuffer;
	const FPositionVertexBuffer& PositionBuffer = StaticMesh->GetRenderData()->LODResources[0].VertexBuffers.PositionVertexBuffer;
	const FStaticMeshVertexBuffer& VertexBuffer = StaticMesh->GetRenderData()->LODResources[0].VertexBuffers.StaticMeshVertexBuffer;

	if (VertexBuffer.GetNumTexCoords() <= (uint32)StaticMesh->GetLightMapCoordinateIndex())
	{
		return 0;
	}

	// Compute the mesh UV density, based on FStaticMeshRenderData::ComputeUVDensities, except that we're only working the Lightmap UV.
	TArray< FVector2D > PolygonAreas;
	const int32 NumberOfTriangles = IndexBuffer.GetNumIndices() / 3;
	for (int32 TriangleIndex = 0; TriangleIndex < NumberOfTriangles; ++TriangleIndex)
	{
		FVector VertexPosition[3];
		FVector2D LightmapUVs[3];

		for (int32 CornerIndex = 0; CornerIndex < 3; ++CornerIndex)
		{
			uint32 VertexIndex = IndexBuffer.GetIndex(TriangleIndex * 3 + CornerIndex);
			VertexPosition[CornerIndex] = (FVector)PositionBuffer.VertexPosition(VertexIndex);
			LightmapUVs[CornerIndex] = FVector2D(VertexBuffer.GetVertexUV(VertexIndex, StaticMesh->GetLightMapCoordinateIndex()));
		}

		const float PolygonArea = DatasmithStaticMeshBlueprintLibraryUtil::ParallelogramArea(VertexPosition[0], VertexPosition[1], VertexPosition[2]);
		const float PolygonUVArea = DatasmithStaticMeshBlueprintLibraryUtil::ParallelogramArea(FVector(LightmapUVs[0], 0.f), FVector(LightmapUVs[1], 0.f), FVector(LightmapUVs[2], 0.f));

		PolygonAreas.Emplace(FMath::Sqrt(PolygonArea), FMath::Sqrt(PolygonArea / PolygonUVArea));
	}

	Algo::Sort( PolygonAreas, []( const FVector2D& ElementA, const FVector2D& ElementB )
	{
		return ElementA[1] < ElementB[1];
	} );

	float WeightedUVDensity = 0.f;
	float Weight = 0.f;

	// Remove 10% of higher and lower texel factors.
	const int32 Threshold = FMath::FloorToInt( 0.1f * (float)PolygonAreas.Num() );
	for (int32 Index = Threshold; Index < PolygonAreas.Num() - Threshold; ++Index)
	{
		WeightedUVDensity += PolygonAreas[ Index ][ 1 ] * PolygonAreas[ Index ][ 0 ];
		Weight += PolygonAreas[ Index ][ 0 ];
	}

	float UVDensity = WeightedUVDensity / Weight;

	int32 LightmapResolution = FMath::FloorToInt( UVDensity * IdealRatio );

	// Ensure that LightmapResolution is a factor of 4
	return FMath::Max( LightmapResolution + 3 & ~3, 4 );
}

FDatasmithImportFactoryCreateFileResult::FDatasmithImportFactoryCreateFileResult()
	: bImportSucceed(false)
	, Scene(nullptr)
{}

void FDatasmithImportFactoryCreateFileResult::FillFromImportContext(const FDatasmithImportContext& ImportContext)
{
	ImportedActors.Append(ImportContext.GetImportedActors());

	ImportedMeshes.Reserve(ImportContext.ImportedStaticMeshes.Num());
	for ( const auto& MeshPair : ImportContext.ImportedStaticMeshes )
	{
		ImportedMeshes.Add( MeshPair.Value );
	}
}

#undef LOCTEXT_NAMESPACE
