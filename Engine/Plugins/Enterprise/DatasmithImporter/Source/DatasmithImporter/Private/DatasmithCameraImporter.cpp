// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithCameraImporter.h"

#include "DatasmithActorImporter.h"
#include "DatasmithImportContext.h"
#include "DatasmithMaterialExpressions.h"
#include "DatasmithPostProcessImporter.h"
#include "DatasmithSceneActor.h"
#include "IDatasmithSceneElements.h"

#include "CineCameraActor.h"
#include "CineCameraComponent.h"

#include "ObjectTemplates/DatasmithCineCameraActorTemplate.h"
#include "ObjectTemplates/DatasmithCineCameraComponentTemplate.h"

#include "Templates/SharedPointer.h"

#define LOCTEXT_NAMESPACE "DatasmithCameraImporter"

AActor* FDatasmithCameraImporter::ImportCameraActor( const TSharedRef< IDatasmithCameraActorElement >& CameraElement, FDatasmithImportContext& ImportContext )
{
	AActor* Actor = FDatasmithActorImporter::ImportActor( ACineCameraActor::StaticClass(), CameraElement, ImportContext, ImportContext.Options->CameraImportPolicy );
	
	ACineCameraActor* CameraActor = Cast< ACineCameraActor >( Actor );

	if ( !CameraActor )
	{
		return nullptr;
	}

	SetupCineCameraComponent( CameraActor->GetCineCameraComponent(), CameraElement );
	
	CameraActor->UpdateComponentTransforms();
	CameraActor->MarkComponentsRenderStateDirty();
	CameraActor->MarkPackageDirty();

	return CameraActor;
}

UCineCameraComponent* FDatasmithCameraImporter::ImportCineCameraComponent( const TSharedRef< IDatasmithCameraActorElement >& CameraElement, FDatasmithImportContext& ImportContext, UObject* Outer, FDatasmithActorUniqueLabelProvider& UniqueNameProvider )
{
	UCineCameraComponent* CineCameraComponent = Cast< UCineCameraComponent >( FDatasmithActorImporter::ImportSceneComponent( UCineCameraComponent::StaticClass(), CameraElement, ImportContext, Outer, UniqueNameProvider ) );

	if ( !CineCameraComponent )
	{
		return nullptr;
	}

	SetupCineCameraComponent( CineCameraComponent, CameraElement );

	CineCameraComponent->RegisterComponent();

	return CineCameraComponent;
}

void FDatasmithCameraImporter::PostImportCameraActor( const TSharedRef< IDatasmithCameraActorElement >& CameraElement, FDatasmithImportContext& ImportContext )
{
	ADatasmithSceneActor* DatasmithSceneActor = ImportContext.ActorsContext.ImportSceneActor;
	TSoftObjectPtr< AActor > ActorObject = DatasmithSceneActor->RelatedActors.FindRef( CameraElement->GetName() );

	if ( !ActorObject )
	{
		return;
	}

	ACineCameraActor* CameraActor = Cast< ACineCameraActor >( ActorObject.Get() );

	if ( !CameraActor )
	{
		return;
	}

	bool bEnableLookAt = FCString::Strlen( CameraElement->GetLookAtActor() ) > 0;

	UDatasmithCineCameraActorTemplate* CameraActorTemplate = NewObject< UDatasmithCineCameraActorTemplate >( CameraActor->GetRootComponent() );

	CameraActorTemplate->LookatTrackingSettings.bEnableLookAtTracking = bEnableLookAt;
	CameraActorTemplate->LookatTrackingSettings.bAllowRoll = CameraElement->GetLookAtAllowRoll();

	if ( bEnableLookAt )
	{
		TSoftObjectPtr< AActor > LookAtActor = DatasmithSceneActor->RelatedActors.FindRef( CameraElement->GetLookAtActor() );
		if (LookAtActor.IsValid())
		{
			CameraActorTemplate->LookatTrackingSettings.ActorToTrack = LookAtActor;
		}
		else
		{
			ImportContext.LogWarning( FText::Format( LOCTEXT( "PostImportCameraActor", "Failed to set camera target (Tracking Actor) on {0}: could not find actor \"{1}\"" ), FText::FromString( CameraActor->GetName() ), FText::FromString( CameraElement->GetLookAtActor() ) ) );
		}
	}

	CameraActorTemplate->Apply( CameraActor );
}

void FDatasmithCameraImporter::SetupCineCameraComponent( UCineCameraComponent* CineCameraComponent, const TSharedRef< IDatasmithCameraActorElement >& CameraElement )
{
	if ( !CineCameraComponent )
	{
		return;
	}

	UDatasmithCineCameraComponentTemplate* CameraTemplate = NewObject< UDatasmithCineCameraComponentTemplate >( CineCameraComponent );

	CameraTemplate->FilmbackSettings.SensorWidth = CameraElement->GetSensorWidth();
	CameraTemplate->FilmbackSettings.SensorHeight = CameraElement->GetSensorWidth() / CameraElement->GetSensorAspectRatio();
	CameraTemplate->LensSettings.MaxFStop = 32.0f;
	CameraTemplate->CurrentFocalLength = CameraElement->GetFocalLength();
	CameraTemplate->CurrentAperture = CameraElement->GetFStop();

	CameraTemplate->FocusSettings.FocusMethod = CameraElement->GetEnableDepthOfField() ? ECameraFocusMethod::Manual : ECameraFocusMethod::DoNotOverride;
	CameraTemplate->FocusSettings.ManualFocusDistance = CameraElement->GetFocusDistance();

	CameraTemplate->PostProcessSettings = FDatasmithPostProcessImporter::CopyDatasmithPostProcessToUEPostProcess( CameraElement->GetPostProcess() );

	CameraTemplate->Apply( CineCameraComponent );
}

#undef LOCTEXT_NAMESPACE
