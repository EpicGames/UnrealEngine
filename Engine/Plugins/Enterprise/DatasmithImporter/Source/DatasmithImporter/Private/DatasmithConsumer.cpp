// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithConsumer.h"

#include "DatasmithActorImporter.h"
#include "DatasmithAreaLightActor.h"
#include "DatasmithAssetImportData.h"
#include "DatasmithAssetUserData.h"
#include "DatasmithContentEditorModule.h"
#include "DatasmithContentBlueprintLibrary.h"
#include "DatasmithScene.h"
#include "DatasmithSceneActor.h"
#include "DatasmithSceneFactory.h"
#include "IDatasmithSceneElements.h"
#include "DatasmithStaticMeshImporter.h"
#include "DataprepAssetUserData.h"
#include "DataprepAssetInterface.h"
#include "LevelVariantSets.h"
#include "ObjectTemplates/DatasmithActorTemplate.h"
#include "ObjectTemplates/DatasmithAreaLightActorTemplate.h"
#include "ObjectTemplates/DatasmithCineCameraComponentTemplate.h"
#include "ObjectTemplates/DatasmithLightComponentTemplate.h"
#include "ObjectTemplates/DatasmithMaterialInstanceTemplate.h"
#include "ObjectTemplates/DatasmithPointLightComponentTemplate.h"
#include "ObjectTemplates/DatasmithObjectTemplate.h"
#include "ObjectTemplates/DatasmithSceneComponentTemplate.h"
#include "ObjectTemplates/DatasmithStaticMeshTemplate.h"
#include "ObjectTemplates/DatasmithSkyLightComponentTemplate.h"
#include "ObjectTemplates/DatasmithStaticMeshComponentTemplate.h"
#include "Utility/DatasmithImporterUtils.h"
#include "Utility/DatasmithImporterImpl.h"

#include "Algo/Count.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "CineCameraActor.h"
#include "CineCameraComponent.h"
#include "ComponentReregisterContext.h"
#include "Components/HierarchicalInstancedStaticMeshComponent.h"
#include "Components/PointLightComponent.h"
#include "Editor.h"
#include "EditorLevelUtils.h"
#include "Engine/Level.h"
#include "Engine/LevelStreamingAlwaysLoaded.h"
#include "Engine/Light.h"
#include "Engine/RectLight.h"
#include "Engine/Selection.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/World.h"
#include "ExternalSourceModule.h"
#include "Factories/WorldFactory.h"
#include "FileHelpers.h"
#include "GameFramework/PhysicsVolume.h"
#include "GameFramework/WorldSettings.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IAssetTools.h"
#include "Internationalization/Internationalization.h"
#include "IUriManager.h"
#include "LevelSequence.h"
#include "LevelUtils.h"
#include "Materials/Material.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialInterface.h"
#include "Misc/Guid.h"
#include "Misc/MessageDialog.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Misc/SecureHash.h"
#include "ObjectTools.h"
#include "PackageTools.h"
#include "SourceUri.h"
#include "UObject/Package.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/UnrealType.h"

#define LOCTEXT_NAMESPACE "DatasmithConsumer"

const FText DatasmithConsumerLabel( LOCTEXT( "DatasmithConsumerLabel", "Datasmith writer" ) );
const FText DatasmithConsumerDescription( LOCTEXT( "DatasmithConsumerDesc", "Writes data prep world's current level and assets to current level" ) );

const TCHAR* DatasmithSceneSuffix = TEXT("_Scene");

namespace DatasmithConsumerUtils
{
	/** Helper to generate actor element from a scene actor */
	void ConvertSceneActorsToActors( FDatasmithImportContext& ImportContext );

	/** Helper to pre-build all static meshes from the array of assets passed to a consumer */
	void AddAssetsToContext( FDatasmithImportContext& ImportContext, TArray< TWeakObjectPtr< UObject > >& Assets );

	FString GetObjectUniqueId(UObject* Object)
	{
		UDatasmithContentBlueprintLibrary* DatasmithContentLibrary = Cast< UDatasmithContentBlueprintLibrary >( UDatasmithContentBlueprintLibrary::StaticClass()->GetDefaultObject() );
		FString DatasmithUniqueId = DatasmithContentLibrary->GetDatasmithUserDataValueForKey( Object, UDatasmithAssetUserData::UniqueIdMetaDataKey );

		return DatasmithUniqueId.Len() == 0 ? Object->GetName() : DatasmithUniqueId;
	}

	void SaveMap(UWorld* WorldToSave);

	const FString& GetMarker(UObject* Object, const FString& Name);

	void SetMarker(UObject* Object, const FString& Name, const FString& Value);

	void MoveActorsToLevel(const TArray<AActor*>& ActorsToMove, ULevel* DestLevel, TMap< FName, TSoftObjectPtr< AActor > >& ActorsMap, const TArray<UPackage*>& PackagesToCheck, bool bDuplicate);

	template<class AssetClass>
	void SetMarker(const TMap<FName, TSoftObjectPtr< AssetClass >>& AssetMap, const FString& Name, const FString& Value)
	{
		for(const TPair<FName, TSoftObjectPtr< AssetClass >>& Entry : AssetMap)
		{
			if(UObject* Asset = Entry.Value.Get())
			{
				SetMarker(Asset, Name, Value);
			}
		}
	}

	template<class AssetClass>
	void CollectAssetsToSave(TMap<FName, TSoftObjectPtr< AssetClass >>& AssetMap, TArray<UPackage*>& OutPackages)
	{
		if(AssetMap.Num() > 0)
		{
			OutPackages.Reserve(OutPackages.Num() + AssetMap.Num());

			for(TPair<FName, TSoftObjectPtr< AssetClass >>& Entry : AssetMap)
			{
				if(UObject* Asset = Entry.Value.Get())
				{
					OutPackages.Add(Asset->GetOutermost());
				}
			}
		}
	}

	template<class AssetClass>
	TArray<UPackage*> ApplyFolderDirective(TMap<FName, TSoftObjectPtr< AssetClass >>& AssetMap, const FString& RootPackagePath, TMap<FSoftObjectPath, FSoftObjectPath>& AssetRedirectorMap, TFunction<void(ELogVerbosity::Type, FText)> ReportCallback, TSet<FString>& OutFolderOverrides)
	{
		auto CanMoveAsset = [&ReportCallback](UObject* Source, UObject* Target) -> bool
		{
			// Overwrite existing owned asset with the new one
			if( GetMarker(Source, UDatasmithConsumer::ConsumerMarkerID) == GetMarker(Target, UDatasmithConsumer::ConsumerMarkerID))
			{
				TArray<UObject*> ObjectsToReplace(&Target, 1);
				ObjectTools::ForceReplaceReferences( Source, ObjectsToReplace );

				Target->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors | REN_NonTransactional);

				return true;
			}

			if(Source->GetClass() != Target->GetClass())
			{
				const FText AssetName = FText::FromString( Source->GetName() );
				const FText AssetFolder = FText::FromString( FPaths::GetPath(Source->GetPathName()) );
				const FText Message = FText::Format( LOCTEXT( "FolderDirective_ClassIssue", "Cannot move {0} to {1}. An asset with same name but different class exists"), AssetName, AssetFolder );
				ReportCallback(ELogVerbosity::Error, Message);
			}
			else
			{
				const FText AssetName = FText::FromString( Source->GetName() );
				const FText AssetFolder = FText::FromString( FPaths::GetPath(Source->GetPathName()) );
				const FText Message = FText::Format( LOCTEXT( "FolderDirective_Overwrite", "Cannot move {0} to {1}. An asset with same name and same class exists"), AssetName, AssetFolder );
				ReportCallback(ELogVerbosity::Error, Message);
			}

			return false;
		};

		TArray<UPackage*> PackagesProcessed;
		PackagesProcessed.Reserve(AssetMap.Num());

		// Reserve room for new entries in remapping table
		AssetRedirectorMap.Reserve(AssetRedirectorMap.Num() + AssetMap.Num());

		for(TPair<FName, TSoftObjectPtr< AssetClass >>& Entry : AssetMap)
		{
			if(AssetClass* Asset = Entry.Value.Get())
			{
				const FString& OutputFolder = GetMarker(Asset, UDataprepContentConsumer::RelativeOutput);
				if(OutputFolder.Len() > 0)
				{
					OutFolderOverrides.Add(FPaths::Combine(RootPackagePath, OutputFolder));

					UPackage* SourcePackage = Entry.Value->GetOutermost();
					FString TargetPackagePath = FPaths::Combine(RootPackagePath, OutputFolder, Asset->GetName());
				
					if( ensure(SourcePackage) && SourcePackage->GetPathName() != TargetPackagePath)
					{
						FString PackageFilename;
						FPackageName::TryConvertLongPackageNameToFilename( TargetPackagePath, PackageFilename, FPackageName::GetAssetPackageExtension() );

						bool bCanMove = true;

						FString TargetAssetFullPath = TargetPackagePath + "." + Asset->GetName();
						if(UObject* MemoryObject = FSoftObjectPath(TargetAssetFullPath).ResolveObject())
						{
							bCanMove = CanMoveAsset( Asset, MemoryObject);
						}
						else if(FPaths::FileExists(PackageFilename))
						{
							bCanMove = CanMoveAsset( Asset, FSoftObjectPath(TargetAssetFullPath).TryLoad());
						}

						if(bCanMove)
						{
							FSoftObjectPath& SoftObjectPathRef = AssetRedirectorMap.Emplace( Asset );

							UPackage* TargetPackage = CreatePackage(*TargetPackagePath);
							TargetPackage->FullyLoad();

							Asset->Rename(nullptr, TargetPackage, REN_DontCreateRedirectors | REN_NonTransactional);

							// Update asset registry with renaming
							FAssetRegistryModule::AssetRenamed( Asset, SourcePackage->GetPathName() + TEXT(".") + Asset->GetName() );

							Entry.Value = Asset;
							SoftObjectPathRef = Asset;
							PackagesProcessed.Add(TargetPackage);

							// Clean up flags on source package. It is not useful anymore
							SourcePackage->SetDirtyFlag(false);
							SourcePackage->SetFlags(RF_Transient);
							SourcePackage->ClearFlags(RF_Standalone | RF_Public);
						}
					}
					else
					{
						PackagesProcessed.Add(SourcePackage);
					}
				}
				else
				{
					PackagesProcessed.Add(Entry.Value->GetOutermost());
				}
			}
		}

		return PackagesProcessed;
	}
}

const FString UDatasmithConsumer::ConsumerMarkerID = TEXT("DatasmithConsumer_UniqueID");

UDatasmithConsumer::UDatasmithConsumer()
	: WorkingWorld(nullptr)
	, PrimaryLevel(nullptr)
{
	if(!HasAnyFlags(RF_NeedLoad|RF_ClassDefaultObject))
	{
		UniqueID = FGuid::NewGuid().ToString(EGuidFormats::Short);
	}
}

void UDatasmithConsumer::PostLoad()
{
	UDataprepContentConsumer::PostLoad();

	// Update UniqueID for previous version of the consumer
	if(HasAnyFlags(RF_WasLoaded))
	{
		bool bMarkDirty = false;
		if(UniqueID.Len() == 0)
		{
			UniqueID = FGuid::NewGuid().ToString(EGuidFormats::Short);
			bMarkDirty = true;
		}

		if(LevelName.Len() == 0)
		{
			LevelName = GetOuter()->GetName() + TEXT("_Map");
		}

		if (OutputLevelSoftObject_DEPRECATED.GetAssetPathString().Len() > 0)
		{
			OutputLevelObjectPath = OutputLevelSoftObject_DEPRECATED.GetAssetPathString();
			OutputLevelSoftObject_DEPRECATED.Reset();

			bMarkDirty = true;
		}
		
		if(OutputLevelObjectPath.Len() == 0)
		{
			OutputLevelObjectPath = FPaths::Combine(TargetContentFolder, LevelName) + "." + LevelName;
		}

		if (DatasmithScene_DEPRECATED.GetAssetName().Len() > 0)
		{
			DatasmithSceneObjectPath = DatasmithScene_DEPRECATED.ToString();
			DatasmithScene_DEPRECATED.Reset();

			bMarkDirty = true;
		}

		if(bMarkDirty)
		{
			UE_LOGF(LogDatasmithImport, Warning, "%ls is from an old version and has been updated. Please save asset to complete update.", *GetOuter()->GetName());
		}
	}
}

void UDatasmithConsumer::PostInitProperties()
{
	UDataprepContentConsumer::PostInitProperties();

	if(!HasAnyFlags(RF_NeedLoad|RF_ClassDefaultObject))
	{
		if(LevelName.Len() == 0)
		{
			LevelName = GetOuter()->GetName() + TEXT("_Map");
		}

		OutputLevelObjectPath = FPaths::Combine( TargetContentFolder, LevelName) + TEXT(".") + LevelName;
	}
}

bool UDatasmithConsumer::Initialize()
{
	FText TaskDescription = LOCTEXT( "DatasmithImportFactory_Initialize", "Preparing world ..." );
	ProgressTaskPtr = MakeUnique< FDataprepWorkReporter >( Context.ProgressReporterPtr, TaskDescription, 3.0f, 1.0f );

	ProgressTaskPtr->ReportNextStep( LOCTEXT( "DatasmithImportFactory_Initialize", "Preparing world ...") );

	if(!ValidateAssets())
	{
		return false;
	}

	UpdateScene();

	// Re-create the DatasmithScene if not in memory yet
	if ( !DatasmithSceneWeakPtr.IsValid() )
	{
		// Check if DatasmithScene can be loaded in memory
		DatasmithSceneWeakPtr = Cast<UDatasmithScene>(FSoftObjectPath(DatasmithSceneObjectPath).TryLoad());

		// Still not in memory, re-create it
		if(!DatasmithSceneWeakPtr.IsValid())
		{
			FString PackageName = FPaths::Combine( GetTargetPackagePath(), GetOuter()->GetName() + DatasmithSceneSuffix );

			UPackage* Package = UPackageTools::FindOrCreatePackageForAssetType( FName( *PackageName ), UDatasmithScene::StaticClass() );
			check( Package );

			FString DatasmithSceneName = FPaths::GetBaseFilename( Package->GetFullName(), true );

			DatasmithSceneWeakPtr = NewObject< UDatasmithScene >( Package, *DatasmithSceneName, GetFlags() | RF_Standalone | RF_Public | RF_Transactional );
			check( DatasmithSceneWeakPtr.IsValid() );

			FAssetRegistryModule::AssetCreated( DatasmithSceneWeakPtr.Get() );

			DatasmithSceneWeakPtr->AssetImportData = NewObject< UDatasmithSceneImportData >( DatasmithSceneWeakPtr.Get(), UDatasmithSceneImportData::StaticClass() );
			check( DatasmithSceneWeakPtr->AssetImportData );

			// Store a Dataprep asset pointer into the scene asset in order to be able to later re-execute the dataprep pipeline
			if ( DatasmithSceneWeakPtr->GetClass()->ImplementsInterface(UInterface_AssetUserData::StaticClass()) )
			{
				if ( IInterface_AssetUserData* AssetUserDataInterface = Cast< IInterface_AssetUserData >( DatasmithSceneWeakPtr.Get() ) )
				{
					UDataprepAssetUserData* DataprepAssetUserData = AssetUserDataInterface->GetAssetUserData< UDataprepAssetUserData >();

					if ( !DataprepAssetUserData )
					{
						EObjectFlags Flags = RF_Public;
						DataprepAssetUserData = NewObject< UDataprepAssetUserData >( DatasmithSceneWeakPtr.Get(), NAME_None, Flags );
						AssetUserDataInterface->AddAssetUserData( DataprepAssetUserData );
					}

					UDataprepAssetInterface* DataprepAssetInterface = Cast< UDataprepAssetInterface >( GetOuter() );
					check( DataprepAssetInterface );

					DataprepAssetUserData->DataprepAssetPtr = DataprepAssetInterface;
				}
			}

			DatasmithSceneWeakPtr->MarkPackageDirty();

			// Make package dirty for new DatasmithScene been archived
			FProperty* Property = FindFProperty<FProperty>( StaticClass(), TEXT("DatasmithSceneWeakPtr") );
			FPropertyChangedEvent PropertyUpdateStruct( Property );
			PostEditChangeProperty( PropertyUpdateStruct );

			DatasmithSceneObjectPath = FSoftObjectPath(DatasmithSceneWeakPtr.Get()).GetAssetPathString();
		}
	}

	CreateWorld();

	if ( !BuildContexts() )
	{
		return false;
	}

	// Check if the finalize should be threated as a reimport
	ImportContextPtr->ActorsContext.FinalSceneActors.Append(FDatasmithImporterUtils::FindSceneActors( ImportContextPtr->ActorsContext.FinalWorld, ImportContextPtr->SceneAsset));
	if (ImportContextPtr->ActorsContext.FinalSceneActors.Num() > 0 )
	{
		ImportContextPtr->bIsAReimport = true;
		ImportContextPtr->Options->ReimportOptions.bRespawnDeletedActors = false;
		ImportContextPtr->Options->ReimportOptions.bUpdateActors = true;
		ImportContextPtr->Options->UpdateNotDisplayedConfig( true );
	}

	return true;
}

// Inspired from FDataprepDatasmithImporter::FinalizeAssets
bool UDatasmithConsumer::Run()
{
	// Pre-build static meshes
	ProgressTaskPtr->ReportNextStep( LOCTEXT( "DatasmithImportFactory_PreBuild", "Pre-building assets ...") );
	FDatasmithStaticMeshImporter::PreBuildStaticMeshes( *ImportContextPtr );

	// No need to have a valid set of assets.
	// All assets have been added to the AssetContext in UDatasmithConsumer::BuildContexts
	ProgressTaskPtr->ReportNextStep( LOCTEXT( "DatasmithImportFactory_Finalize", "Finalizing commit ...") );
	FDatasmithImporter::FinalizeImport( *ImportContextPtr, TSet<UObject*>() );

	// Apply UDataprepConsumerUserData directives for assets
	UDatasmithScene* SceneAsset = ImportContextPtr->SceneAsset;

	TFunction<void(ELogVerbosity::Type, FText)> ReportFunc = [&](ELogVerbosity::Type Verbosity, FText Message)
	{
		switch(Verbosity)
		{
			case ELogVerbosity::Warning:
			LogWarning(Message);
			break;

			case ELogVerbosity::Error:
			LogError(Message);
			break;

			default:
			LogInfo(Message);
			break;
		}
	};

	// Array to store materials which soft references might need to be fixed
	TArray<UPackage*> PackagesToCheck;

	// Array to store level sequences and level variants which soft references might need to be fixed
	TArray<UPackage*> PackagesToFix;

	// Table of remapping to contain moved assets
	TMap<FSoftObjectPath, FSoftObjectPath> AssetRedirectorMap;

	// Gether all the output overrides to be able to later delete empty folder in them
	TSet<FString> OutputFolderOverrides;

	DatasmithConsumerUtils::SetMarker(SceneAsset->Textures, UDatasmithConsumer::ConsumerMarkerID, UniqueID);
	DatasmithConsumerUtils::ApplyFolderDirective(SceneAsset->Textures, TargetContentFolder, AssetRedirectorMap, ReportFunc, OutputFolderOverrides);

	DatasmithConsumerUtils::SetMarker(SceneAsset->StaticMeshes, UDatasmithConsumer::ConsumerMarkerID, UniqueID);
	DatasmithConsumerUtils::ApplyFolderDirective(SceneAsset->StaticMeshes, TargetContentFolder, AssetRedirectorMap, ReportFunc, OutputFolderOverrides);

	DatasmithConsumerUtils::SetMarker(SceneAsset->Materials, UDatasmithConsumer::ConsumerMarkerID, UniqueID);
	PackagesToCheck.Append(DatasmithConsumerUtils::ApplyFolderDirective(SceneAsset->Materials, TargetContentFolder, AssetRedirectorMap, ReportFunc, OutputFolderOverrides));

	DatasmithConsumerUtils::SetMarker(SceneAsset->MaterialFunctions, UDatasmithConsumer::ConsumerMarkerID, UniqueID);
	DatasmithConsumerUtils::ApplyFolderDirective(SceneAsset->MaterialFunctions, TargetContentFolder, AssetRedirectorMap, ReportFunc, OutputFolderOverrides);

	DatasmithConsumerUtils::SetMarker(SceneAsset->LevelSequences, UDatasmithConsumer::ConsumerMarkerID, UniqueID);
	PackagesToFix.Append(DatasmithConsumerUtils::ApplyFolderDirective(SceneAsset->LevelSequences, TargetContentFolder, AssetRedirectorMap, ReportFunc, OutputFolderOverrides));

	DatasmithConsumerUtils::SetMarker(SceneAsset->LevelVariantSets, UDatasmithConsumer::ConsumerMarkerID, UniqueID);
	PackagesToFix.Append(DatasmithConsumerUtils::ApplyFolderDirective(SceneAsset->LevelVariantSets, TargetContentFolder, AssetRedirectorMap, ReportFunc, OutputFolderOverrides));

	if(AssetRedirectorMap.Num() > 0 && PackagesToCheck.Num() > 0)
	{
		IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
		AssetTools.RenameReferencingSoftObjectPaths(PackagesToCheck, AssetRedirectorMap);
	}

	// Remove empty output folders

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	TFunction<void(const FString&)> DeleteFolder = [&AssetRegistryModule](const FString& InFolderPath)
	{
		struct FEmptyFolderVisitor : public IPlatformFile::FDirectoryVisitor
		{
			bool bIsEmpty;

			FEmptyFolderVisitor()
				: bIsEmpty(true)
			{
			}

			virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
			{
				if (!bIsDirectory)
				{
					bIsEmpty = false;
					return false; // abort searching
				}

				return true; // continue searching
			}
		};

		bool bFolderWasRemoved = false;

		FString PathToDeleteOnDisk;
		if (FPackageName::TryConvertLongPackageNameToFilename(InFolderPath, PathToDeleteOnDisk))
		{
			// Look for files on disk in case the folder contains things not tracked by the asset registry
			FEmptyFolderVisitor EmptyFolderVisitor;
			IFileManager::Get().IterateDirectoryRecursively(*PathToDeleteOnDisk, EmptyFolderVisitor);

			if (EmptyFolderVisitor.bIsEmpty)
			{
				bFolderWasRemoved = IFileManager::Get().DeleteDirectory(*PathToDeleteOnDisk, false, true);
			}
		}

		if (bFolderWasRemoved)
		{
			AssetRegistryModule.Get().RemovePath(InFolderPath);
		}
	};

	TArray<FString> SubPaths;

	for (const FString& OuputFolder : OutputFolderOverrides)
	{
		SubPaths.Empty();
		AssetRegistryModule.Get().GetSubPaths(OuputFolder, SubPaths, true);

		// Sort to start from deeper folders first
		SubPaths.Sort([](const FString& A, const FString& B) -> bool
		{
			return A.Len() > B.Len();
		});

		for(auto SubPathIt(SubPaths.CreateConstIterator()); SubPathIt; SubPathIt++)	
		{
			TArray<FAssetData> AssetsInFolder;
			AssetRegistryModule.Get().GetAssetsByPath(FName(*SubPathIt), AssetsInFolder, true);
			if (AssetsInFolder.Num() == 0)
			{
				DeleteFolder(*SubPathIt);
			}
		}
	}

	return true;
}

bool UDatasmithConsumer::FinalizeRun()
{
	UDatasmithScene* SceneAsset = ImportContextPtr->SceneAsset;

	// Save all assets
	TArray<UPackage*> PackagesToSave;
	PackagesToSave.Add(DatasmithSceneWeakPtr->GetOutermost());

	DatasmithConsumerUtils::CollectAssetsToSave(SceneAsset->Textures, PackagesToSave);
	DatasmithConsumerUtils::CollectAssetsToSave(SceneAsset->MaterialFunctions, PackagesToSave);
	DatasmithConsumerUtils::CollectAssetsToSave(SceneAsset->Materials, PackagesToSave);
	DatasmithConsumerUtils::CollectAssetsToSave(SceneAsset->StaticMeshes, PackagesToSave);
	DatasmithConsumerUtils::CollectAssetsToSave(SceneAsset->LevelSequences, PackagesToSave);
	DatasmithConsumerUtils::CollectAssetsToSave(SceneAsset->LevelVariantSets, PackagesToSave);

	const bool bCheckDirty = false;
	const bool bPromptToSave = false;
	const FEditorFileUtils::EPromptReturnCode Return = FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, bCheckDirty, bPromptToSave);

	// Save secondary levels
	const TArray<ULevel*>& Levels = WorkingWorld->GetLevels();

	for(ULevel* Level : Levels)
	{
		if(Level)
		{
			UWorld* World = Cast<UWorld>(Level->GetOuter());

			if(World != WorkingWorld.Get())
			{
				DatasmithConsumerUtils::SaveMap(World);
			}
		}
	}

	// Save primary level now
	WorkingWorld->PersistentLevel = PrimaryLevel;
	DatasmithConsumerUtils::SaveMap(WorkingWorld.Get());

	return true;
}

bool UDatasmithConsumer::CreateWorld()
{
	ensure(!WorkingWorld.IsValid());

	WorkingWorld = TStrongObjectPtr<UWorld>(GWorld.GetReference());

	TArray<ULevel*> Levels = WorkingWorld->GetLevels();

	// Find level associated with this consumer
	PrimaryLevel = nullptr;
	for(ULevel* Level : Levels)
	{
		if(Level && Level->GetOuter()->GetName() == LevelName)
		{
			// Found a level with the same name, make sure it is the same package path
			if(UWorld* LevelWorld = Cast<UWorld>(Level->GetOuter()))
			{
				FSoftObjectPath LevelWorldPath(LevelWorld);

				// If paths differ, remove level with same name from world
				if(LevelWorldPath.GetAssetPathString() != OutputLevelObjectPath)
				{
					if(DatasmithConsumerUtils::GetMarker(Level, ConsumerMarkerID) == UniqueID)
					{
						if(Levels.Num() > 0)
						{
							const bool bShowDialog = !Context.bSilentMode && !IsRunningCommandlet();
							bool bUnloadLevel = true;

							if(bShowDialog)
							{
								const FTextFormat Format(LOCTEXT("DatasmithConsumer_AlreadyLoaded_Dlg", "Level {0} with a different path from the same Dataprep asset is already loaded.\n\nDo you want to unload it?"));
								const FText WarningMessage = FText::Format( Format, FText::FromString(LevelName) );
								const FText DialogTitle( LOCTEXT("DatasmithConsumerAlreadyLoaded_DlgTitle", "Warning - Level already loaded") );

								if(FMessageDialog::Open(EAppMsgType::YesNo, WarningMessage, DialogTitle) != EAppReturnType::Yes)
								{
									bUnloadLevel = false;
								}
							}
							else
							{
								const FTextFormat Format(LOCTEXT("DatasmithConsumer_AlreadyLoaded_Overwrite", "Level {0} with a different path from the same Dataprep asset is already loaded.It will be unloaded."));
								const FText WarningMessage = FText::Format( Format, FText::FromString(LevelName));
								LogWarning(WarningMessage);
							}

							if(bUnloadLevel)
							{
								WorkingWorld->RemoveLevel(Level);
								if(ULevelStreaming* StreamingLevel = WorkingWorld->GetLevelStreamingForPackageName(*LevelWorldPath.GetLongPackageName()))
								{
									WorkingWorld->RemoveStreamingLevel(StreamingLevel);
									WorkingWorld->UpdateLevelStreaming();
								}
							}
						}
					}
				}
				else
				{
					PrimaryLevel = Level;
				}
			}
			else
			{
				check(false);
			}

			break;
		}
	}

	if(PrimaryLevel == nullptr)
	{
		PrimaryLevel = FindOrAddLevel(LevelName);
		ensure(PrimaryLevel);
	}

	OriginalCurrentLevel = WorkingWorld->GetCurrentLevel();
	WorkingWorld->SetCurrentLevel(PrimaryLevel);

	return true;
}

void UDatasmithConsumer::ClearWorld()
{
	if(WorkingWorld.IsValid())
	{
		WorkingWorld->SetCurrentLevel(OriginalCurrentLevel);
		WorkingWorld.Reset();
	}
}

void UDatasmithConsumer::Reset()
{
	ImportContextPtr.Reset();
	ProgressTaskPtr.Reset();
	UDataprepContentConsumer::Reset();

	ClearWorld();
}

const FText& UDatasmithConsumer::GetLabel() const
{
	return DatasmithConsumerLabel;
}

const FText& UDatasmithConsumer::GetDescription() const
{
	return DatasmithConsumerDescription;
}

bool UDatasmithConsumer::BuildContexts()
{
	using namespace UE::DatasmithImporter;

	const FString FilePath = FPaths::Combine( FPaths::ProjectIntermediateDir(), ( DatasmithSceneWeakPtr->GetName() + TEXT( ".udatasmith" ) ) );
	const FSourceUri FileUri = FSourceUri::FromFilePath( FilePath );
	TSharedPtr<FExternalSource> ExternalSource = IExternalSourceModule::GetOrCreateExternalSource( FileUri );
	check( ExternalSource.IsValid() );

	ImportContextPtr = MakeUnique< FDatasmithImportContext >( ExternalSource.ToSharedRef(), false, TEXT("DatasmithImport"), LOCTEXT("DatasmithImportFactoryDescription", "Datasmith") );

	// Update import context with consumer's data
	ImportContextPtr->Options->BaseOptions.SceneHandling = EDatasmithImportScene::CurrentLevel;
	ImportContextPtr->SceneAsset = DatasmithSceneWeakPtr.Get();
	ImportContextPtr->ActorsContext.ImportWorld = Context.WorldPtr.Get();
	ImportContextPtr->InitScene(FDatasmithSceneFactory::CreateScene(*DatasmithSceneWeakPtr->GetName()));

	// Convert all incoming Datasmith scene actors as regular actors
	DatasmithConsumerUtils::ConvertSceneActorsToActors( *ImportContextPtr );

	// Recreate scene graph from actors in world
	ImportContextPtr->Scene->SetHost( TEXT( "DatasmithConsumer" ) );

	TArray<AActor*> RootActors;
	ImportContextPtr->ActorsContext.ImportSceneActor->GetAttachedActors( RootActors );
	FDatasmithImporterUtils::FillSceneElement( ImportContextPtr->Scene, RootActors );

	// Initialize context
	FString SceneOuterPath = DatasmithSceneWeakPtr->GetOutermost()->GetName();
	FString RootPath = FPackageName::GetLongPackagePath( SceneOuterPath );

	if ( Algo::Count( RootPath, TEXT('/') ) > 1 )
	{
		// Remove the scene folder as it shouldn't be considered in the import path
		RootPath.Split( TEXT("/"), &RootPath, nullptr, ESearchCase::IgnoreCase, ESearchDir::FromEnd );
	}

	FPaths::NormalizeDirectoryName( RootPath );

	const bool bSilent = true;
	if (!ImportContextPtr->Init(RootPath, RF_Public | RF_Standalone | RF_Transactional, GWarn, TSharedPtr< FJsonObject >(), bSilent))
	{
		FText Message = LOCTEXT( "DatasmithConsumer_BuildContexts", "Initialization of consumer failed" );
		LogError( Message );
		return false;
	}

	// Set the feedback context
	ImportContextPtr->FeedbackContext = Context.ProgressReporterPtr ? Context.ProgressReporterPtr->GetFeedbackContext() : nullptr;

	// Update ImportContext's package data
	ImportContextPtr->AssetsContext.RootFolderPath = TargetContentFolder;
	ImportContextPtr->AssetsContext.TransientFolderPath = Context.TransientContentFolder;

	ImportContextPtr->AssetsContext.StaticMeshesFinalPackage.Reset();
	ImportContextPtr->AssetsContext.MaterialsFinalPackage.Reset();
	ImportContextPtr->AssetsContext.TexturesFinalPackage.Reset();
	ImportContextPtr->AssetsContext.LightPackage.Reset();
	ImportContextPtr->AssetsContext.LevelSequencesFinalPackage.Reset();
	ImportContextPtr->AssetsContext.LevelVariantSetsFinalPackage.Reset();

	ImportContextPtr->AssetsContext.StaticMeshesImportPackage.Reset();
	ImportContextPtr->AssetsContext.TexturesImportPackage.Reset();
	ImportContextPtr->AssetsContext.MaterialsImportPackage.Reset();
	ImportContextPtr->AssetsContext.ReferenceMaterialsImportPackage.Reset();
	ImportContextPtr->AssetsContext.MaterialFunctionsImportPackage.Reset();
	ImportContextPtr->AssetsContext.LevelSequencesImportPackage.Reset();
	ImportContextPtr->AssetsContext.LevelVariantSetsImportPackage.Reset();

	// Set the destination world as the one in the level editor
	ImportContextPtr->ActorsContext.FinalWorld = WorkingWorld.Get();

	// Initialize ActorsContext's UniqueNameProvider with actors in the GWorld not the Import world
	ImportContextPtr->ActorsContext.UniqueNameProvider.Clear();
	ImportContextPtr->ActorsContext.UniqueNameProvider.PopulateLabelFrom( ImportContextPtr->ActorsContext.FinalWorld );

	// Add assets as if they have been imported using the current import context
	DatasmithConsumerUtils::AddAssetsToContext( *ImportContextPtr, Context.Assets );

	// Store IDatasmithScene(Element) in UDatasmithScene
	FDatasmithImporterUtils::SaveDatasmithScene( ImportContextPtr->Scene.ToSharedRef(), ImportContextPtr->SceneAsset );
	return true;
}

bool UDatasmithConsumer::SetLevelNameImplementation(const FString& InLevelName, FText& OutReason, const bool bIsAutomated)
{
	if(InLevelName.Len() == 0)
	{
		OutReason = LOCTEXT( "DatasmithConsumer_NameEmpty", "The level name is empty. Please enter a valid name." );
		return false;
	}

	int32 Index;
	if(InLevelName.FindChar( L'.', Index) || InLevelName.FindChar( L'/', Index))
	{
		OutReason = LOCTEXT( "DatasmithConsumer_IsAPath", "Path or relative path is not accepted as a folder name. Please enter a valid name." );
		return false;
	}

	FString SanitizedName = ObjectTools::SanitizeObjectName(InLevelName);
	if(SanitizedName != InLevelName)
	{
		OutReason = LOCTEXT( "DatasmithConsumer_InValidCharacters", "The level name contains invalid characters. Please enter a valid name." );
		return false;
	}

	if(!CanCreateLevel(TargetContentFolder, InLevelName, !bIsAutomated && !IsRunningCommandlet()))
	{
		return false;
	}

	if(SetOutputLevel(InLevelName))
	{
		Modify();

		LevelName = InLevelName;

		OnChanged.Broadcast();

		return true;
	}

	// Warn user new name has not been set
	OutReason = FText::Format( LOCTEXT("DatasmithConsumer_BadLevelName", "Cannot create level named {0}."), FText::FromString( InLevelName ) );

	return false;
}

bool UDatasmithConsumer::CanCreateLevel(const FString& RequestedFolder, const FString& RequestedName, const bool bShowDialog)
{
	FSoftObjectPath ObjectPath(FPaths::Combine(RequestedFolder, RequestedName) + TEXT(".") + RequestedName);

	const FString AssetPathName = ObjectPath.GetAssetPathString();

	bool bCanCreateAsset = false;

	// Check in the asset registry to see if there could be any conflict
	if(FDatasmithImporterUtils::CanCreateAsset(AssetPathName, UWorld::StaticClass()) == FDatasmithImporterUtils::EAssetCreationStatus::CS_CanCreate)
	{
		bCanCreateAsset = true;

		// No conflict in the AssetRegistry, check if no unregistered asset in memory could conflict
		if(UObject* Asset = ObjectPath.ResolveObject())
		{
			bCanCreateAsset = Cast<UWorld>(Asset) != nullptr;
		}
	}

	if(bCanCreateAsset)
	{
		FText OutReason;
		if(FDatasmithImporterImpl::CheckAssetPersistenceValidity(ObjectPath.GetLongPackageName(), *ImportContextPtr, FPackageName::GetMapPackageExtension(), OutReason))
		{
			FString PackageFilename;
			FPackageName::TryConvertLongPackageNameToFilename( ObjectPath.GetLongPackageName(), PackageFilename, FPackageName::GetMapPackageExtension() );

			if(FPaths::FileExists(PackageFilename))
			{
				if(UWorld* World = Cast<UWorld>(ObjectPath.TryLoad()))
				{
					if(DatasmithConsumerUtils::GetMarker(World->PersistentLevel, ConsumerMarkerID) != UniqueID)
					{
						if(bShowDialog)
						{
							const FTextFormat Format(LOCTEXT("DatasmithConsumer_SetTargetContentFolder_Update_Dlg", "Level {0} already exists in {1} but is not from this Dataprep asset.\n\nDo you want to update it?"));
							const FText WarningMessage = FText::Format( Format, FText::FromString(RequestedName), FText::FromString(RequestedFolder));
							const FText DialogTitle( LOCTEXT("DatasmithConsumer_Update_DlgTitle", "Warning - Level already exists") );

							EAppReturnType::Type Result = FMessageDialog::Open(EAppMsgType::YesNo, EAppReturnType::No, WarningMessage, DialogTitle);

							if(Result != EAppReturnType::Yes)
							{
								return false;
							}
						}
						else
						{
							const FTextFormat Format(LOCTEXT("DatasmithConsumer_SetTargetContentFolder_Overwrite", "Level {0} already exists in {1} but is not from this Dataprep asset.It will be updated."));
							const FText WarningMessage = FText::Format( Format, FText::FromString(RequestedName), FText::FromString(RequestedFolder));
							LogWarning(WarningMessage);
						}
					}
				}
			}
		}
		else
		{
			FString PackageFilename;
			FPackageName::TryConvertLongPackageNameToFilename( ObjectPath.GetLongPackageName(), PackageFilename, FPackageName::GetMapPackageExtension() );

			const FTextFormat Format(LOCTEXT("DatasmithConsumer_SetTargetContentFolder_CantCreateFile_Dlg", "Cannot create map file associated with level {0} in directory {1}.\nPlease choose another name for the level or fix the creation issue."));
			const FText Message = FText::Format( Format, FText::FromString(RequestedName), FText::FromString( FPaths::ConvertRelativePathToFull(FPaths::GetPath(PackageFilename)) ));

			if(bShowDialog)
			{
				const FText DialogTitle( LOCTEXT("DatasmithConsumer_CantCreateFile_DlgTitle", "Warning - Cannot create level") );

				FMessageDialog::Open(EAppMsgType::Ok, Message, DialogTitle);
			}
			else
			{
				LogError(Message);
			}

			return false;
		}
	}
	else
	{
		const FTextFormat Format(LOCTEXT("DatasmithConsumer_SetTargetContentFolder_CantCreate_Dlg", "Cannot create level {0} in folder {1}.\nAn asset of different class already exists in this folder with the same name."));
		const FText Message = FText::Format( Format, FText::FromString(RequestedName), FText::FromString(RequestedFolder));

		if(bShowDialog)
		{
			const FText DialogTitle( LOCTEXT("DatasmithConsumer_CantCreate_DlgTitle", "Warning - Cannot create level") );

			FMessageDialog::Open(EAppMsgType::Ok, Message, DialogTitle);
		}
		else
		{
			LogError(Message);
		}

		return false;
	}

	return true;
}

bool UDatasmithConsumer::SetTargetContentFolderImplementation(const FString& InTargetContentFolder, FText& OutFailureReason, const bool bIsAutomated)
{
	if(InTargetContentFolder == TargetContentFolder)
	{
		// #ueent_todo: This is weird it happens in some cases to investigate
		return true;
	}

	if(InTargetContentFolder.StartsWith( TEXT("..")))
	{
		const FText Message = LOCTEXT( "DatasmithConsumer_RelativePath", "Relative path is not accepted as level name. Please enter a valid name." );
		LogInfo(Message);
		return false;
	}

	if(!CanCreateLevel(InTargetContentFolder, LevelName, !bIsAutomated && !IsRunningCommandlet()))
	{
		return false;
	}

	if ( Super::SetTargetContentFolderImplementation( InTargetContentFolder, OutFailureReason, bIsAutomated ) )
	{
		// Inform user if related Datasmith scene is not in package path and force re-creation of Datasmith scene
		const FText Message = FText::Format( LOCTEXT("DatasmithConsumer_SetTargetContentFolder", "Package path {0} different from path previously used. Previous content will not be updated."), FText::FromString( TargetContentFolder ) );
		LogInfo(Message);

		DatasmithSceneWeakPtr.Reset();

		return SetOutputLevel( LevelName );
	}

	return false;
}

void UDatasmithConsumer::UpdateScene()
{
	// Nothing to check if DatasmithScene is explicitly pointing to no object
	if( !DatasmithSceneWeakPtr.IsValid() )
	{
		return;
	}

	// If name of owning Dataprep asset has changed, remove reference to UDatasmithScene 
	// #ueent_todo: Listen to asset renaming event to detect when owning Dataprep's name changes
	FString AssetName;
	DatasmithSceneObjectPath.Split(TEXT("."), nullptr, &AssetName, ESearchCase::CaseSensitive, ESearchDir::FromStart);
	if( !AssetName.StartsWith(GetOuter()->GetName() + DatasmithSceneSuffix) )
	{
		if (UDatasmithScene* OldDatasmithScene = DatasmithSceneWeakPtr.Get())
		{
			// Remove reference to Dataprep asset. The old UDatasmithScene cannot be re-imported anymore
			if ( OldDatasmithScene->GetClass()->ImplementsInterface(UInterface_AssetUserData::StaticClass()) )
			{
				if ( IInterface_AssetUserData* AssetUserDataInterface = Cast< IInterface_AssetUserData >( OldDatasmithScene ) )
				{
					UDataprepAssetUserData* DataprepAssetUserData = AssetUserDataInterface->GetAssetUserData< UDataprepAssetUserData >();

					if ( DataprepAssetUserData )
					{
						DataprepAssetUserData->DataprepAssetPtr.Reset();
						OldDatasmithScene->MarkPackageDirty();
					}
				}
			}
		}

		// Force re-creation of Datasmith scene
		DatasmithSceneWeakPtr.Reset();
	}
}

bool UDatasmithConsumer::SetOutputLevel(const FString& InLevelName)
{
	if(InLevelName.Len() > 0)
	{
		Modify();

		OutputLevelObjectPath = FPaths::Combine( TargetContentFolder, InLevelName) + TEXT(".") + InLevelName;

		MarkPackageDirty();

		OnChanged.Broadcast();

		return true;
	}

	return false;
}

ULevel* UDatasmithConsumer::FindOrAddLevel(const FString& InLevelName)
{
	FString LevelPackageName = FPaths::Combine(TargetContentFolder, InLevelName);

	if(ULevelStreaming* StreamingLevel = FLevelUtils::FindStreamingLevel(WorkingWorld.Get(), *LevelPackageName))
	{
		if(ULevel* Level = StreamingLevel->GetLoadedLevel())
		{
			return Level;
		}
		else
		{
			WorkingWorld->LoadSecondaryLevels();
			ensure(StreamingLevel->GetLoadedLevel());
			return StreamingLevel->GetLoadedLevel();
		}
	}

	ULevel* CurrentLevel = WorkingWorld->PersistentLevel;

	// The level is not part of the world
	FString PackageFilename;
	FPackageName::TryConvertLongPackageNameToFilename( LevelPackageName, PackageFilename, FPackageName::GetMapPackageExtension() );
	FSoftObjectPath LevelSoftObjectPath(LevelPackageName + TEXT(".") + InLevelName);

	ULevelStreaming* StreamingLevel = nullptr;
	// If it already exists on disk or in memory, add it to the working world
	if(FPaths::FileExists(PackageFilename) || LevelSoftObjectPath.ResolveObject())
	{
		FTransform LevelTransform;
		StreamingLevel = UEditorLevelUtils::AddLevelToWorld(WorkingWorld.Get(), *LevelPackageName, ULevelStreamingAlwaysLoaded::StaticClass(), LevelTransform);
		if(StreamingLevel)
		{

			WorkingWorld->LoadSecondaryLevels();
			ensure(StreamingLevel->GetLoadedLevel());
		}
		else
		{
			ensure(false);
		}
	}
	// If it does not exist, create a new one and add it to the working world
	else
	{
		StreamingLevel = EditorLevelUtils::CreateNewStreamingLevelForWorld( *WorkingWorld, ULevelStreamingAlwaysLoaded::StaticClass(), PackageFilename, false, nullptr, false );

		if( StreamingLevel )
		{
			// Register the newly created map asset (associated with this consumer) to the asset registry
			UPackage* WorldPackage = FindPackage(nullptr, *StreamingLevel->GetWorldAssetPackageName());
			ensure(WorldPackage);

			UWorld* World = nullptr;
			ForEachObjectWithPackage(WorldPackage, [&World](UObject* Object)
			{
				if (UWorld* WorldAsset = Cast<UWorld>(Object))
				{
					World = WorldAsset;
					return false;
				}
				return true;
			}, EGetObjectsFlags::None);

			ensure(World);
		
			FAssetRegistryModule::AssetCreated(World);
		}
		else
		{
			ensure(false);
		}
	}

	WorkingWorld->PersistentLevel = CurrentLevel;
	WorkingWorld->SetCurrentLevel(CurrentLevel);

	// Mark level as generated by this consumer
	if(StreamingLevel)
	{
		if(ULevel* NewLevel = StreamingLevel->GetLoadedLevel())
		{
			DatasmithConsumerUtils::SetMarker(NewLevel, ConsumerMarkerID, UniqueID);

			WorkingWorld->AddLevel(NewLevel);

			return NewLevel;
		}
		else
		{
			ensure(false);
		}

	}

	return nullptr;
}

bool UDatasmithConsumer::ValidateAssets()
{
	auto CanCreateAsset = [&](const FString& AssetPathName,const UClass* AssetClass)
	{
		FText OutReason;
		if(!FDatasmithImporterUtils::CanCreateAsset(AssetPathName, AssetClass, OutReason))
		{
			const FTextFormat TextFormat(LOCTEXT( "DatasmithConsumer_CannotCreateAsset", "Cannot create asset {0}. {1}" ));
			const FText Message = FText::Format( TextFormat, FText::FromString(AssetPathName), OutReason );
			LogError(Message);

			return false;
		}

		return true;
	};
	
	// Collect garbage to clear out the destroyed level
	CollectGarbage( GARBAGE_COLLECTION_KEEPFLAGS );

	TArray< TWeakObjectPtr< UObject > > Assets = MoveTemp(Context.Assets);
	Context.Assets.Reserve(Assets.Num());

	// Map holding requested package path and actual package path
	TMap<UObject*, TPair<FString, FString> > AssetPackageInfoMap;
	AssetPackageInfoMap.Reserve(Assets.Num());

	TSet< FString > AssetsToCreate;
	AssetsToCreate.Reserve(Assets.Num());

	TSet< UObject* > AssetsWithIssues;
	AssetsWithIssues.Reserve(Assets.Num());

	for(const TWeakObjectPtr< UObject >& AssetPtr : Assets)
	{
		if(UObject* Asset = AssetPtr.Get())
		{
			TPair<FString, FString>& AssetPackageInfo = AssetPackageInfoMap.Add(Asset);

			bool bAssetWithMarker = false;

			const FString& OutputFolder = DatasmithConsumerUtils::GetMarker(Asset, UDataprepContentConsumer::RelativeOutput);
			if(!OutputFolder.IsEmpty())
			{
				const FString AssetName = Asset->GetName();
				const FSoftObjectPath AssetSoftObjectPath(FPaths::Combine(TargetContentFolder, OutputFolder, AssetName) + "." + AssetName);

				AssetPackageInfo.Key = AssetSoftObjectPath.GetLongPackageName();

				// If asset can be created in memory, do nothing
				if(!CanCreateAsset(AssetSoftObjectPath.GetAssetPathString(), Asset->GetClass()))
				{
				}
				// Verify there is no collision with other assets to be moved
				else if(AssetsToCreate.Contains(AssetPackageInfo.Key))
				{
					const FTextFormat TextFormat(LOCTEXT( "DatasmithConsumer_DuplicateAsset", "Cannot create asset {0}. Another asset with the same name will be created in the same folder {1}." ));
					const FText Message = FText::Format( TextFormat, FText::FromString(AssetSoftObjectPath.GetAssetPathString()), FText::FromString(FPaths::Combine(TargetContentFolder, OutputFolder)) );
					LogError(Message);
				}
				else
				{
					// Asset could be created in memory using output folder directive
					AssetsToCreate.Add(AssetPackageInfo.Key);
					AssetPackageInfo.Value = AssetPackageInfo.Key;
					bAssetWithMarker = true;
				}
			}

			// Check if asset can be created in memory using policy based on target content folder
			if(AssetPackageInfo.Value.IsEmpty())
			{
				const FSoftObjectPath AssetSoftObjectPath(Asset);
				const FString AssetPath = AssetSoftObjectPath.GetAssetPathString().Replace(*Context.TransientContentFolder, *TargetContentFolder);
				const FString LongPackageName = AssetSoftObjectPath.GetLongPackageName().Replace(*Context.TransientContentFolder, *TargetContentFolder);

				if(!CanCreateAsset(AssetPath, Asset->GetClass()))
				{
					// Asset without output folder directive cannot be saved
					if(AssetPackageInfo.Key.IsEmpty())
					{
						AssetPackageInfo.Key = LongPackageName;
					}

					AssetsWithIssues.Add(Asset);
				}
				// Asset with output folder directive could be saved using regular policy
				else if(!AssetPackageInfo.Key.IsEmpty())
				{
					DatasmithConsumerUtils::SetMarker(Asset, UDataprepContentConsumer::RelativeOutput, TEXT(""));
					AssetPackageInfo.Value = LongPackageName;
					AssetsWithIssues.Add(Asset);
				}
				else
				{
					AssetPackageInfo.Key = LongPackageName;
					AssetPackageInfo.Value = AssetPackageInfo.Key;
				}
			}

			// Verify asset can be saved to disk
			if(!AssetPackageInfo.Value.IsEmpty())
			{
				FText OutReason = FText::GetEmpty();

				// If asset cannot be saved to disk using output folder, check if it can do so using regular folder policy
				if(!FDatasmithImporterImpl::CheckAssetPersistenceValidity(AssetPackageInfo.Value, *ImportContextPtr, FPackageName::GetAssetPackageExtension(), OutReason) && bAssetWithMarker)
				{
					LogWarning(OutReason);

					// Overwrite output folder directive
					DatasmithConsumerUtils::SetMarker(Asset, UDataprepContentConsumer::RelativeOutput, TEXT(""));

					FSoftObjectPath AssetSoftObjectPath(Asset);
					const FString AssetPath = AssetSoftObjectPath.GetAssetPathString().Replace(*Context.TransientContentFolder, *TargetContentFolder);

					const FTextFormat Format(LOCTEXT("DatasmithConsumer_OutputFolderFailed", "Cannot save {0} in output folder, trying to save as {1}."));
					OutReason = FText::Format( Format, FText::FromString(Asset->GetName()), FText::FromString(FPaths::GetBaseFilename(AssetPath, false)));
					LogWarning(OutReason);

					if(CanCreateAsset(AssetPath, Asset->GetClass()))
					{
						const FString LongPackageName = AssetSoftObjectPath.GetLongPackageName().Replace(*Context.TransientContentFolder, *TargetContentFolder);
						OutReason = FText::GetEmpty();

						if(FDatasmithImporterImpl::CheckAssetPersistenceValidity(LongPackageName, *ImportContextPtr, FPackageName::GetAssetPackageExtension(), OutReason))
						{
							AssetPackageInfo.Value = LongPackageName;
							Context.Assets.Emplace(Asset);

							AssetsWithIssues.Add(Asset);
						}
						else
						{
							LogWarning(OutReason);

							AssetPackageInfo.Value = FString();
							AssetsWithIssues.Add(Asset);
						}
					}
					else
					{
						AssetPackageInfo.Value = FString();
						AssetsWithIssues.Add(Asset);
					}
				}
				else
				{
					Context.Assets.Emplace(Asset);
				}
			}
		}
	}

	if(AssetsWithIssues.Num() > 0)
	{
		const bool bShowDialog = !Context.bSilentMode && !IsRunningCommandlet();

		
		if(bShowDialog)
		{
			FString AssetsToBeSkippedString;
			FString AssetsNotMovedString;
			for(UObject* Asset : AssetsWithIssues)
			{
				const TPair<FString, FString>& AssetPackageInfo = AssetPackageInfoMap[Asset];
				if(AssetPackageInfo.Value.IsEmpty())
				{
					AssetsToBeSkippedString.Append(FString::Printf(TEXT("\t%s\n"), *AssetPackageInfo.Key));
				}
				else
				{
					AssetsNotMovedString.Append(FString::Printf(TEXT("\t%s will be created as %s\n"), *AssetPackageInfo.Key, *AssetPackageInfo.Value));
				}
			}

			FText AssetsToBeSkippedText;
			if(!AssetsToBeSkippedString.IsEmpty())
			{
				AssetsToBeSkippedText = FText::Format( LOCTEXT( "DatasmithConsumer_AssetsToBeSkipped", "The follwing assets will not be saved:\n{0}\n"), FText::FromString(AssetsToBeSkippedString) );
			}

			FText AssetsNotMovedText;
			if(!AssetsNotMovedString.IsEmpty())
			{
				AssetsNotMovedText = FText::Format( LOCTEXT( "DatasmithConsumer_AssetsNotMoved", "The follwing assets will not be saved in their output folder:\n{0}\n"), FText::FromString(AssetsNotMovedString) );
			}

			const FText Title( LOCTEXT( "DatasmithConsumer_SavingIssues", "Some assets may not be saved properly..." ) );
			const FText Message = FText::Format( LOCTEXT( "DatasmithConsumer_AssetsWithIssues", "All assets cannot be created in their destination folder.\nBelow is the list of assets with issues. See output log for details.\nClick \'Yes\' to continue with the commit.\n\n{0}{1}\n" ), AssetsToBeSkippedText, AssetsNotMovedText);

			if(FMessageDialog::Open(EAppMsgType::YesNo, Message, Title) != EAppReturnType::Yes)
			{
				return false;
			}
		}
		else
		{
			const FText Message = LOCTEXT( "DatasmithConsumer_CommitWithErrors", "All assets could not be saved in their destination folder. Committing anyway. Check your log for details." );
			LogWarning(Message);
		}
	}

	return true;
}

void UDatasmithConsumer::ApplySubLevelDirective(const TArray<UPackage*>& PackagesToCheck)
{
	TMap< FName, TSoftObjectPtr< AActor > >& RelatedActors = ImportContextPtr->ActorsContext.CurrentTargetedScene->RelatedActors;

	TMap<FString, ULevel*> LevelMap;
	TMap<ULevel*, TArray<AActor*>> ActorsToMove;

	LevelMap.Add(LevelName, PrimaryLevel);
	ActorsToMove.Add(PrimaryLevel);

	for(TPair< FName, TSoftObjectPtr< AActor > >& Entry : RelatedActors)
	{
		if(AActor* Actor = Entry.Value.Get())
		{
			ULevel* TargetLevel = PrimaryLevel;

			const FString& OutputDirectiveName = DatasmithConsumerUtils::GetMarker(Actor->GetRootComponent(), UDataprepContentConsumer::RelativeOutput);
			if(OutputDirectiveName.Len() > 0 && OutputDirectiveName != LevelName)
			{
				ULevel* Level = nullptr;
				if(ULevel** OutputLevelPtr = LevelMap.Find(OutputDirectiveName))
				{
					Level = *OutputLevelPtr;
				}
				else
				{
					Level = FindOrAddLevel(OutputDirectiveName);

					if(Level)
					{
						// Tag new level as owned by consumer
						LevelMap.Add(OutputDirectiveName, Level);
						DatasmithConsumerUtils::SetMarker(Level, ConsumerMarkerID, UniqueID);
					}
					else
					{
						FText Message = LOCTEXT( "DatasmithConsumer_ApplySubLevelDirective", "Cannot create level..." );
						LogWarning( Message );
					}
				}

				if(Level)
				{
					TargetLevel = Level;
				}

			}

			if(Actor->GetLevel() != TargetLevel)
			{
				ActorsToMove.FindOrAdd(TargetLevel).Add(Actor);
			}
		}
	}

	TMap<FSoftObjectPath, FSoftObjectPath> AssetRedirectorMap;

	for(TPair<ULevel*, TArray<AActor*>> Entry : ActorsToMove)
	{
		DatasmithConsumerUtils::MoveActorsToLevel( Entry.Value, Entry.Key, RelatedActors, PackagesToCheck, false);
	}
}

namespace DatasmithConsumerUtils
{
	const FString& GetMarker(UObject* Object, const FString& Name)
	{
		if ( IInterface_AssetUserData* AssetUserDataInterface = Cast< IInterface_AssetUserData >( Object ) )
		{
			UDataprepConsumerUserData* DataprepContentUserData = AssetUserDataInterface->GetAssetUserData< UDataprepConsumerUserData >();

			if ( DataprepContentUserData )
			{
				return DataprepContentUserData->GetMarker(Name);
			}
		}

		static FString NullString;

		return NullString;
	}

	void SetMarker(UObject* Object, const FString& Name, const FString& Value)
	{
		if ( IInterface_AssetUserData* AssetUserDataInterface = Cast< IInterface_AssetUserData >( Object ) )
		{
			UDataprepConsumerUserData* DataprepContentUserData = AssetUserDataInterface->GetAssetUserData< UDataprepConsumerUserData >();

			if ( !DataprepContentUserData )
			{
				EObjectFlags Flags = RF_Public;
				DataprepContentUserData = NewObject< UDataprepConsumerUserData >( Object, NAME_None, Flags );
				AssetUserDataInterface->AddAssetUserData( DataprepContentUserData );
			}

			return DataprepContentUserData->AddMarker(Name, Value);
		}
	}

	void ConvertSceneActorsToActors( FDatasmithImportContext& ImportContext )
	{
		UWorld* ImportWorld = ImportContext.ActorsContext.ImportWorld;

		// Find all ADatasmithSceneActor in the world
		TArray< ADatasmithSceneActor* > SceneActorsToConvert;
		TArray<AActor*> Actors( ImportWorld->GetCurrentLevel()->Actors );
		for( AActor* Actor : Actors )
		{
			if( ADatasmithSceneActor* ImportSceneActor = Cast<ADatasmithSceneActor>( Actor ) )
			{
				SceneActorsToConvert.Add( ImportSceneActor );
			}
		}

		// Create the import scene actor for the import context
		ADatasmithSceneActor* RootSceneActor = FDatasmithImporterUtils::CreateImportSceneActor( ImportContext, FTransform::Identity );
		if (RootSceneActor == nullptr)
		{
			return;
		}
		RootSceneActor->Scene = ImportContext.SceneAsset;

		ImportContext.ActorsContext.ImportSceneActor = RootSceneActor;

		// Add existing scene actors as regular actors
		TMap< FName, TSoftObjectPtr< AActor > >& RelatedActors = RootSceneActor->RelatedActors;
		RelatedActors.Reserve( ImportWorld->GetCurrentLevel()->Actors.Num() );

		USceneComponent* NewSceneActorRootComponent = RootSceneActor->GetRootComponent();
		ImportContext.Hierarchy.Push( NewSceneActorRootComponent );

		TArray<AActor*> ActorsToVisit;

		for( ADatasmithSceneActor* SceneActor : SceneActorsToConvert )
		{
			// Create AActor to replace scene actor
			const FString SceneActorName = SceneActor->GetName();
			const FString SceneActorLabel = SceneActor->GetActorLabel();
			SceneActor->Rename( nullptr, nullptr, REN_DontCreateRedirectors | REN_NonTransactional );

			// Use actor's label instead of name.
			// Rationale: Datasmith scene actors are created with the same name and label and their name can change when calling SetLabel.
			TSharedRef< IDatasmithActorElement > RootActorElement = FDatasmithSceneFactory::CreateActor( *SceneActorLabel );
			RootActorElement->SetLabel( *SceneActorLabel );

			AActor* Actor = FDatasmithActorImporter::ImportBaseActor( ImportContext, RootActorElement );
			check( Actor && Actor->GetRootComponent());

			FDatasmithImporter::ImportMetaDataForObject( ImportContext, RootActorElement, Actor );

			// Copy Tags
			Actor->Tags = MoveTemp(SceneActor->Tags);

			// Copy the transforms
			USceneComponent* ActorRootComponent = Actor->GetRootComponent();
			check( ActorRootComponent );

			USceneComponent* SceneActorRootComponent = SceneActor->GetRootComponent();

			ActorRootComponent->SetRelativeTransform( SceneActorRootComponent->GetRelativeTransform() );
			ActorRootComponent->SetComponentToWorld( SceneActorRootComponent->GetComponentToWorld() );

			// Reparent children of root scene actor to new root actor
			TArray<USceneComponent*> AttachedChildren;
			SceneActor->GetRootComponent()->GetChildrenComponents(false, AttachedChildren);

			for(USceneComponent* SceneComponent : AttachedChildren)
			{
				SceneComponent->AttachToComponent( ActorRootComponent, FAttachmentTransformRules::KeepRelativeTransform );
			}

			// Attach new actor to root scene actor
			ActorRootComponent->AttachToComponent( NewSceneActorRootComponent, FAttachmentTransformRules::KeepRelativeTransform );

			// Copy AssetUserData - it is done by known classes but should be improved
			if ( IInterface_AssetUserData* SourceAssetUserDataInterface = Cast< IInterface_AssetUserData >( SceneActorRootComponent ) )
			{
				if(IInterface_AssetUserData* TargetAssetUserDataInterface = Cast< IInterface_AssetUserData >(ActorRootComponent))
				{
					if(UDatasmithAssetUserData* SourceDatasmithUserData = SourceAssetUserDataInterface->GetAssetUserData<UDatasmithAssetUserData>())
					{
						if(UDatasmithAssetUserData* TargetDatasmithUserData = TargetAssetUserDataInterface->GetAssetUserData<UDatasmithAssetUserData>())
						{
							TargetDatasmithUserData->MetaData.Append(SourceDatasmithUserData->MetaData);
							TargetDatasmithUserData->ObjectTemplates.Append(SourceDatasmithUserData->ObjectTemplates);
						}
						else
						{
							TargetDatasmithUserData = DuplicateObject<UDatasmithAssetUserData>(SourceDatasmithUserData, ActorRootComponent);
							TargetAssetUserDataInterface->AddAssetUserData(TargetDatasmithUserData);
						}
					}

					if(UAssetUserData* SourceConsumerUserData = SourceAssetUserDataInterface->GetAssetUserData<UDataprepConsumerUserData>())
					{
						UAssetUserData* TargetConsumerUserData = DuplicateObject<UAssetUserData>(SourceConsumerUserData, ActorRootComponent);
						TargetAssetUserDataInterface->AddAssetUserData(TargetConsumerUserData);
					}
				}
			}


			// Delete root scene actor since it is not needed anymore
			ImportWorld->DestroyActor( SceneActor, false, true );
			SceneActor->UnregisterAllComponents();

			SceneActor->Rename( nullptr, GetTransientPackage(), REN_DontCreateRedirectors | REN_NonTransactional );

			Actor->RegisterAllComponents();

			// Append children of actor to be later added as related actors
			TArray<AActor*> Children;
			Actor->GetAttachedActors( Children );

			ActorsToVisit.Append( Children );
		}

		// Recursively add all children of previous scene actors as related to new scene actor
		while ( ActorsToVisit.Num() > 0)
		{
			AActor* VisitedActor = ActorsToVisit.Pop();
			if (VisitedActor == nullptr)
			{
				continue;
			}

			// Add visited actor as actor related to scene actor
			RelatedActors.Add( FName( *GetObjectUniqueId( VisitedActor ) ), VisitedActor );

			// Continue with children
			TArray<AActor*> Children;
			VisitedActor->GetAttachedActors( Children );

			ActorsToVisit.Append( Children );
		}

		auto IsUnregisteredActor = [&](AActor* Actor)
		{
			// Skip non-imported actors
			if( Actor == RootSceneActor || Actor == nullptr || Actor->GetRootComponent() == nullptr || Actor->IsA<AWorldSettings>() || Actor->IsA<APhysicsVolume>() || Actor->IsA<ABrush>() || Actor->IsA<ARectLight>())
			{
				if (Actor != nullptr && Actor->IsA<ARectLight>())
				{
					UE_LOGF(LogDatasmithImport, Warning, "Import of RectLight is not yet supported");
				}
				return false;
			}

			// Skip actor which we have already processed
			return RelatedActors.Find( *GetObjectUniqueId( Actor ) ) ? false : true;
		};

		// Find remaining root actors (non scene actors)
		TArray< AActor* > RootActors;
		for( AActor* Actor : ImportWorld->GetCurrentLevel()->Actors )
		{
			if( IsUnregisteredActor( Actor ) )
			{
				// Find root actor
				AActor* RootActor = Actor;

				while( RootActor->GetAttachParentActor() != nullptr )
				{
					RootActor = RootActor->GetAttachParentActor();
				}

				// Attach root actor to root scene actor
				RootActor->GetRootComponent()->AttachToComponent( NewSceneActorRootComponent, FAttachmentTransformRules::KeepRelativeTransform );

				// Add root actor and its children as related to new scene actor
				ActorsToVisit.Add( RootActor );

				while ( ActorsToVisit.Num() > 0)
				{
					AActor* VisitedActor = ActorsToVisit.Pop();
					if (VisitedActor == nullptr)
					{
						continue;
					}

					// Add visited actor as actor related to scene actor
					RelatedActors.Add( FName( *GetObjectUniqueId( VisitedActor ) ), VisitedActor );

					// Continue with children
					TArray<AActor*> Children;
					VisitedActor->GetAttachedActors( Children );

					ActorsToVisit.Append( Children );
				}
			}
		}
	}

	void AddAssetsToContext(FDatasmithImportContext& ImportContext, TArray<TWeakObjectPtr<UObject>>& Assets)
	{
		// Addition is done in 2 passes to properly collect UMaterial objects referenced by UMaterialInstance ones
		// Templates are added to assets which have not been created through Datasmith

		// Add template and Datasmith unique Id to source object
		auto AddTemplate = [](UClass* TemplateClass, UObject* Source)
		{
			UDatasmithObjectTemplate* DatasmithTemplate = NewObject< UDatasmithObjectTemplate >( Source, TemplateClass );
			DatasmithTemplate->Load( Source );
			FDatasmithObjectTemplateUtils::SetObjectTemplate( Source, DatasmithTemplate );

			UDatasmithAssetUserData::SetDatasmithUserDataValueForKey(Source, UDatasmithAssetUserData::UniqueIdMetaDataKey, Source->GetName() );
		};

		// First skip UMaterial objects which are not referenced by a UmaterialInstance one
		int32 MaterialCount = 0;
		TSet< UMaterialInterface* > ParentMaterials;
		TSet< UMaterialFunctionInterface* > MaterialFunctions;
		for(TWeakObjectPtr<UObject>& AssetPtr : Assets)
		{
			if( UObject* Asset = AssetPtr.Get() )
			{
				FString AssetTag = FDatasmithImporterUtils::GetDatasmithElementIdString( Asset );

				if(UTexture* Texture = Cast<UTexture>(Asset))
				{
					TSharedRef< IDatasmithTextureElement > TextureElement = FDatasmithSceneFactory::CreateTexture( *AssetTag );
					TextureElement->SetLabel( *Texture->GetName() );

					UE::Interchange::FAssetImportResultRef& AssetImportResults = ImportContext.ImportedTextures.Add( TextureElement, MakeShared< UE::Interchange::FImportResult, ESPMode::ThreadSafe >() );
					AssetImportResults->AddImportedObject( Texture );
					AssetImportResults->SetDone();

					ImportContext.Scene->AddTexture( TextureElement );
				}
				else if(UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(Asset))
				{
					TSharedRef< IDatasmithBaseMaterialElement > MaterialElement = FDatasmithSceneFactory::CreateMaterial( *AssetTag );
					MaterialElement->SetLabel( *MaterialInstance->GetName() );

					if ( UMaterialInterface* MaterialParent = MaterialInstance->Parent )
					{
						FString MaterialInstancePath = MaterialInstance->GetOutermost()->GetName();
						FString ParentPath = MaterialParent->GetOutermost()->GetName();

						// Add parent material to ImportedParentMaterials if applicable
						if ( ParentPath.StartsWith( MaterialInstancePath ) )
						{
							ImportContext.ImportedParentMaterials.Add( MaterialCount, MaterialParent );
							MaterialCount++;

							ParentMaterials.Add( MaterialParent );
						}
					}

					if(UMaterialInstanceConstant* MaterialInstanceConstant = Cast<UMaterialInstanceConstant>(MaterialInstance))
					{
						if(!FDatasmithObjectTemplateUtils::GetObjectTemplate<UDatasmithMaterialInstanceTemplate>( MaterialInstanceConstant ))
						{
							AddTemplate( UDatasmithMaterialInstanceTemplate::StaticClass(), MaterialInstanceConstant );
						}
					}

					ImportContext.ImportedMaterials.Add( MaterialElement, MaterialInstance );
					ImportContext.Scene->AddMaterial( MaterialElement );
				}
				else if(UStaticMesh* StaticMesh = Cast<UStaticMesh>(Asset))
				{
					// Clean up static meshes which have incomplete render data.
					if(StaticMesh->GetRenderData() && !StaticMesh->GetRenderData()->IsInitialized())
					{
						StaticMesh->SetRenderData(nullptr);
					}

					if(FDatasmithObjectTemplateUtils::GetObjectTemplate<UDatasmithStaticMeshTemplate>( StaticMesh ) == nullptr)
					{
						AddTemplate( UDatasmithStaticMeshTemplate::StaticClass(), StaticMesh );
					}

					TSharedRef< IDatasmithMeshElement > MeshElement = FDatasmithSceneFactory::CreateMesh( *AssetTag );
					MeshElement->SetLabel( *StaticMesh->GetName() );


					for(int32 Index = 0; Index < StaticMesh->GetNumSections( 0 ); ++Index)
					{
						const FString MaterialTag = FDatasmithImporterUtils::GetDatasmithElementIdString( StaticMesh->GetMaterial( Index ) );
						MeshElement->SetMaterial( *MaterialTag, Index );
					}

					ImportContext.ImportedStaticMeshes.Add( MeshElement, StaticMesh );
					ImportContext.Scene->AddMesh( MeshElement );
				}
				else if(ULevelSequence* LevelSequence = Cast<ULevelSequence>(Asset))
				{
					TSharedRef< IDatasmithLevelSequenceElement > LevelSequenceElement = FDatasmithSceneFactory::CreateLevelSequence( *AssetTag );
					LevelSequenceElement->SetLabel( *LevelSequence->GetName() );

					ImportContext.ImportedLevelSequences.Add( LevelSequenceElement, LevelSequence );
					ImportContext.Scene->AddLevelSequence( LevelSequenceElement );
				}
				else if(ULevelVariantSets* LevelVariantSets = Cast<ULevelVariantSets>(Asset))
				{
					TSharedRef< IDatasmithLevelVariantSetsElement > LevelVariantSetsElement = FDatasmithSceneFactory::CreateLevelVariantSets( *AssetTag );
					LevelVariantSetsElement->SetLabel( *LevelVariantSets->GetName() );

					ImportContext.ImportedLevelVariantSets.Add( LevelVariantSetsElement, LevelVariantSets );
					ImportContext.Scene->AddLevelVariantSets( LevelVariantSetsElement );
				}
				// #ueent_todo: Add support for assets which are not of the classes above
			}
		}

		// Second take care UMaterial objects which are not referenced by a UmaterialInstance one
		for( TWeakObjectPtr<UObject>& AssetPtr : Assets )
		{
			UObject* AssetObject = AssetPtr.Get();
			if( UMaterial* Material = Cast<UMaterial>( AssetObject ) )
			{
				if( !ParentMaterials.Contains( Material ) )
				{
					FString AssetTag = FDatasmithImporterUtils::GetDatasmithElementIdString( Material );
					TSharedRef< IDatasmithMaterialElement > MaterialElement = FDatasmithSceneFactory::CreateMaterial( *AssetTag );
					MaterialElement->SetLabel( *Material->GetName() );

					ImportContext.ImportedMaterials.Add( MaterialElement, Material );
					ImportContext.Scene->AddMaterial( MaterialElement );
				}
			}
			else if( UMaterialFunction* MaterialFunction = Cast<UMaterialFunction>( AssetObject ) )
			{
				if( !MaterialFunctions.Contains( Cast<UMaterialFunctionInterface>( MaterialFunction ) ) )
				{
					FString AssetTag = FDatasmithImporterUtils::GetDatasmithElementIdString( MaterialFunction );

					TSharedRef< IDatasmithUEPbrMaterialElement > UEPbrMaterialFunctionElement = FDatasmithSceneFactory::CreateUEPbrMaterial( *AssetTag );

					UEPbrMaterialFunctionElement->SetLabel( *MaterialFunction->GetName() );
					UEPbrMaterialFunctionElement->SetMaterialFunctionOnly( true );

					TSharedRef< IDatasmithBaseMaterialElement > BaseMaterialElement = StaticCastSharedRef< IDatasmithBaseMaterialElement >( UEPbrMaterialFunctionElement );

					ImportContext.ImportedMaterialFunctions.Add( BaseMaterialElement, MaterialFunction );
					ImportContext.ImportedMaterialFunctionsByName.Add( BaseMaterialElement->GetName(), BaseMaterialElement );

					ImportContext.Scene->AddMaterial( BaseMaterialElement );
				}
			}
		}
	}

	void SaveMap(UWorld* WorldToSave)
	{
		const bool bHasStandaloneFlag = WorldToSave->HasAnyFlags(RF_Standalone);
		FSoftObjectPath WorldSoftObject(WorldToSave);

		// Delete map file if it already exists
		FString PackageFilename;
		FPackageName::TryConvertLongPackageNameToFilename( WorldSoftObject.GetLongPackageName(), PackageFilename, FPackageName::GetMapPackageExtension() );

		IFileManager::Get().Delete(*PackageFilename, /*RequireExists=*/ false, /*EvenReadOnly=*/ true, /*Quiet=*/ true);

		// Add RF_Standalone flag to properly save the completed world
		WorldToSave->SetFlags(RF_Standalone);

		UEditorLoadingAndSavingUtils::SaveMap(WorldToSave, WorldSoftObject.GetLongPackageName() );

		// Clear RF_Standalone from flag to properly delete and garbage collect the completed world
		if(!bHasStandaloneFlag)
		{
			WorldToSave->ClearFlags(RF_Standalone);
		}

		WorldToSave->GetOutermost()->SetDirtyFlag(false);
	}

	TArray<AActor*> MoveActorsToLevel(const TArray<AActor*>& ActorsToMove, ULevel* DestLevel, const TArray<UPackage*>& PackagesToCheck, bool bDuplicate)
	{
		if(DestLevel == nullptr || ActorsToMove.Num() == 0)
		{
			return TArray<AActor*>();
		}

		UWorld* OwningWorld = DestLevel->OwningWorld;

		// Backup the current contents of the clipboard string as we'll be using cut/paste features to move actors
		// between levels and this will trample over the clipboard data.
		FString OriginalClipboardContent;
		FPlatformApplicationMisc::ClipboardPaste(OriginalClipboardContent);

		TMap<FSoftObjectPath, FSoftObjectPath> ActorPathMapping;
		GEditor->SelectNone(false, true, false);

		USelection* ActorSelection = GEditor->GetSelectedActors();
		ActorSelection->BeginBatchSelectOperation();
		for (AActor* Actor : ActorsToMove)
		{
			ActorPathMapping.Add(FSoftObjectPath(Actor), FSoftObjectPath());
			GEditor->SelectActor(Actor, true, false);
		}
		ActorSelection->EndBatchSelectOperation(false);

		if(GEditor->GetSelectedActorCount() == 0)
		{
			return TArray<AActor*>();
		}

		// Cache the old level
		ULevel* OldCurrentLevel = OwningWorld->GetCurrentLevel();

		// If we are moving the actors, cut them to remove them from the existing level
		const bool bShoudCut = !bDuplicate;
		const bool bIsMove = bShoudCut;
		GEditor->CopySelectedActorsToClipboard(OwningWorld, bShoudCut, bIsMove, /*bWarnAboutReferences =*/ false);

		UEditorLevelUtils::SetLevelVisibility(DestLevel, true, false, ELevelVisibilityDirtyMode::DontModify);

		// Set the new level and force it visible while we do the paste
		OwningWorld->SetCurrentLevel(DestLevel);

		//const bool bDuplicate = false;
		const bool bOffsetLocations = false;
		const bool bWarnIfHidden = false;
		GEditor->edactPasteSelected(OwningWorld, bDuplicate, bOffsetLocations, bWarnIfHidden);

		// Restore the original current level
		OwningWorld->SetCurrentLevel(OldCurrentLevel);
		
		TArray<AActor*> NewActors;
		NewActors.Reserve(GEditor->GetSelectedActorCount());

		// Build a remapping of old to new names so we can do a fixup
		for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
		{
			AActor* Actor = static_cast<AActor*>(*It);
			if(!Actor)
			{
				continue;
			}

			NewActors.Add(Actor);
			FSoftObjectPath NewPath = FSoftObjectPath(Actor);

			bool bFoundMatch = false;

			// First try exact match
			for (TPair<FSoftObjectPath, FSoftObjectPath>& Pair : ActorPathMapping)
			{
				if (Pair.Value.IsNull() && NewPath.GetSubPathString() == Pair.Key.GetSubPathString())
				{
					bFoundMatch = true;
					Pair.Value = NewPath;
					break;
				}
			}

			if (!bFoundMatch)
			{
				// Remove numbers from end as it may have had to add some to disambiguate
				FString PartialPath = NewPath.GetSubPathString();
				int32 IgnoreNumber;
				FActorLabelUtilities::SplitActorLabel(PartialPath, IgnoreNumber);

				for (TPair<FSoftObjectPath, FSoftObjectPath>& Pair : ActorPathMapping)
				{
					if (Pair.Value.IsNull())
					{
						FString KeyPartialPath = Pair.Key.GetSubPathString();
						FActorLabelUtilities::SplitActorLabel(KeyPartialPath, IgnoreNumber);
						if (PartialPath == KeyPartialPath)
						{
							bFoundMatch = true;
							Pair.Value = NewPath;
							break;
						}
					}
				}
			}

			if (!bFoundMatch)
			{
				UE_LOGF(LogDatasmithImport, Error, "Cannot find remapping for moved actor ID %ls, any soft references pointing to it will be broken!", *Actor->GetPathName());
			}
		}

		IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
		TArray<FAssetRenameData> RenameData;

		for (TPair<FSoftObjectPath, FSoftObjectPath>& Pair : ActorPathMapping)
		{
			if (Pair.Value.IsValid())
			{
				RenameData.Add(FAssetRenameData(Pair.Key, Pair.Value, true));
			}
		}

		if (RenameData.Num() > 0)
		{
			AssetTools.RenameAssets(RenameData);

			// Fix soft references in level sequences and variants
			if(PackagesToCheck.Num() > 0)
			{
				AssetTools.RenameReferencingSoftObjectPaths(PackagesToCheck, ActorPathMapping);
			}
		}

		// Restore the original clipboard contents
		FPlatformApplicationMisc::ClipboardCopy(*OriginalClipboardContent);

		return NewActors;
	}

	void MoveActorsToLevel(const TArray<AActor*>& ActorsToMove, ULevel* DestLevel, TMap<FName,TSoftObjectPtr<AActor>>& ActorsMap, const TArray<UPackage*>& PackagesToCheck, bool bDuplicate)
	{
		if(ActorsToMove.Num() > 0)
		{
			UWorld* PrevGWorld = GWorld;
			GWorld = DestLevel->OwningWorld;

			// Cache Destination flags
			EObjectFlags DestLevelFlags = DestLevel->GetFlags();
			EObjectFlags DestWorldFlags = DestLevel->GetOuter()->GetFlags();
			EObjectFlags DestPackageFlags = DestLevel->GetOutermost()->GetFlags();

			TArray<AActor*> NewActors = MoveActorsToLevel( ActorsToMove, DestLevel, PackagesToCheck, bDuplicate);

			GWorld = PrevGWorld;

			// Update map of related actors with new actors
			UDatasmithContentBlueprintLibrary* DatasmithContentLibrary = Cast< UDatasmithContentBlueprintLibrary >( UDatasmithContentBlueprintLibrary::StaticClass()->GetDefaultObject() );

			for(AActor* Actor : DestLevel->Actors)
			{
				const FString DatasmithUniqueId = DatasmithContentLibrary->GetDatasmithUserDataValueForKey( Actor, UDatasmithAssetUserData::UniqueIdMetaDataKey );
				if(DatasmithUniqueId.Len() > 0)
				{
					if(TSoftObjectPtr<AActor>* SoftObjectPtr = ActorsMap.Find(FName(*DatasmithUniqueId)))
					{
						*SoftObjectPtr = Actor;
					}
				}
			}

			// Restore Destination flags
			DestLevel->SetFlags(DestLevelFlags);
			DestLevel->GetOuter()->SetFlags(DestWorldFlags);
			DestLevel->GetOutermost()->SetFlags(DestPackageFlags);
		}
	}
}

#undef LOCTEXT_NAMESPACE
