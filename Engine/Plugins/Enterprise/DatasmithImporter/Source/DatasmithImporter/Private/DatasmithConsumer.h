// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DatasmithImporter.h"
#include "DatasmithScene.h"

#include "DataprepContentConsumer.h"

#include "UObject/SoftObjectPath.h"
#include "UObject/StrongObjectPtr.h"

#include "DatasmithConsumer.generated.h"

namespace DatasmithConsumerDetailsUtil
{
	class SLevelProperty;
}
class UDatasmithScene;
class ULevel;
class UPackage;
class UWorld;

UCLASS(Experimental, config = EditorSettings, HideCategories = (DatasmithConsumerInternal))
class DATASMITHIMPORTER_API UDatasmithConsumer : public UDataprepContentConsumer
{
	GENERATED_BODY()

public:
	UDatasmithConsumer();

	// UObject interface
	virtual void PostLoad() override;
	virtual void PostInitProperties() override;
	// End of UObject interface

	/** DEPRECATED: Removing use of TSoftObjectPtr */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use GetDatasmithScene method to obtain the associated UDatasmithScene.") )
	TSoftObjectPtr<UDatasmithScene> DatasmithScene_DEPRECATED;

	/** Stores the level used on the last call to UDatasmithConsumer::Run */
	UPROPERTY( BlueprintReadOnly, Category = DatasmithConsumerInternal )
	FString UniqueID;


	/** Marker to tag levels, assets and actors processed by the consumer */
	static const FString ConsumerMarkerID;

	// Begin UDataprepContentConsumer overrides
	virtual const FText& GetLabel() const override;
	virtual const FText& GetDescription() const override;

	UFUNCTION(BlueprintCallable, Category = DatasmithConsumerInternal)
	UDatasmithScene* GetDatasmithScene() { return DatasmithSceneWeakPtr.Get(); }

protected:
	/** Getter/setter on consumer's output level */
	FString GetOutputLevelPath() const { return OutputLevelObjectPath; }
	bool SetOutputLevel(const FString& LevelName);

	virtual bool SetLevelNameImplementation(const FString& InLevelName, FText& OutFailureReason, const bool bIsAutomated) override;
	virtual bool SetTargetContentFolderImplementation(const FString& InTargetContentFolder, FText& OutFailureReason, const bool bIsAutomated) override;

	virtual bool Initialize() override;
	virtual bool Run() override;
	virtual void Reset() override;
	// End UDataprepContentConsumer overrides

	/** Path to UDatasmithScene potentially linked to */
	UPROPERTY( BlueprintReadOnly, Category = DatasmithConsumerInternal )
	FString DatasmithSceneObjectPath;

	/** Path to ULevel potentially linked to */
	UPROPERTY( BlueprintReadOnly, Category = DatasmithConsumerInternal )
	FString OutputLevelObjectPath;

private:
	/** Temporary code to work with UDataprepContentConsumer */
	bool BuildContexts();

	/** Move assets if destination package path has changed since last call to UDatasmithConsumer::Run */
	void UpdateScene();

	/** Process actors which have been marked with a UDataprepConsumerUserData */
	void ApplySubLevelDirective(const TArray<UPackage*>& PackagesToCheck);

	/** Find or add a new level to the target world */
	ULevel* FindOrAddLevel(const FString& InLevelName);

	/**
	 * Check if output directives set on assets or actors can be honored
	 * If some issues are identified, the user is asked to proceed with execution of consumer or not.
	 * @return True if no issue or user has accepted to continue with execution of consumer
	 * If in command mode, the default behavior is to continue.
	 */
	bool ValidateAssets();

	/** Create the world which will be used to execute the consumer */
	bool CreateWorld();

	/** Delete the world which was used to execute the consumer */
	void ClearWorld();

	/** Save all assets and levels */
	bool FinalizeRun();

	/** Returns true if a umap file can be saved in specified folder with specified name */
	bool CanCreateLevel( const FString& RequestedFolder, const FString& RequestedName, const bool bShowDialog);

private:
	TUniquePtr< FDatasmithImportContext > ImportContextPtr;
	TUniquePtr< FDataprepWorkReporter > ProgressTaskPtr;

	/** World used by the consumer to create new output levels */
	TStrongObjectPtr<UWorld> WorkingWorld;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = ""))
	FSoftObjectPath OutputLevelSoftObject_DEPRECATED;

	UPROPERTY(Transient)
	TWeakObjectPtr<UDatasmithScene> DatasmithSceneWeakPtr;

	/** Level associated with the consumer */
	ULevel* PrimaryLevel;

	/** Current level used by the global world before executing the consumer */
	ULevel* OriginalCurrentLevel;
};