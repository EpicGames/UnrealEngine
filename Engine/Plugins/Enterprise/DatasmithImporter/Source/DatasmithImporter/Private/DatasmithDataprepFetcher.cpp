// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithDataprepFetcher.h"

#include "DatasmithContentBlueprintLibrary.h"

#define LOCTEXT_NAMESPACE "DatasmithDataprepFetcher"

/* UDataprepStringMetadataValueFetcher methods
 *****************************************************************************/
TArray<FString> UDatasmithStringMetadataValueFetcher::Fetch_Implementation(const UObject* Object, bool& bOutFetchSucceded) const
{
	if ( Object ) 
	{
		bOutFetchSucceded = true;
		bool bMatchPartialKey = KeyMatch == EMetadataKeyMatchingCriteria::Contains;
		return UDatasmithContentBlueprintLibrary::GetDatasmithUserDataValuesForKey( const_cast< UObject* >( Object ), Key, bMatchPartialKey);
	}

	bOutFetchSucceded = false;
	return {};
}

FText UDatasmithStringMetadataValueFetcher::GetNodeDisplayFetcherName_Implementation() const
{
	return LOCTEXT("NodeDisplayFetcher_MetadataValue", "Metadata");
}

bool UDatasmithStringMetadataValueFetcher::IsThreadSafe() const
{
	return true;
}

#undef LOCTEXT_NAMESPACE
