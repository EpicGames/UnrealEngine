// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SelectionSystem/DataprepStringsArrayFetcher.h"

#include "CoreMinimal.h"

#include "DatasmithDataprepFetcher.generated.h"

UENUM()
enum EMetadataKeyMatchingCriteria : int
{
	ExactMatch,
	Contains
};

UCLASS(BlueprintType, NotBlueprintable, Meta = (DisplayName="Metadata Value", ToolTip="Filter objects based on the key value of their metadata."))
class UDatasmithStringMetadataValueFetcher final : public UDataprepStringsArrayFetcher
{
	GENERATED_BODY()
public:
	//~ UDataprepStringsArrayFetcher interface
	virtual TArray<FString> Fetch_Implementation(const UObject* Object, bool& bOutFetchSucceded) const override;
	//~ End of UDataprepStringsArrayFetcher interface

	//~ UDataprepFetcher interface
	virtual FText GetNodeDisplayFetcherName_Implementation() const;
	virtual bool IsThreadSafe() const final;
	//~ End of UDataprepFetcher interface

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "Key")
	TEnumAsByte<EMetadataKeyMatchingCriteria> KeyMatch;

	// The key for the for the string
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "Key")
	FName Key;
};
