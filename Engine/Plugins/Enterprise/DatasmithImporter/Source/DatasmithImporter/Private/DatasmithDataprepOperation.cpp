// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithDataprepOperation.h"

#include "DatasmithBlueprintLibrary.h"
#include "Utility/DatasmithImporterUtils.h"

#define LOCTEXT_NAMESPACE "DatasmithDataprepOperation"

void UDataprepSetupStaticLightingOperation::OnExecution_Implementation(const FDataprepContext& InContext)
{
#ifdef LOG_TIME
	DataprepOperationTime::FTimeLogger TimeLogger(TEXT("SetupStaticLighting"), [&](FText Text) { this->LogInfo(Text); });
#endif

	// Execute operation
	UDatasmithStaticMeshBlueprintLibrary::SetupStaticLighting(InContext.Objects, false, bEnableLightmapUVGeneration, LightmapResolutionIdealRatio);
}

#undef LOCTEXT_NAMESPACE
