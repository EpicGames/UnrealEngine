// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DataprepOperation.h"

#include "DatasmithDataprepOperation.generated.h"

UCLASS(Experimental, Category = LightmapOptions, Meta = (DisplayName = "Setup Static Lighting", ToolTip = "For each static mesh to process, setup the settings to enable lightmap UVs generation and compute the lightmap resolution."))
class UDataprepSetupStaticLightingOperation : public UDataprepOperation
{
	GENERATED_BODY()

		UDataprepSetupStaticLightingOperation()
		: bEnableLightmapUVGeneration(true),
		LightmapResolutionIdealRatio(0.2f)
	{
	}

public:
	// The value to set for the generate lightmap uvs flag on each static mesh
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LightmapOptions, meta = (DisplayName = "Enable Lightmap UV Generation", ToolTip = "Enable the lightmap UV generation."))
	bool bEnableLightmapUVGeneration;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LightmapOptions, meta = (DisplayName = "Resolution Ideal Ratio", ToolTip = "The ratio used to compute the resolution of the lightmap."))
	float LightmapResolutionIdealRatio;

protected:
	//~ Begin UDataprepOperation Interface
public:
	virtual FText GetCategory_Implementation() const override
	{
		return FDataprepOperationCategories::MeshOperation;
	}

protected:
	virtual void OnExecution_Implementation(const FDataprepContext& InContext) override;
	//~ End UDataprepOperation Interface
};