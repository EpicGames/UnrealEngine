// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithFileProducer.h"

#include "Engine/Level.h"
#include "Engine/TextureLightProfile.h"
#include "IDataprepLogger.h"
#include "IDataprepProgressReporter.h"

#include "DatasmithActorImporter.h"
#include "DatasmithAssetUserData.h"
#include "DatasmithContentBlueprintLibrary.h"
#include "DatasmithImportContext.h"
#include "DatasmithImportOptions.h"
#include "DatasmithImporter.h"
#include "DatasmithScene.h"
#include "DatasmithSceneActor.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithStaticMeshImporter.h"
#include "DataprepAssetProducers.h"
#include "DatasmithTranslatorManager.h"
#include "IDatasmithSceneElements.h"
#include "Utility/DatasmithImporterUtils.h"

#include "Async/ParallelFor.h"
#include "Algo/Count.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "Containers/UnrealString.h"
#include "DesktopPlatformModule.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "EditorDirectories.h"
#include "EditorFontGlyphs.h"
#include "Engine/StaticMesh.h"
#include "ExternalSource.h"
#include "ExternalSourceModule.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/FileManager.h"
#include "IDesktopPlatform.h"
#include "Input/HittestGrid.h"
#include "Internationalization/Internationalization.h"
#include "Interfaces/IMainFrameModule.h"
#include "IUriManager.h"
#include "LevelSequence.h"
#include "LevelVariantSets.h"
#include "Materials/Material.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialInstance.h"
#include "MaterialCachedData.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "ObjectTools.h"
#include "ObjectEditorUtils.h"
#include "PropertyHandle.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "SourceUri.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "DatasmithFileProducer"

const FText DatasmithFileProducerLabel( LOCTEXT( "DatasmithFileProducerLabel", "Datasmith file importer" ) );
const FText DatasmithFileProducerDescription( LOCTEXT( "DatasmithFileProducerDesc", "Reads a Datasmith or CAD file and its dependent assets" ) );

const FText DatasmithDirProducerLabel( LOCTEXT( "DatasmithDirProducerLabel", "Datasmith folder importer" ) );
const FText DatasmithDirProducerDescription( LOCTEXT( "DatasmithDirProducerDesc", "Reads all Datasmith or CAD files and their dependent assets from a directory" ) );

const TCHAR* WildCharCriteria = TEXT( "*.*" );
const TCHAR* ExtensionPrefix = TEXT( "*." );

TSet< FString > UDatasmithDirProducer::SupportedFormats;

FDatasmithTessellationOptions UDatasmithFileProducer::DefaultTessellationOptions( 0.3f, 0.0f, 30.0f, EDatasmithCADStitchingTechnique::StitchingSew );
FDatasmithImportBaseOptions UDatasmithFileProducer::DefaultImportOptions;

namespace FDatasmithFileProducerUtils
{
	/** Delete all the assets stored under the specified path */
	void DeletePackagePath( const FString& PathToDelete );

	/** Delete all the packages created by the Datasmith importer */
	void DeletePackagesPath(TSet<FString>& PathsToDelete)
	{
		for(FString& PathToDelete : PathsToDelete)
		{
			DeletePackagePath( PathToDelete );
		}
	}

	/** Display OS browser, i.e. Windows explorer, to let user select a file */
	FString SelectFileToImport();

	/** Display OS browser, i.e. Windows explorer, to let user select a directory */
	FString SelectDirectory();


	class SFileProducerImportOptionsWindow : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SFileProducerImportOptionsWindow)
		{}
		SLATE_ARGUMENT(TArray<UObject*>, ImportOptions)
		SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
		SLATE_ARGUMENT(int32, MinDetailHeight)
		SLATE_ARGUMENT(int32, MinDetailWidth)
		SLATE_END_ARGS()

	public:
		void Construct(const FArguments& InArgs)
		{
			ImportOptions = InArgs._ImportOptions;
			Window = InArgs._WidgetWindow;

			TSharedPtr<SBox> DetailsViewBox;

			ChildSlot
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2)
				[
					SAssignNew(DetailsViewBox, SBox)
					.MinDesiredHeight(InArgs._MinDetailHeight)
					.MinDesiredWidth(InArgs._MinDetailWidth)
				]
			];

			FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
			FDetailsViewArgs DetailsViewArgs;
			DetailsViewArgs.bAllowSearch = false;
			DetailsViewArgs.bAllowMultipleTopLevelObjects = true;
			DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
			TSharedPtr<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

			DetailsViewBox->SetContent(DetailsView.ToSharedRef());
			DetailsView->SetObjects(ImportOptions);
		}

		virtual bool SupportsKeyboardFocus() const override { return true; }

		virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
		{
			if (InKeyEvent.GetKey() == EKeys::Escape)
			{
				if (Window.IsValid())
				{
					Window.Pin()->RequestDestroyWindow();
				}
				return FReply::Handled();
			}

			return FReply::Unhandled();
		}

	private:
		TArray<UObject*> ImportOptions;
		TWeakPtr< SWindow > Window;
	};
}

bool UDatasmithFileProducer::Initialize()
{
	using namespace UE::DatasmithImporter;
	FText TaskDescription = FText::Format( LOCTEXT( "DatasmithFileProducer_LoadingFile", "Loading {0} ..."), FText::FromString( FilePath ) );
	ProgressTaskPtr = MakeUnique< FDataprepWorkReporter >( Context.ProgressReporterPtr, TaskDescription, 10.0f, 1.0f );

	ProgressTaskPtr->ReportNextStep( TaskDescription, 7.0f );

	if( FilePath.IsEmpty() )
	{
		LogError( LOCTEXT( "DatasmithFileProducer_Incomplete", "No file has been selected." ) );
		return false;
	}

	// Check file exists
	if(!FPaths::FileExists( FilePath ))
	{
		LogError( FText::Format( LOCTEXT( "DatasmithFileProducer_NotFound", "File {0} does not exist." ), FText::FromString( FilePath ) ) );
		return false;
	}

	// Create transient package if it wasn't specified
	if (!TransientPackage)
	{
		TransientPackage = NewObject< UPackage >(nullptr, *FPaths::Combine(Context.RootPackagePtr->GetPathName(), *GetName()), RF_Transient);
	}

	TransientPackage->FullyLoad();

	// Create the transient Datasmith scene
	DatasmithScene =  NewObject< UDatasmithScene >( TransientPackage, *GetName() );
	check( DatasmithScene->IsValidLowLevel() );

	if (!ExternalSourcePtr.IsValid() && !InitTranslator())
	{
		return false;
	}

	// Create and initialize context
	ImportContextPtr = MakeUnique< FDatasmithImportContext >( ExternalSourcePtr.ToSharedRef(), false, TEXT("DatasmithFileProducer"), LOCTEXT("DatasmithFileProducerDescription", "Datasmith File Producer") );

	// Set import options to default
	ImportContextPtr->Options->BaseOptions = DefaultImportOptions;
	ImportContextPtr->AdditionalImportOptions = ImportOptionsOverride.Num() > 0 ? ImportOptionsOverride : GetTranslatorImportOptions();

	ImportContextPtr->SceneAsset = DatasmithScene;
	ImportContextPtr->ActorsContext.ImportWorld = Context.WorldPtr.Get();

	FString SceneOuterPath = DatasmithScene->GetOutermost()->GetName();
	FString RootPath = FPackageName::GetLongPackagePath( SceneOuterPath );

	if ( Algo::Count( RootPath, TEXT('/') ) > 1 )
	{
		// Remove the scene folder as it shouldn't be considered in the import path
		RootPath.Split( TEXT("/"), &RootPath, nullptr, ESearchCase::IgnoreCase, ESearchDir::FromEnd );
	}

	FPaths::NormalizeDirectoryName( RootPath );

	constexpr EObjectFlags LocalObjectFlags = RF_Public | RF_Standalone | RF_Transactional;
	FFeedbackContext* InFeedbackContext = Context.ProgressReporterPtr ? Context.ProgressReporterPtr->GetFeedbackContext() : nullptr;
	const bool bIsSilent = true;
	if (!ImportContextPtr->Init( RootPath, LocalObjectFlags, InFeedbackContext, TSharedPtr< FJsonObject >(), bIsSilent ))
	{
		LogError( LOCTEXT( "DatasmithFileProducer_Initialization", "Initialization of producer failed." ) );
		return false;
	}

	// Fill up scene element with content of input file
	if (TSharedPtr< IDatasmithScene > SceneElement = ExternalSourcePtr->TryLoad())
	{
		ImportContextPtr->InitScene( SceneElement.ToSharedRef() );
	}
	else
	{
		LogError( LOCTEXT( "DatasmithFileProducer_Translation", "Translation to Datasmith scene failed." ) );
		return false;
	}

	return true;
}

bool UDatasmithFileProducer::InitTranslator()
{
	using namespace UE::DatasmithImporter;

	const FSourceUri SourceUri = FSourceUri::FromFilePath( FilePath );
	ExternalSourcePtr = IExternalSourceModule::GetOrCreateExternalSource( SourceUri );

	if ( !ExternalSourcePtr || !ExternalSourcePtr->GetAssetTranslator() )
	{
		LogError( LOCTEXT( "DatasmithFileProducer_CannotInit", "No suitable translator found for this source." ) );
		ExternalSourcePtr.Reset();
		return false;
	}

	return true;
}

TArray<TObjectPtr<UDatasmithOptionsBase>> UDatasmithFileProducer::GetTranslatorImportOptions()
{
	TArray<TObjectPtr<UDatasmithOptionsBase>> Result;

	if (!ExternalSourcePtr.IsValid() && !InitTranslator())
	{
		return Result;
	}

	if (!bTranslatorImportOptionsInitialized)
	{
		// Set all import options to defaults for Dataprep

		TSharedPtr<IDatasmithTranslator> TranslatorPtr = ExternalSourcePtr->GetAssetTranslator();

		if (IDatasmithTranslator* Translator = TranslatorPtr.Get())
		{
			TArray< TObjectPtr<UDatasmithOptionsBase> > Options;
			Translator->GetSceneImportOptions(Options);

			for (TObjectPtr<UDatasmithOptionsBase> Option : Options)
			{
				UDatasmithOptionsBase* MyOption = DuplicateObject<UDatasmithOptionsBase>(Option, this);
				if (UDatasmithCommonTessellationOptions* TesselationOption = Cast<UDatasmithCommonTessellationOptions>(MyOption))
				{
					TesselationOption->Options = DefaultTessellationOptions;
					TranslatorImportOptions.Add(MyOption);
					continue;
				}
				else if (UDatasmithImportOptions* DatasmithOptions = Cast<UDatasmithImportOptions>(Option))
				{
					continue;
				}

				// Only add option if it has visible properties
				for (TFieldIterator<FProperty> It(MyOption->GetClass()); It; ++It)
				{
					if (!FObjectEditorUtils::IsVariableCategoryHiddenFromClass(*It, MyOption->GetClass()))
					{
						TranslatorImportOptions.Add(MyOption);
						break;
					}
				}
			}
		}

		bTranslatorImportOptionsInitialized = true;
	}

	for (UDatasmithOptionsBase* Option : TranslatorImportOptions)
	{
		Result.Add(Option);
	}

	return MoveTemp(Result);
}

bool UDatasmithFileProducer::Execute(TArray< TWeakObjectPtr< UObject > >& OutAssets)
{
	if ( !IsValid() )
	{
		return false;
	}

	if ( IsCancelled() )
	{
		return false;
	}

	ProgressTaskPtr->ReportNextStep( FText::Format( LOCTEXT( "DatasmithFileProducer_ConvertingFile", "Converting {0} ..."), FText::FromString( FilePath ) ), 2.0f );
	SceneElementToWorld();

	if ( IsCancelled() )
	{
		return false;
	}

	ProgressTaskPtr->ReportNextStep( LOCTEXT( "DatasmithFileProducer_CleaningData", "Cleaning data ...") );
	PreventNameCollision();

	OutAssets.Append( MoveTemp( Assets ) );

	return !IsCancelled();
}

// Borrowed from DatasmithImportFactoryImpl::ImportDatasmithScene
void UDatasmithFileProducer::SceneElementToWorld()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UDatasmithFileProducer::SceneElementToWorld);

	// Filter element that need to be imported depending on dirty state (or eventually depending on options)
	FDatasmithImporter::FilterElementsToImport( *ImportContextPtr ); // #ueent_wip_reimport handle hashes

	// TEXTURES
	// We need the textures before the materials
	FDatasmithImporter::ImportTextures( *ImportContextPtr );

	// MATERIALS
	// We need to import the materials before the static meshes to know about the meshes build requirements that are driven by the materials
	FDatasmithImporter::ImportMaterials( *ImportContextPtr );

	// SCENE ASSET
	// Note the Datasmith scene has already been created, it will be reused
	//DatasmithImportFactoryImpl::CreateSceneAsset( *ImportContextPtr );

	// STATIC MESHES
	FDatasmithImporter::ImportStaticMeshes( *ImportContextPtr );

	FDatasmithStaticMeshImporter::PreBuildStaticMeshes( *ImportContextPtr );

	// ACTORS
	{
		FDatasmithImporter::ImportActors( *ImportContextPtr );
		
		// ImportSceneActor is created in FDatasmithImporter::ImportActors
		ImportSceneActor = ImportContextPtr->ActorsContext.ImportSceneActor;

		// Level sequences have to be imported after the actors to be able to bind the tracks to the actors to be animated
		FDatasmithImporter::ImportLevelSequences( *ImportContextPtr );

		// Level variant sets have to be imported after the actors and materials to be able to bind to them correctly
		FDatasmithImporter::ImportLevelVariantSets( *ImportContextPtr );
	}

	// Find the lights texture profile (This is for the IES textures)
	UPackage* LightPackage = ImportContextPtr->AssetsContext.LightPackage.Get();
	TArray< FAssetData > AssetsData;
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	AssetRegistryModule.Get().GetAssetsByPath( *LightPackage->GetPathName( ), AssetsData, true );

	Assets.Reserve( ImportContextPtr->ImportedStaticMeshes.Num() +
		ImportContextPtr->ImportedTextures.Num() +
		ImportContextPtr->ImportedMaterials.Num() +
		ImportContextPtr->ImportedParentMaterials.Num() +
		ImportContextPtr->ImportedLevelSequences.Num() +
		ImportContextPtr->ImportedLevelVariantSets.Num() +
		AssetsData.Num()
	);

	for ( FAssetData& AssetData : AssetsData )
	{
		if ( UObject* Object = AssetData.GetAsset() )
		{
			Assets.Emplace( Object );
		}
	}

	// Note: Some of the assets might be null (incomplete or failed import), only add non-null ones to Assets

	for(const auto& KV : ImportContextPtr->ImportedStaticMeshes)
	{
		if (KV.Value)
		{
			Assets.Emplace(KV.Value);
		}
	}

	for ( TPair< TSharedRef< IDatasmithTextureElement >, UE::Interchange::FAssetImportResultRef >& AssetPair : ImportContextPtr->ImportedTextures )
	{
		AssetPair.Value->WaitUntilDone();

		if ( UTexture* ImportedTexture = Cast< UTexture >( AssetPair.Value->GetFirstAssetOfClass( UTexture::StaticClass() ) ) )
		{
			Assets.Emplace( ImportedTexture );
		}
	}

	for ( auto& AssetPair : ImportContextPtr->ImportedMaterials )
	{
		if(AssetPair.Value)
		{
			Assets.Emplace( AssetPair.Value );

			if (UMaterial* SourceMaterial = Cast< UMaterial >(AssetPair.Value))
			{
				SourceMaterial->UpdateCachedExpressionData();

				for (const FMaterialFunctionInfo& MaterialFunctionInfo : SourceMaterial->GetCachedExpressionData().FunctionInfos)
				{
					if (MaterialFunctionInfo.Function && MaterialFunctionInfo.Function->GetOutermost() == SourceMaterial->GetOutermost())
					{
						Assets.Emplace( MaterialFunctionInfo.Function );
					}
				}
			}
		}
	}

	for ( auto& AssetPair : ImportContextPtr->ImportedParentMaterials )
	{
		if(AssetPair.Value)
		{
			Assets.Emplace( AssetPair.Value );
		}
	}

	for ( TPair< TSharedRef< IDatasmithBaseMaterialElement >, UMaterialFunction* >& AssetPair : ImportContextPtr->ImportedMaterialFunctions )
	{
		if(AssetPair.Value)
		{
			Assets.Emplace( AssetPair.Value );
		}
	}

	for ( auto& AssetPair : ImportContextPtr->ImportedLevelSequences )
	{
		if(AssetPair.Value)
		{
			Assets.Emplace( AssetPair.Value );
		}
	}

	for ( auto& AssetPair : ImportContextPtr->ImportedLevelVariantSets )
	{
		if(AssetPair.Value)
		{
			Assets.Emplace( AssetPair.Value );
		}
	}
}

void UDatasmithFileProducer::PreventNameCollision()
{
	// Create packages where assets must be moved to avoid name collision
	FString TransientFolderPath = DatasmithScene->GetOutermost()->GetPathName();

	// Clean up transient package path. It should be empty
	FDatasmithFileProducerUtils::DeletePackagePath( TransientFolderPath );

	// Create packages to move assets to
	UPackage* StaticMeshesImportPackage = NewObject< UPackage >( nullptr, *FPaths::Combine( TransientFolderPath, TEXT("Geometries") ), RF_Transient );
	StaticMeshesImportPackage->FullyLoad();

	UPackage* TexturesImportPackage = NewObject< UPackage >( nullptr, *FPaths::Combine( TransientFolderPath, TEXT("Textures") ), RF_Transient );
	TexturesImportPackage->FullyLoad();

	UPackage* MaterialsImportPackage = NewObject< UPackage >( nullptr, *FPaths::Combine( TransientFolderPath, TEXT("Materials") ), RF_Transient );
	MaterialsImportPackage->FullyLoad();

	UPackage* ReferenceMaterialsImportPackage = NewObject< UPackage >(nullptr, *FPaths::Combine(TransientFolderPath, TEXT("Materials/Reference")), RF_Transient);
	ReferenceMaterialsImportPackage->FullyLoad();

	UPackage* LevelSequencesImportPackage = NewObject< UPackage >( nullptr, *FPaths::Combine( TransientFolderPath, TEXT("Animations") ), RF_Transient );
	LevelSequencesImportPackage->FullyLoad();

	UPackage* LevelVariantSetsImportPackage = NewObject< UPackage >( nullptr, *FPaths::Combine( TransientFolderPath, TEXT("Variants") ), RF_Transient );
	LevelVariantSetsImportPackage->FullyLoad();

	UPackage* LightsImportPackage = NewObject< UPackage >(nullptr, *FPaths::Combine(TransientFolderPath, TEXT("Lights")), RF_Transient);
	LightsImportPackage->FullyLoad();

	UPackage* OtherImportPackage = NewObject< UPackage >( nullptr, *FPaths::Combine( TransientFolderPath, TEXT("Others") ), RF_Transient );
	OtherImportPackage->FullyLoad();

	// Set of transient packages which are not used anymore
	TSet<FString> PathsToDelete;

	// Set of packages containing level sequences which actor's reference will need to be updated after actors are renamed
	TSet<UPackage*> LevelSequencePackagesToCheck;

	// Set of packages containing level variant sets which actor's reference will need to be updated after actors are renamed
	TSet<UPackage*> LevelVariantSetsPackagesToCheck;

	// Move assets in 2 passes: 1st pass skip UMaterial objects which are not referenced by a UMaterialInstance one, 2nd pass move unreferenced UMaterial objects
	// This is done to mimic how the direct import (from the editor's toolbar) behaves
	{
		// Array of packages containing templates which are referring to assets as TSoftObjectPtr or FSoftObjectPath
		TArray<UPackage*> PackagesToCheck;

		// Map containing mapping between previous package to new one
		TMap<FSoftObjectPath, FSoftObjectPath> AssetRedirectorMap;

		UDatasmithContentBlueprintLibrary* DatasmithContentLibrary = Cast< UDatasmithContentBlueprintLibrary >( UDatasmithContentBlueprintLibrary::StaticClass()->GetDefaultObject() );

		auto MoveAsset = [&](UObject* Object, UPackage* NewPackage, bool bCheckPackage)
		{
			if(Object->GetOutermost()->GetName() != NewPackage->GetName())
			{
				FSoftObjectPath PreviousObjectPath(Object);

				Object->Rename( nullptr, NewPackage, REN_DontCreateRedirectors | REN_NonTransactional );

				// Replace unique name id with hash of package path to avoid asset's name collision
				const FString DatasmithUniqueId = DatasmithContentLibrary->GetDatasmithUserDataValueForKey( Object, UDatasmithAssetUserData::UniqueIdMetaDataKey );
				const FString ObjectIdent = DatasmithUniqueId + DatasmithScene->GetName() + Object->GetName() + Object->GetClass()->GetName();
				FString NewUniqueId = FMD5::HashBytes( reinterpret_cast<const uint8*>(*ObjectIdent), ObjectIdent.Len() * sizeof(TCHAR) );

				UDatasmithAssetUserData::SetDatasmithUserDataValueForKey(Object, *(FString(TEXT("Old")) + UDatasmithAssetUserData::UniqueIdMetaDataKey), DatasmithUniqueId );
				UDatasmithAssetUserData::SetDatasmithUserDataValueForKey(Object, UDatasmithAssetUserData::UniqueIdMetaDataKey, NewUniqueId );

				AssetRedirectorMap.Emplace( PreviousObjectPath, Object );
				if(bCheckPackage)
				{
					PackagesToCheck.Add( Object->GetOutermost() );
				}
			}
		};

		// First pass: No UMaterial objects, parent materials are collected if applicable
		TSet<UMaterialInterface*> ParentMaterials;
		TSet<UMaterialFunctionInterface*> MaterialFunctions;

		for(int32 Index = 0; Index < Assets.Num(); ++Index)
		{
			if( UObject* Object = Assets[Index].Get() )
			{
				PathsToDelete.Add( Object->GetOutermost()->GetPathName() );

				if( Cast<UStaticMesh>( Object ) != nullptr )
				{
					MoveAsset( Object, StaticMeshesImportPackage, false );
				}
				else if ( Cast<UTextureLightProfile>(Object) )
				{
					MoveAsset( Object, LightsImportPackage, false );
				}
				else if( Cast<UTexture>(Object) != nullptr )
				{
					MoveAsset( Object, TexturesImportPackage, false );
				}
				else if( Cast<UMaterialFunctionInterface>(Object) != nullptr )
				{
					MoveAsset( Object, MaterialsImportPackage, true );
				}
				else if( UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(Object) )
				{
					if ( UMaterialInterface* MaterialParent = MaterialInstance->Parent )
					{
						FString MaterialInstancePath = MaterialInstance->GetOutermost()->GetName();
						FString ParentPath = MaterialParent->GetOutermost()->GetName();

						if ( ParentPath.StartsWith( MaterialInstancePath ) )
						{
							MoveAsset( MaterialParent, ReferenceMaterialsImportPackage, true );
							ParentMaterials.Add( MaterialParent );
						}
					}

					MoveAsset( Object, MaterialsImportPackage, true );
				}
				else if( Cast<ULevelSequence>(Object) != nullptr )
				{
					MoveAsset( Object, LevelSequencesImportPackage, false );
					LevelSequencePackagesToCheck.Add( Object->GetOutermost() );
				}
				else if( Cast<ULevelVariantSets>(Object) != nullptr )
				{
					MoveAsset( Object, LevelVariantSetsImportPackage, false );
					LevelVariantSetsPackagesToCheck.Add( Object->GetOutermost() );
				}
				// Move unsupported asset types to Others package, except UMaterial objects which are dealt with in two passes
				else if( Cast<UMaterial>(Object) == nullptr )
				{
					MoveAsset( Object, OtherImportPackage, false );
				}
			}
		}

		// 2nd pass: Move UMaterial objects which are not referenced
		for(int32 Index = 0; Index < Assets.Num(); ++Index)
		{
			if( UMaterial* Material = Cast<UMaterial>( Assets[Index].Get() ) )
			{
				if( !ParentMaterials.Contains( Material ) )
				{
					PathsToDelete.Add( FPaths::GetPath( Material->GetOutermost()->GetName() ) );
					MoveAsset( Material, MaterialsImportPackage, true );
				}
			}
			else if( UMaterialFunctionInterface* MaterialFunction = Cast<UMaterialFunctionInterface>( Assets[Index].Get() ) )
			{
				if( !MaterialFunctions.Contains( MaterialFunction ) )
				{
					PathsToDelete.Add( FPaths::GetPath( MaterialFunction->GetOutermost()->GetName() ) );
					MoveAsset( MaterialFunction, MaterialsImportPackage, true );
				}
			}
		}

		// Apply soft object path redirection to identified packages
		if (PackagesToCheck.Num() > 0 && AssetRedirectorMap.Num() > 0)
		{
			IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
			AssetTools.RenameReferencingSoftObjectPaths(PackagesToCheck, AssetRedirectorMap);
		}
	}

	// Prefix UniqueIdMetaDataKey of newly created actors with producer namespace to avoid name collision
	const FString Namespace = GetNamespace();

	TMap< FSoftObjectPath, FSoftObjectPath > ActorRedirectorMap;

	for (ULevel* Level : Context.WorldPtr->GetLevels())
	{
		for (int32 i = 0; i < Level->Actors.Num(); i++)
		{
			if( ADatasmithSceneActor* SceneActor = Cast<ADatasmithSceneActor>( Level->Actors[i] ) )
			{
				if(SceneActor->Scene == DatasmithScene)
				{
					// Append prefix to all children of scene actor
					for( TPair< FName, TSoftObjectPtr< AActor > >& ActorPair : SceneActor->RelatedActors)
					{
						if ( AActor* Actor = ActorPair.Value.Get() )
						{
							if( UDatasmithAssetUserData* AssetUserData = UDatasmithContentBlueprintLibrary::GetDatasmithUserData( Actor ) )
							{
								if( FString* ValuePtr = AssetUserData->MetaData.Find( UDatasmithAssetUserData::UniqueIdMetaDataKey ) )
								{
									FSoftObjectPath PreviousActorSoftPath(Actor);

									FString& Value = *ValuePtr;

									// Set Actor's name to the one from its old unique Id.
									// Rationale: The unique Id is used to reconstruct the IDatasmithActorElement in the Datasmith consumer.
									// Important Note: No need to prefix the actor's name with the namespace, it will be done by the parent class, UDataprepContentProducer
									// Important Note: Value of unique Id might collide with name of scene actor. See JIRA UE-80831
									if( !Actor->Rename( *Value, nullptr, REN_Test ) )
									{
										Value = MakeUniqueObjectName( Actor->GetOuter(), Actor->GetClass(), *Value ).ToString();
									}

									Actor->Rename( *Value );

									ActorRedirectorMap.Emplace( PreviousActorSoftPath, Actor );

									// Prefix actor's unique Id with the namespace
									Value = Namespace + TEXT("_") + Value;
								}
							}
						}
					}

					// Remove reference to Datasmith scene
					SceneActor->Scene = nullptr;
				}
			}
		}
	}

	// Update reference of LevelSequence assets if necessary
	if(LevelSequencePackagesToCheck.Num() > 0)
	{
		IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
		AssetTools.RenameReferencingSoftObjectPaths( LevelSequencePackagesToCheck.Array(), ActorRedirectorMap );
	}

	// Update reference of LevelVariantSets assets if necessary
	if(LevelVariantSetsPackagesToCheck.Num() > 0)
	{
		IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
		AssetTools.RenameReferencingSoftObjectPaths( LevelVariantSetsPackagesToCheck.Array(), ActorRedirectorMap );
	}

	FDatasmithFileProducerUtils::DeletePackagesPath( PathsToDelete );
}

void UDatasmithFileProducer::OnFilePathChanged()
{
	FilePath = FPaths::ConvertRelativePathToFull( FilePath );
	ExternalSourcePtr.Reset();
	TranslatorImportOptions.Empty();
	bTranslatorImportOptionsInitialized = false;
	UpdateName();
	OnChanged.Broadcast( this );
}

void UDatasmithFileProducer::Reset()
{
	DatasmithScene = nullptr;
	ImportContextPtr.Reset();
	ExternalSourcePtr.Reset();
	ProgressTaskPtr.Reset();
	Assets.Empty();
	TransientPackage = nullptr;

	UDataprepContentProducer::Reset();
}

const FText& UDatasmithFileProducer::GetLabel() const
{
	return DatasmithFileProducerLabel;
}

const FText& UDatasmithFileProducer::GetDescription() const
{
	return DatasmithFileProducerDescription;
}

FString UDatasmithFileProducer::GetNamespace() const
{
	return FString::FromInt( GetTypeHash( FilePath ) );
}

void UDatasmithFileProducer::SetFilePath( const FString& InFilePath )
{
	Modify();

	FilePath = InFilePath;

	OnFilePathChanged();
}

void UDatasmithFileProducer::OnChangeImportSettings()
{
	if (!ExternalSourcePtr.IsValid() && !InitTranslator())
	{
		return;
	}

	TSharedPtr<SWindow> ParentWindow;

	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
		ParentWindow = MainFrame.GetParentWindow();
	}

	TSharedPtr<IDatasmithTranslator> TranslatorPtr = ExternalSourcePtr->GetAssetTranslator();

	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(FText::Format(LOCTEXT("ImportSettingsTitle", "{0} Import Settings"), FText::FromName(TranslatorPtr->GetFName())))
		.SizingRule(ESizingRule::Autosized);

	TArray<TObjectPtr<UDatasmithOptionsBase>> Options = GetTranslatorImportOptions();

	if (Options.Num() == 0)
	{
		UE_LOGF(LogDatasmithImport, Warning, "No import settings for selected file type.");
		return;
	}

	TArray<UObject*> OptionsRaw;
	for(TObjectPtr<UDatasmithOptionsBase>& ObjectPtr : Options)
	{
		OptionsRaw.Add(ObjectPtr.Get());
	}

	TSharedPtr<FDatasmithFileProducerUtils::SFileProducerImportOptionsWindow> OptionsWindow;
	Window->SetContent
	(
		SAssignNew(OptionsWindow, FDatasmithFileProducerUtils::SFileProducerImportOptionsWindow)
		.ImportOptions(OptionsRaw)
		.WidgetWindow(Window)
		.MinDetailHeight(320.f)
		.MinDetailWidth(450.f)
	);

	FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);
}

void UDatasmithFileProducer::UpdateName()
{
	if(!FilePath.IsEmpty())
	{
		// Rename producer to name of file
		FString CleanName = ObjectTools::SanitizeObjectName( FPaths::GetCleanFilename( FilePath ) );
		if ( !Rename( *CleanName, nullptr, REN_Test ) )
		{
			CleanName = MakeUniqueObjectName( GetOuter(), GetClass(), *CleanName ).ToString();
		}

		Rename( *CleanName, nullptr, REN_DontCreateRedirectors | REN_NonTransactional );
	}
}

bool UDatasmithFileProducer::Supersede(const UDataprepContentProducer* OtherProducer) const
{
	const UDatasmithFileProducer* OtherFileProducer = Cast<const UDatasmithFileProducer>(OtherProducer);

	return OtherFileProducer != nullptr &&
		!OtherFileProducer->FilePath.IsEmpty() &&
		FilePath == OtherFileProducer->FilePath;
}

bool UDatasmithFileProducer::CanAddToProducersArray(bool bIsAutomated)
{
	if ( bIsAutomated )
	{
		return true;
	}

	FilePath = FDatasmithFileProducerUtils::SelectFileToImport();
	if ( !FilePath.IsEmpty() )
	{
		UpdateName();
		return true;
	}

	return false;
}

void UDatasmithFileProducer::PostEditUndo()
{
	UDataprepContentProducer::PostEditUndo();

	OnChanged.Broadcast( this );
}

void UDatasmithFileProducer::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FProperty* EditedProperty = PropertyChangedEvent.Property;

	if ( !EditedProperty || EditedProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UDatasmithFileProducer, FilePath) )
	{
		OnFilePathChanged();
	}
}

UDatasmithDirProducer::UDatasmithDirProducer()
	: ExtensionString( TEXT("*.*") )
	, bRecursive(true)
	, bHasWildCardSearch(true)
{
	TessellationOptions = CreateDefaultSubobject<UDatasmithCommonTessellationOptions>(FName(TEXT("TessellationOptions")));

	if( !HasAnyFlags( RF_ClassDefaultObject ) && SupportedFormats.Num() == 0 )
	{
		const TArray<FString>& Formats = FDatasmithTranslatorManager::Get().GetSupportedFormats();
		// Verify that at least one required extension is supported
		for( const FString& Format : Formats )
		{
			TArray<FString> FormatComponents;
			Format.ParseIntoArray( FormatComponents, TEXT( ";" ), false );

			for ( int32 ComponentIndex = 0; ComponentIndex < FormatComponents.Num(); ComponentIndex += 2 )
			{
				check( FormatComponents.IsValidIndex( ComponentIndex + 1 ) );
				const FString Extension = FormatComponents[ComponentIndex].ToLower();

				SupportedFormats.Add( Extension );
			}
		}
	}
}

bool UDatasmithDirProducer::Initialize()
{
	if( FolderPath.IsEmpty() )
	{
		LogError( LOCTEXT( "DatasmithDirProducerNoFolder", "Initialization failed: No folder has been specified" ) );
		return false;
	}

	// Abort initialization as there is no extension to look for
	if( !bHasWildCardSearch && FixedExtensionSet.Num() == 0 )
	{
		LogError( LOCTEXT( "DatasmithDirProducerNoExtension", "Initialization failed: No extension has been specified" ) );
		return false;
	}

	FilesToProcess = GetSetOfFiles();

	// Abort initialization as there is no file to process
	if( FilesToProcess.Num() == 0 )
	{
		LogError( LOCTEXT( "DatasmithDirProducerNoFile", "Initialization failed: No file to process: either no file matched the extension set or none of the file's extensions were supported" ) );
		return false;
	}

	FileProducer = NewObject< UDatasmithFileProducer >( GetTransientPackage(), NAME_None, RF_Transient );

	return true;
}

bool UDatasmithDirProducer::Execute(TArray< TWeakObjectPtr< UObject > >& OutAssets)
{
	if(!IsValid())
	{
		LogError( LOCTEXT( "DatasmithProducerInvalid", "Execution failed: Producer is not valid." ) );
		return false;
	}

	FDataprepWorkReporter Task( Context.ProgressReporterPtr, LOCTEXT( "DatasmithFileProducer_LoadingFromDirectory", "Loading files from directory ..." ), (float)FilesToProcess.Num(), 1.0f );

	// Cache context's package
	UPackage* CachedPackage = Context.RootPackagePtr.Get();

	FString RootPath = FPaths::Combine( Context.RootPackagePtr->GetPathName(), *GetName() );
	UPackage* RootTransientPackage = NewObject< UPackage >( nullptr, *RootPath, RF_Transient );
	RootTransientPackage->FullyLoad();

	TArray<FString> FilesNotProcessed; // these files will be processed one by one
	if (!ImportAsPlmXml(RootTransientPackage, OutAssets, FilesNotProcessed))
	{
		FilesNotProcessed = FilesToProcess.Array();
	}
	
	for( const FString& FileName : FilesNotProcessed )
	{
		if ( IsCancelled() )
		{
			break;
		}

		// Import content of file into the proper content folder to avoid name collision
		UPackage* TransientPackage = RootTransientPackage;
		FString FilePath = FPaths::GetPath( FileName );
		if(FilePath != FolderPath)
		{
			FString SubFolder = FilePath.Right(FilePath.Len() - FolderPath.Len() - 1 /* Remove leading '/' */);
			TransientPackage = NewObject< UPackage >( nullptr, *FPaths::Combine( RootPath, SubFolder ), RF_Transient );
			TransientPackage->FullyLoad();
		}

		Context.SetRootPackage( TransientPackage );

		// Update file producer's filename
		FileProducer->FilePath =  FPaths::ConvertRelativePathToFull( FileName );
		FileProducer->UpdateName();

		Task.ReportNextStep( FText::Format( LOCTEXT( "DatasmithFileProducer_LoadingFile", "Loading {0} ..."), FText::FromString( FileName ) ) );

		SetFileProducerSettings();

		if( !FileProducer->Produce( Context, OutAssets ) )
		{
			FText ErrorReport = FText::Format( LOCTEXT( "DatasmithDirProducer_FailedLoad", "Failed to load {0} ..."), FText::FromString( FileName ) );
			LogError( ErrorReport );
		}
	}

	// Restore context's package
	Context.SetRootPackage( CachedPackage );

	return !IsCancelled();
}

void UDatasmithDirProducer::OnChangeImportSettings()
{
	TSharedPtr<SWindow> ParentWindow;

	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
		ParentWindow = MainFrame.GetParentWindow();
	}

	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(FText::Format(LOCTEXT("ImportSettingsTitle", "{0} Import Settings"), FText::FromString(TEXT("Datasmith Tessellation"))))
		.SizingRule(ESizingRule::Autosized);

	TArray<UObject*> OptionsRaw = { TessellationOptions };

	TSharedPtr<FDatasmithFileProducerUtils::SFileProducerImportOptionsWindow> OptionsWindow;
	Window->SetContent
	(
		SAssignNew(OptionsWindow, FDatasmithFileProducerUtils::SFileProducerImportOptionsWindow)
		.ImportOptions(OptionsRaw)
		.WidgetWindow(Window)
		.MinDetailHeight(320.f)
		.MinDetailWidth(450.f)
	);

	FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);
}

void UDatasmithDirProducer::SetFileProducerSettings()
{
	// Set translator options
	TArray< TObjectPtr<UDatasmithOptionsBase> > Options = FileProducer->GetTranslatorImportOptions();

	bool bHasTesselationOptions = false;
	for (TObjectPtr<UDatasmithOptionsBase> Option : Options)
	{
		if (UDatasmithCommonTessellationOptions* TesselationOption = Cast<UDatasmithCommonTessellationOptions>(Option.Get()))
		{
			TesselationOption->Options = TessellationOptions->Options;
			bHasTesselationOptions = true;
		}
	}

	if (bHasTesselationOptions)
	{
		FileProducer->SetTranslatorImportOptions(Options);
	}
}

// #ueent_todo:  ImportAsPlmXml should be replaced by a new object which will be responsible for the dispatch of import files which are supported by Datasmith
/** Try to import as many files as possible using PlmXml translator. This is a temporary solution which enables importing multiple files at once using as many CPU cores as possible(fastest).
 *  DatasmithDispatcher is able to import multiple files in parallel and PlmXmlTranslator is making use of this property of the DatasmithDispatcher.
 *  Returns false if no files were processed
 */
bool UDatasmithDirProducer::ImportAsPlmXml(UPackage* RootPackage, TArray<TWeakObjectPtr<UObject>>& OutAssets, TArray<FString>& FilesNotProcessed)
{
	FString PlmXmlFileName;
	FString TempDir = FPaths::Combine(FPaths::ProjectIntermediateDir(), TEXT("DatasmithDirProducerTemp"));
	if (!IFileManager::Get().DirectoryExists(*TempDir))
	{
		IFileManager::Get().MakeDirectory(*TempDir);
	}
	PlmXmlFileName = FPaths::Combine(TempDir, FPaths::GetBaseFilename(FolderPath) + TEXT(".plmxml"));

	if (!FDatasmithImporterUtils::CreatePlmXmlSceneFromCADFiles(PlmXmlFileName, FilesToProcess, FilesNotProcessed))
	{
		return false;
	}

	// Import content of file into the proper content folder to avoid name collision
	UPackage* TransientPackage = RootPackage;

	Context.SetRootPackage(TransientPackage);

	// Update file producer's filename
	FileProducer->FilePath = PlmXmlFileName;
	FileProducer->UpdateName();

	// Set transient package to be the root package, overriding default behavior - this way we avoid extra 'PlmXml' folder  in content folder hierarchy
	FileProducer->TransientPackage = RootPackage;

	SetFileProducerSettings();

	bool bSuccess = FileProducer->Produce(Context, OutAssets);
	if (bSuccess)
	{
		FixPlmXmlHierarchy();
	}
	else
	{
		FText ErrorReport = FText::Format(LOCTEXT("DatasmithPlmXmlProducer_FailedProduce", "Failed to produce assets with PlmXml file, {0}, for parallel loading ..."), FText::FromString(PlmXmlFileName));
		LogError(ErrorReport);
	}

	return bSuccess;
}

void UDatasmithDirProducer::FixPlmXmlHierarchy()
{
	TFunction<void(AActor*, ADatasmithSceneActor*, const TMap<AActor*, FName>&)> AddActorHierachyToRelatedActors = [&AddActorHierachyToRelatedActors](AActor* Actor, ADatasmithSceneActor* SceneActor, const TMap<AActor*, FName>& NameForRelatedActor)
	{
		// Add all attached actors to related actors of a SceneActor
		const FName* NamePtr = NameForRelatedActor.Find(Actor);
		if (ensure(NamePtr))
		{
			SceneActor->RelatedActors.FindOrAdd(*NamePtr) = Actor;
		}

		TArray<class AActor*> AttachedActors;
		Actor->GetAttachedActors(AttachedActors);
		for (AActor* Child : AttachedActors)
		{
			AddActorHierachyToRelatedActors(Child, SceneActor, NameForRelatedActor);
		}
	};

	ADatasmithSceneActor* ImportSceneActor = FileProducer->ImportSceneActor.Get();
	if (ImportSceneActor)
	{
		// Convert each child actor of ImportSceneActor into ADatasmithSceneActor
		TArray<class AActor*> ImportSceneActorChildren;
		ImportSceneActor->GetAttachedActors(ImportSceneActorChildren);
		for (AActor* ImportSceneActorChild : ImportSceneActorChildren)
		{
			FString Label = ImportSceneActorChild->GetActorLabel();

			ADatasmithSceneActor* SceneActor = Cast<ADatasmithSceneActor>(Context.WorldPtr->SpawnActor(ADatasmithSceneActor::StaticClass(), nullptr, nullptr));
			if (!ensure(SceneActor))
			{
				continue;
			}
			SceneActor->SetActorLabel(Label);
			SceneActor->SpriteScale = 0.1f;

			USceneComponent* RootComponent = NewObject<USceneComponent>(SceneActor, *Label, RF_Transactional);
			RootComponent->SetWorldTransform(FTransform::Identity);
			RootComponent->Mobility = EComponentMobility::Static;
			RootComponent->bVisualizeComponent = true;
			RootComponent->RegisterComponent();
			SceneActor->SetRootComponent(RootComponent);
			SceneActor->AddInstanceComponent(RootComponent);

			// Create map of related actors to their names 
			TMap<AActor*, FName> NameForRelatedActor;
			for (TPair<FName, TSoftObjectPtr<AActor>>& ActorPair : ImportSceneActor->RelatedActors)
			{
				NameForRelatedActor.Add(ActorPair.Value.LoadSynchronous(), ActorPair.Key);
			}

			// Move child actors and related actors to new SceneActor
			TArray<class AActor*> AttachedActors;
			ImportSceneActorChild->GetAttachedActors(AttachedActors);
			for (AActor* AttachedActor : AttachedActors)
			{
				AttachedActor->DetachFromActor(FDetachmentTransformRules::KeepWorldTransform);
				AttachedActor->AttachToActor(SceneActor, FAttachmentTransformRules::KeepWorldTransform);
				AddActorHierachyToRelatedActors(AttachedActor, SceneActor, NameForRelatedActor);
			}
			Context.WorldPtr.Get()->EditorDestroyActor(ImportSceneActorChild, true);
		}
		Context.WorldPtr.Get()->EditorDestroyActor(ImportSceneActor, true);
		FileProducer->ImportSceneActor.Reset();
	}

}

void UDatasmithDirProducer::Reset()
{
	FilesToProcess.Empty();
	FileProducer = nullptr;

	UDataprepContentProducer::Reset();
}

const FText& UDatasmithDirProducer::GetLabel() const
{
	return DatasmithDirProducerLabel;
}

const FText& UDatasmithDirProducer::GetDescription() const
{
	return DatasmithDirProducerDescription;
}

FString UDatasmithDirProducer::GetNamespace() const
{
	return FString::FromInt( GetTypeHash( FolderPath ) );
}

void UDatasmithDirProducer::Serialize( FArchive& Ar )
{
	Super::Serialize( Ar );

	if( Ar.IsLoading() )
	{
		UpdateExtensions();
	}
}

void UDatasmithDirProducer::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FProperty* EditedProperty = PropertyChangedEvent.Property;

	if ( !EditedProperty || EditedProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UDatasmithDirProducer, FolderPath) )
	{
		//OnEditFolderPathChanged();
	}

	if ( !EditedProperty || EditedProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UDatasmithDirProducer, ExtensionString) )
	{
		OnExtensionsChanged();
	}

	if ( !EditedProperty || EditedProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UDatasmithDirProducer, bRecursive) )
	{
		OnRecursivityChanged();
	}
}

void UDatasmithDirProducer::SetFolderPath( const FString& InFolderPath )
{
	Modify();
	FolderPath = InFolderPath;
	OnFolderPathChanged();
}

FString UDatasmithDirProducer::GetFolderPath() const
{
	return FolderPath;
}

void UDatasmithDirProducer::SetExtensionString(const FString& InExtensionString)
{
	Modify();
	ExtensionString = InExtensionString;
	OnExtensionsChanged();
}

FString UDatasmithDirProducer::GetExtensionString() const
{
	return ExtensionString;
}

void UDatasmithDirProducer::SetIsRecursive(bool bInRecursive)
{
	Modify();
	bRecursive = bInRecursive;
	OnRecursivityChanged();
}

bool UDatasmithDirProducer::IsRecursive() const
{
	return bRecursive;
}

void UDatasmithDirProducer::UpdateName()
{
	if(!FolderPath.IsEmpty())
	{
		FString BaseName = FPaths::IsDrive( FolderPath ) ? (FolderPath.Left(1) + TEXT("_Drive")) : (FPaths::GetBaseFilename( FolderPath ) + TEXT("_Dir"));

		// Rename producer to name of file
		FString CleanName = ObjectTools::SanitizeObjectName( BaseName );
		if ( !Rename( *CleanName, nullptr, REN_Test ) )
		{
			CleanName = MakeUniqueObjectName( GetOuter(), GetClass(), *CleanName ).ToString();
		}

		Rename( *CleanName, nullptr, REN_DontCreateRedirectors | REN_NonTransactional );
	}
}

bool UDatasmithDirProducer::Supersede(const UDataprepContentProducer* OtherProducer) const
{
	// If ExtensionString is empty, this producer does not generate anything
	if( FolderPath.IsEmpty() || ExtensionString.IsEmpty() )
	{
		return false;
	}

	if( const UDatasmithDirProducer* OtherDirProducer = Cast<const UDatasmithDirProducer>( OtherProducer ) )
	{
		if( OtherDirProducer->FolderPath.IsEmpty() || OtherDirProducer->ExtensionString.IsEmpty() )
		{
			return false;
		}

		// Potential superseding if other producer has same path and same recursiveness
		// or other producer's path is a sub folder and this producer is recursive
		bool bCouldSupersede = ( OtherDirProducer->FolderPath == FolderPath && OtherDirProducer->bRecursive == bRecursive ) ||
							   ( OtherDirProducer->FolderPath.StartsWith( FolderPath ) && bRecursive );

		// Check if this producer will generate a super-set of the set of files generated by the other one
		if(bCouldSupersede)
		{
			TSet< FString > ThisFilesToProcess = GetSetOfFiles();
			TSet< FString > OtherFilesToProcess = OtherDirProducer->GetSetOfFiles();

			if( OtherFilesToProcess.Num() > ThisFilesToProcess.Num() )
			{
				return false;
			}

			for(const FString& OtherFileToProcess : OtherFilesToProcess)
			{
				if( !ThisFilesToProcess.Contains( OtherFileToProcess ) )
				{
					return false;
				}
			}
		}

		return bCouldSupersede;
	}
	else if(const UDatasmithFileProducer* OtherFileProducer = Cast<const UDatasmithFileProducer>(OtherProducer))
	{
		const FString& FilePath = OtherFileProducer->GetFilePath();
		if( FilePath.StartsWith( FolderPath ) )
		{
			if( bHasWildCardSearch && ( FPaths::GetPath( FilePath ) == FolderPath || bRecursive == true ) )
			{
				return true;
			}

			FString Extension = FPaths::GetExtension( FilePath ).ToLower();
			return FixedExtensionSet.Contains( Extension );
		}
	}

	return false;
}

bool UDatasmithDirProducer::CanAddToProducersArray(bool bIsAutomated)
{
	if ( bIsAutomated )
	{
		return true;
	}


	FolderPath = FDatasmithFileProducerUtils::SelectDirectory();
	if ( !FolderPath.IsEmpty() )
	{
		UpdateName();
		return true;
	}

	return false;
}

void UDatasmithDirProducer::OnFolderPathChanged()
{
	FolderPath = FPaths::ConvertRelativePathToFull( FolderPath );
	UpdateName();
	OnChanged.Broadcast(this);
}

void UDatasmithDirProducer::OnRecursivityChanged()
{
	OnChanged.Broadcast( this );
}

void UDatasmithDirProducer::OnExtensionsChanged()
{
	UpdateExtensions();
	OnChanged.Broadcast( this );
}

void UDatasmithDirProducer::UpdateExtensions()
{
	bHasWildCardSearch = ExtensionString.Contains( WildCharCriteria );

	FixedExtensionSet.Reset();

	if( !bHasWildCardSearch)
	{
		TArray<FString> StringArray;
		ExtensionString.ParseIntoArray( StringArray, TEXT(";"), true );

		// #ueent_todo: Handle extension with a wildcard, i.e. prt* from Creo
		TSet<FString> StringSet;
		for (const FString& String : StringArray)
		{
			if( String.StartsWith( ExtensionPrefix ) )
			{
				// Only store the extension without its prefix and if it is supported
				FString Extension( *String.ToLower() + 2 );

				// If this is an extension with wild card, look for matching supported format
				if( Extension.Contains( TEXT("*") ) )
				{
					for( const FString& Format : SupportedFormats )
					{
						if( Format.MatchesWildcard( Extension ) )
						{
							FixedExtensionSet.Add( Format );
						}
					}
				}
				else if(SupportedFormats.Contains( Extension ))
				{
					FixedExtensionSet.Add( Extension );
				}
			}
		}
	}
}

TSet<FString> UDatasmithDirProducer::GetSetOfFiles() const
{
	TSet< FString > FoundFiles;

	const TSet< FString >& ExtensionSearchSet = bHasWildCardSearch ? SupportedFormats : FixedExtensionSet;

	// Build the list of files to process
	auto VisitDirectory = [this, &FoundFiles, &ExtensionSearchSet](const TCHAR* InFilenameOrDirectory, const bool bIsDirectory) -> bool
	{
		if (!bIsDirectory)
		{
			FString Extension = FPaths::GetExtension( InFilenameOrDirectory ).ToLower();

			if ( ExtensionSearchSet.Find( Extension ) )
			{
				FoundFiles.Add( FPaths::ConvertRelativePathToFull( InFilenameOrDirectory ) );
			}
		}

		return true; // continue iteration
	};

	if(bRecursive)
	{
		IFileManager::Get().IterateDirectoryRecursively( *FolderPath, VisitDirectory );
	}
	else
	{
		IFileManager::Get().IterateDirectory( *FolderPath, VisitDirectory );
	}

	return FoundFiles;
}

class SDatasmithFileProducerFileProperty : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDatasmithFileProducerFileProperty)
	{}

	SLATE_ARGUMENT(UDatasmithFileProducer*, Producer)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs)
	{
		ProducerPtr = TWeakObjectPtr< UDatasmithFileProducer >( InArgs._Producer );

		FSlateFontInfo FontInfo = IDetailLayoutBuilder::GetDetailFont();

		ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SAssignNew(FileName, SEditableText)
				.IsReadOnly(true)
				.Text(this, &SDatasmithFileProducerFileProperty::GetFilenameText)
				.ToolTipText(this, &SDatasmithFileProducerFileProperty::GetFilenameText)
				.Font( FontInfo )
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SButton)
				.OnClicked(this, &SDatasmithFileProducerFileProperty::OnChangePathClicked )
				.ToolTipText(LOCTEXT("ChangeSourcePath_Tooltip", "Browse for a new source file path"))
				[
					SNew(STextBlock)
					.Text(LOCTEXT("...", "..."))
					.Font( FontInfo )
				]
			]
		];
	}

private:
	FReply OnChangePathClicked() const
	{
		if( !ProducerPtr.IsValid() )
		{
			return FReply::Unhandled();
		}

		FString SelectedFile = FDatasmithFileProducerUtils::SelectFileToImport();
		if(!SelectedFile.IsEmpty())
		{
			const FScopedTransaction Transaction( LOCTEXT("Producer_SetFilename", "Set Filename") );

			ProducerPtr->SetFilePath( SelectedFile );
			FileName->SetText( GetFilenameText() );
		}

		return FReply::Handled();
	}

	FText GetFilenameText() const
	{
		return ProducerPtr->FilePath.IsEmpty() ? FText::FromString( TEXT("Select a file") ) : FText::FromString( ProducerPtr->FilePath );
	}

private:
	TWeakObjectPtr< UDatasmithFileProducer > ProducerPtr;
	TSharedPtr< SEditableText > FileName;
};

void FDataprepContentProducerDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray< TWeakObjectPtr< UObject > > Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);
	check(Objects.Num() > 0);

	Producer = Cast< UDataprepContentProducer >(Objects[0].Get());
	check(Producer);

	AssetProducers = Cast<UDataprepAssetProducers>(Producer->GetOuter());
	check(AssetProducers);

	ProducerIndex = INDEX_NONE;
	for (int Index = 0; Index < AssetProducers->GetProducersCount(); ++Index)
	{
		if (Producer == AssetProducers->GetProducer(Index))
		{
			ProducerIndex = Index;
			break;
		}
	}
}

FSlateColor FDataprepContentProducerDetails::GetStatusColorAndOpacity() const
{
	return  IsProducerSuperseded() ? FLinearColor::Red : FAppStyle::Get().GetSlateColor("DefaultForeground");
}

bool FDataprepContentProducerDetails::IsProducerSuperseded() const
{
	return ProducerIndex != INDEX_NONE ? AssetProducers->IsProducerSuperseded(ProducerIndex) : false;
}

void FDatasmithFileProducerDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	FDataprepContentProducerDetails::CustomizeDetails(DetailBuilder);

	UDatasmithFileProducer* FileProducer = Cast< UDatasmithFileProducer >(Producer);
	check( FileProducer );

	// #ueent_todo: Remove handling of warning category when this is not considered experimental anymore
	TArray<FName> CategoryNames;
	DetailBuilder.GetCategoryNames( CategoryNames );
	CategoryNames.Remove( FName(TEXT("Warning")) );

	DetailBuilder.HideCategory(FName( TEXT( "Warning" ) ) );

	//FName CategoryName( TEXT("DatasmithFileProducerCustom") );
	IDetailCategoryBuilder& ImportSettingsCategoryBuilder = DetailBuilder.EditCategory( NAME_None, FText::GetEmpty(), ECategoryPriority::Important );

	TSharedRef<IPropertyHandle> PropertyHandle = DetailBuilder.GetProperty( TEXT("FilePath"), UDatasmithFileProducer::StaticClass() );
	PropertyHandle->MarkHiddenByCustomization();

	FDetailWidgetRow& CustomAssetImportRow = ImportSettingsCategoryBuilder.AddCustomRow( FText::FromString( TEXT( "Import File" ) ) );

	TSharedPtr<STextBlock> IconText;

	TArray<TObjectPtr<UDatasmithOptionsBase>> Options = FileProducer->GetTranslatorImportOptions();

	CustomAssetImportRow.NameContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(0, 0, 3, 0)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.IsFocusable(false)
			.IsEnabled(Options.Num() > 0)
			.OnClicked_Lambda( [FileProducer]() -> FReply 
			{ 
				FileProducer->OnChangeImportSettings(); 
				return FReply::Handled(); 
			})
			.ToolTipText(LOCTEXT("ChangeImportSettings_Tooltip", "Import Settings"))
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
				.ColorAndOpacity(FLinearColor::White)
				.Text(FEditorFontGlyphs::Cog)
			]
		]

		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SAssignNew(IconText, STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
				.Text(MakeAttributeLambda([this]
				{
					return IsProducerSuperseded() ? FEditorFontGlyphs::Exclamation_Triangle : FEditorFontGlyphs::File;
				}))
				.ColorAndOpacity(this, &FDataprepContentProducerDetails::GetStatusColorAndOpacity)
		]
	];

	IconText->SetToolTipText(MakeAttributeLambda([this]
	{
		if (IsProducerSuperseded())
		{
			return LOCTEXT("FDatasmithFileProducerDetails_StatusTextTooltip_Superseded", "This producer is superseded by another one and will be skipped when run.");
		}
		return LOCTEXT("FDatasmithFileProducerDetails_StatusTextTooltip", "File input");
	}));

	CustomAssetImportRow.ValueContent()
	.MinDesiredWidth( 2000.0f )
	[
		SNew( SDatasmithFileProducerFileProperty )
		.Producer( FileProducer )
	];
}

class SDatasmithDirProducerFolderProperty : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDatasmithDirProducerFolderProperty)
	{}

	SLATE_ARGUMENT( UDatasmithDirProducer*, Producer )
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs)
	{
		ProducerPtr = TWeakObjectPtr< UDatasmithDirProducer >( InArgs._Producer );

		FSlateFontInfo FontInfo = IDetailLayoutBuilder::GetDetailFont();

		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SAssignNew(FolderName, SEditableText)
					.IsReadOnly(true)
					.Text(this, &SDatasmithDirProducerFolderProperty::GetFilenameText)
					.ToolTipText(this, &SDatasmithDirProducerFolderProperty::GetFilenameText)
					.Font(FontInfo)
				]
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SButton)
					.OnClicked( this, &SDatasmithDirProducerFolderProperty::OnChangePathClicked )
					.ToolTipText(LOCTEXT("ChangePath_Tooltip", "Browse for a new folder path"))
					[
						SNew(STextBlock)
						.Text(LOCTEXT("...", "..."))
						.Font( FontInfo )
					]
				]
			]
			+ SVerticalBox::Slot()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(2, 0, 0, 0)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("FileProducer_Extension", "Extension"))
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(2, 0, 0, 0)
					[
						SNew(SBox)
						.WidthOverride(80)
						[
							SNew(SEditableTextBox)
							.Text(LOCTEXT("*.*", "*.*"))
							.Font(IDetailLayoutBuilder::GetDetailFont())
							.OnTextCommitted(this, &SDatasmithDirProducerFolderProperty::OnExtensionStringChanged)
						]
					]
				]
				+ SHorizontalBox::Slot()
				.Padding(30, 0, 0, 0)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("FileProducer_Recursive", "Recursive"))
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
					+ SHorizontalBox::Slot()
					.Padding(2, 0, 0, 0)
					[
						SNew(SCheckBox)
						.IsChecked(this, &SDatasmithDirProducerFolderProperty::IsRecursiveChecked)
						.OnCheckStateChanged(this, &SDatasmithDirProducerFolderProperty::OnRecursiveStateChanged)
					]
				]
			]
		];
	}

private:
	FReply OnChangePathClicked() const
	{
		if( !ProducerPtr.IsValid() )
		{
			return FReply::Unhandled();
		}

		const FString SelectedFolder = FDatasmithFileProducerUtils::SelectDirectory();
		if( !SelectedFolder.IsEmpty() )
		{
			const FScopedTransaction Transaction( LOCTEXT("Producer_SetFolderName", "Set Folder Name") );
			ProducerPtr->SetFolderPath( SelectedFolder );

			FolderName->SetText( GetFilenameText() );
		}

		return FReply::Handled();
	}

	FText GetFilenameText() const
	{
		return ProducerPtr->FolderPath.IsEmpty() ? FText::FromString( TEXT("Select a folder") ) : FText::FromString( ProducerPtr->FolderPath );
	}

	ECheckBoxState IsRecursiveChecked() const
	{
		return ProducerPtr->bRecursive ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	void OnRecursiveStateChanged(ECheckBoxState NewState)
	{
		ProducerPtr->bRecursive = (NewState == ECheckBoxState::Checked);
	}

	void OnExtensionStringChanged( const FText &NewExtensionString, ETextCommit::Type CommitType )
	{
		ProducerPtr->ExtensionString = NewExtensionString.ToString();
		ProducerPtr->UpdateExtensions();

	}

private:
	TWeakObjectPtr< UDatasmithDirProducer > ProducerPtr;
	TSharedPtr< SEditableText > FolderName;
};

void FDatasmithDirProducerDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	FDataprepContentProducerDetails::CustomizeDetails(DetailBuilder);

	UDatasmithDirProducer* DirProducer = Cast< UDatasmithDirProducer >(Producer);
	check(DirProducer);

	// #ueent_todo: Remove handling of warning category when this is not considered experimental anymore
	TArray<FName> CategoryNames;
	DetailBuilder.GetCategoryNames( CategoryNames );
	CategoryNames.Remove( FName(TEXT("Warning")) );

	DetailBuilder.HideCategory(FName( TEXT( "Warning" ) ) );

	FName CategoryName( TEXT("DatasmithDirProducerCustom") );
	IDetailCategoryBuilder& ImportSettingsCategoryBuilder = DetailBuilder.EditCategory( CategoryName, FText::GetEmpty(), ECategoryPriority::Important );

	FDetailWidgetRow& CustomAssetImportRow = ImportSettingsCategoryBuilder.AddCustomRow( FText::FromString( TEXT( "Import Folder" ) ) );

	TSharedPtr<STextBlock> IconText;

	CustomAssetImportRow.NameContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Top)
		.AutoWidth()
		.Padding(0, 0, 3, 0)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.IsFocusable(false)
			.OnClicked_Lambda([DirProducer]() -> FReply
			{
				DirProducer->OnChangeImportSettings();
				return FReply::Handled();
			})
			.ToolTipText(LOCTEXT("ChangeImportSettings_Tooltip", "Import Settings"))
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
				.ColorAndOpacity(FLinearColor::White)
				.Text(FEditorFontGlyphs::Cog)
			]
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Top)
		.AutoWidth()
		.Padding(0, 3, 3, 0)
		[
			SAssignNew(IconText, STextBlock)
			.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
			.Text(MakeAttributeLambda([this]
			{
				return IsProducerSuperseded() ? FEditorFontGlyphs::Exclamation_Triangle : FEditorFontGlyphs::Folder;
			}))
			.ColorAndOpacity(this, &FDataprepContentProducerDetails::GetStatusColorAndOpacity)
		]
	];

	IconText->SetToolTipText(MakeAttributeLambda([this]
	{
		if (IsProducerSuperseded())
		{
			return LOCTEXT("FDatasmithDirProducerDetails_StatusTextTooltip_Superseded", "This producer is superseded by another one and will be skipped when run.");
		}
		return LOCTEXT("DatasmithDirProducerDetails_ImportDirTooltip", "The folder which to load files from");
	}));

	CustomAssetImportRow.ValueContent()
	.MinDesiredWidth( 2000.0f )
	[
		SNew( SDatasmithDirProducerFolderProperty )
		.Producer( DirProducer )
	];

	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UDatasmithDirProducer, ExtensionString));
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UDatasmithDirProducer, bRecursive));
}

void FDatasmithFileProducerUtils::DeletePackagePath( const FString& PathToDelete )
{
	if(PathToDelete.IsEmpty())
	{
		return;
	}

	// Inspired from ContentBrowserUtils::DeleteFolders
	// Inspired from ContentBrowserUtils::LoadAssetsIfNeeded

	// Form a filter from the paths
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	FARFilter Filter;
	Filter.bRecursivePaths = true;
	new (Filter.PackagePaths) FName(*PathToDelete);

	// Query for a list of assets in the selected paths
	TArray<FAssetData> AssetDataList;
	AssetRegistryModule.Get().GetAssets(Filter, AssetDataList);

	bool bAllowFolderDelete = false;

	{
		struct FEmptyFolderVisitor : public IPlatformFile::FDirectoryVisitor
		{
			bool bIsEmpty;

			FEmptyFolderVisitor()
				: bIsEmpty(true)
			{
			}

			virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
			{
				if (!bIsDirectory)
				{
					bIsEmpty = false;
					return false; // abort searching
				}

				return true; // continue searching
			}
		};

		FString PathToDeleteOnDisk;
		if (FPackageName::TryConvertLongPackageNameToFilename(PathToDelete, PathToDeleteOnDisk))
		{
			// Look for files on disk in case the folder contains things not tracked by the asset registry
			FEmptyFolderVisitor EmptyFolderVisitor;
			IFileManager::Get().IterateDirectoryRecursively(*PathToDeleteOnDisk, EmptyFolderVisitor);

			if (EmptyFolderVisitor.bIsEmpty && IFileManager::Get().DeleteDirectory(*PathToDeleteOnDisk, false, true))
			{
				AssetRegistryModule.Get().RemovePath(PathToDelete);
			}
		}
	}
}

void UDatasmithFileProducer::LoadDefaultSettings()
{
	// Read default settings, tessellation and import, for Datasmith file producer
	const FString DatasmithImporterIni = FConfigCacheIni::NormalizeConfigIniPath(FString::Printf(TEXT("%s%s/%s.ini"), *FPaths::GeneratedConfigDir(), ANSI_TO_TCHAR(FPlatformProperties::PlatformName()), TEXT("DatasmithImporter")));

	const TCHAR* TessellationSectionName = TEXT("FileProducerTessellationOptions");
	if(GConfig->DoesSectionExist( TessellationSectionName, DatasmithImporterIni ))
	{

		GConfig->GetFloat( TessellationSectionName, TEXT("ChordTolerance"), DefaultTessellationOptions.ChordTolerance, DatasmithImporterIni);
		GConfig->GetFloat( TessellationSectionName, TEXT("MaxEdgeLength"), DefaultTessellationOptions.MaxEdgeLength, DatasmithImporterIni);
		GConfig->GetFloat( TessellationSectionName, TEXT("NormalTolerance"), DefaultTessellationOptions.NormalTolerance, DatasmithImporterIni);

		FString StitchingTechnique = GConfig->GetStr( TessellationSectionName, TEXT("StitchingTechnique"), DatasmithImporterIni);
		if(StitchingTechnique == TEXT("StitchingHeal"))
		{
			DefaultTessellationOptions.StitchingTechnique =  EDatasmithCADStitchingTechnique::StitchingHeal;
		}
		else if(StitchingTechnique == TEXT("StitchingSew"))
		{
			DefaultTessellationOptions.StitchingTechnique =  EDatasmithCADStitchingTechnique::StitchingSew;
		}
		else
		{
			DefaultTessellationOptions.StitchingTechnique =  EDatasmithCADStitchingTechnique::StitchingNone;
		}
	}

	const TCHAR* ImportSectionName = TEXT("FileProducerImportOptions");
	if(GConfig->DoesSectionExist( ImportSectionName, DatasmithImporterIni ))
	{
		GConfig->GetBool( ImportSectionName, TEXT("IncludeGeometry"), DefaultImportOptions.bIncludeGeometry, DatasmithImporterIni);
		GConfig->GetBool( ImportSectionName, TEXT("IncludeMaterial"), DefaultImportOptions.bIncludeMaterial, DatasmithImporterIni);
		GConfig->GetBool( ImportSectionName, TEXT("IncludeLight"), DefaultImportOptions.bIncludeLight, DatasmithImporterIni);
		GConfig->GetBool( ImportSectionName, TEXT("IncludeCamera"), DefaultImportOptions.bIncludeCamera, DatasmithImporterIni);
		GConfig->GetBool( ImportSectionName, TEXT("IncludeAnimation"), DefaultImportOptions.bIncludeAnimation, DatasmithImporterIni);

		FString SceneHandling = GConfig->GetStr( ImportSectionName, TEXT("SceneHandling"), DatasmithImporterIni);
		if(SceneHandling == TEXT("NewLevel"))
		{
			DefaultImportOptions.SceneHandling =  EDatasmithImportScene::NewLevel;
		}
		else if(SceneHandling == TEXT("AssetsOnly"))
		{
			DefaultImportOptions.SceneHandling =  EDatasmithImportScene::AssetsOnly;
		}
		else
		{
			DefaultImportOptions.SceneHandling =  EDatasmithImportScene::CurrentLevel;
		}
	}
}

FString FDatasmithFileProducerUtils::SelectFileToImport()
{
	const TArray<FString>& Formats = FDatasmithTranslatorManager::Get().GetSupportedFormats();

	FString FileTypes;
	FString AllExtensions;

	for( const FString& Format : Formats )
	{
		TArray<FString> FormatComponents;
		Format.ParseIntoArray( FormatComponents, TEXT( ";" ), false );

		for ( int32 ComponentIndex = 0; ComponentIndex < FormatComponents.Num(); ComponentIndex += 2 )
		{
			check( FormatComponents.IsValidIndex( ComponentIndex + 1 ) );
			const FString& Extension = FormatComponents[ComponentIndex];
			const FString& Description = FormatComponents[ComponentIndex + 1];

			if ( !AllExtensions.IsEmpty() )
			{
				AllExtensions.AppendChar( TEXT( ';' ) );
			}
			AllExtensions.Append( TEXT( "*." ) );
			AllExtensions.Append( Extension );

			if ( !FileTypes.IsEmpty() )
			{
				FileTypes.AppendChar( TEXT( '|' ) );
			}

			FileTypes.Append( FString::Printf( TEXT( "%s (*.%s)|*.%s" ), *Description, *Extension, *Extension ) );
		}
	}

	FString SupportedExtensions( FString::Printf( TEXT( "All Files (%s)|%s|%s" ), *AllExtensions, *AllExtensions, *FileTypes ) );

	TArray<FString> OpenedFiles;
	FString DefaultLocation( FEditorDirectories::Get().GetLastDirectory( ELastDirectory::GENERIC_IMPORT ) );
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();

	bool bOpened = false;
	if ( DesktopPlatform )
	{
		bOpened = DesktopPlatform->OpenFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs( nullptr ),
			LOCTEXT( "FileDialogTitle", "Import Datasmith" ).ToString(),
			DefaultLocation,
			TEXT( "" ),
			SupportedExtensions,
			EFileDialogFlags::None,
			OpenedFiles
		);
	}

	if ( bOpened && OpenedFiles.Num() > 0 )
	{
		const FString& OpenedFile = OpenedFiles[0];
		FEditorDirectories::Get().SetLastDirectory( ELastDirectory::GENERIC_IMPORT, FPaths::GetPath(OpenedFile) );

		return FPaths::ConvertRelativePathToFull( OpenedFile );
	}

	return FString();
}

FString FDatasmithFileProducerUtils::SelectDirectory()
{
	if( IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get() )
	{
		FString DestinationFolder;
		const void* ParentWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);
		const FString Title = LOCTEXT("DatasmithDirProducerFolderTitle", "Choose a folder").ToString();
		const FString DefaultLocation( FEditorDirectories::Get().GetLastDirectory( ELastDirectory::GENERIC_IMPORT ) );

		const bool bFolderSelected = DesktopPlatform->OpenDirectoryDialog(
			ParentWindowHandle,
			Title,
			DefaultLocation,
			DestinationFolder
		);

		if( bFolderSelected )
		{
			FEditorDirectories::Get().SetLastDirectory(ELastDirectory::GENERIC_EXPORT, DestinationFolder);

			return FPaths::ConvertRelativePathToFull( DestinationFolder );
		}
	}

	return FString();
}

#undef LOCTEXT_NAMESPACE
