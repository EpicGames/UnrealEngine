// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DataprepContentProducer.h"

#include "DatasmithContentEditorModule.h"
#include "DatasmithImportContext.h"
#include "DatasmithImportOptions.h"
#include "DatasmithScene.h"
#include "DatasmithTranslatableSource.h"

#include "IDetailCustomization.h"
#include "Input/Reply.h"

#include "DatasmithFileProducer.generated.h"

class IDetailLayoutBuilder;

namespace UE::DatasmithImporter
{
	class FExternalSource;
}

UCLASS(Experimental, HideCategories = (DatasmithProducer_Internal))
class DATASMITHIMPORTER_API UDatasmithFileProducer : public UDataprepContentProducer
{
	GENERATED_BODY()

public:
	// UObject interface
	virtual void PostEditUndo() override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UObject interface

	/** Update producer with the desired file */
	void SetFilePath( const FString& InFilePath );

	const FString& GetFilePath() const { return FilePath; }

	/** Load default settings for file producer in DatasmithImporter.ini */
	static void LoadDefaultSettings();

	UE_DEPRECATED(4.26, "SetFilename was renamed to SetFilePath")
	void SetFilename(const FString& InFilename)
	{
		SetFilePath( InFilename );
	}

	void OnChangeImportSettings();

	TArray<TObjectPtr<UDatasmithOptionsBase>> GetTranslatorImportOptions();

	void SetTranslatorImportOptions(TArray<TObjectPtr<UDatasmithOptionsBase>> InOptions)
	{
		ImportOptionsOverride = InOptions;
	}

	// Begin UDataprepContentProducer overrides
	virtual const FText& GetLabel() const override;
	virtual const FText& GetDescription() const override;
	virtual FString GetNamespace() const override;
	virtual bool Supersede(const UDataprepContentProducer* OtherProducer) const override;
	virtual bool CanAddToProducersArray(bool bIsAutomated) override;

protected:
	virtual bool Initialize() override;
	virtual bool Execute(TArray< TWeakObjectPtr< UObject > >& OutAssets) override;
	virtual void Reset() override;
	// End UDataprepContentProducer overrides

	UPROPERTY( EditAnywhere, Category = DatasmithProducer )
	FString FilePath;

	TWeakObjectPtr<ADatasmithSceneActor> ImportSceneActor;

private:
	/** Fill up world with content of Datasmith scene element */
	void SceneElementToWorld();

	/** Fill up world with content of Datasmith scene element */
	void PreventNameCollision();

	/** Does what is required after setting a new FilePath */
	void OnFilePathChanged();

	/** Update the name of the producer based on the filename */
	void UpdateName();

	bool InitTranslator();

private:
	TUniquePtr< FDatasmithImportContext > ImportContextPtr;
	TUniquePtr< FDataprepWorkReporter > ProgressTaskPtr;
	TSharedPtr< UE::DatasmithImporter::FExternalSource > ExternalSourcePtr;

	TArray<TObjectPtr<UDatasmithOptionsBase>> ImportOptionsOverride;

	UPROPERTY( Transient, DuplicateTransient )
	TObjectPtr<UDatasmithScene> DatasmithScene;

	UPROPERTY( Transient, DuplicateTransient )
	TObjectPtr<UPackage> TransientPackage = nullptr;

	TArray< TWeakObjectPtr< UObject > > Assets;

	UPROPERTY()
	TArray<TObjectPtr<UDatasmithOptionsBase>> TranslatorImportOptions;

	UPROPERTY()
	bool bTranslatorImportOptionsInitialized = false;

	static FDatasmithTessellationOptions DefaultTessellationOptions;
	static FDatasmithImportBaseOptions DefaultImportOptions;

	friend class SDatasmithFileProducerFileProperty;
	friend class UDatasmithDirProducer;
};

class FDataprepContentProducerDetails : public IDetailCustomization
{
public:
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;
	FSlateColor GetStatusColorAndOpacity() const;
	bool IsProducerSuperseded() const;

protected:
	class UDataprepAssetProducers* AssetProducers = nullptr;
	UDataprepContentProducer* Producer = nullptr;
	int32 ProducerIndex = INDEX_NONE; // This index does not change for the lifetime of the property widget
};

// Customization of the details of the Datasmith Scene for the data prep editor.
class DATASMITHIMPORTER_API FDatasmithFileProducerDetails : public FDataprepContentProducerDetails
{
public:
	static TSharedRef< IDetailCustomization > MakeDetails() { return MakeShared<FDatasmithFileProducerDetails>(); };

	/** Called when details should be customized */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;
};

UCLASS(Experimental, HideCategories = (DatasmithDirProducer_Internal))
class DATASMITHIMPORTER_API UDatasmithDirProducer : public UDataprepContentProducer
{
	GENERATED_BODY()

protected:
	UDatasmithDirProducer();

public:
	/** Update producer with the selected folder name */
	void SetFolderPath(const FString& InFolderPath);
	FString GetFolderPath() const;

	void SetExtensionString(const FString& InExtensionString);
	FString GetExtensionString() const;

	void SetIsRecursive(bool bInRecursive);
	bool IsRecursive() const;

	// Begin UObject interface
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UObject interface

	UE_DEPRECATED(4.26, "SetFolderName was renamed to SetFolderPath")
	void SetFolderName(const FString& InFolderName)
	{
		SetFolderPath( InFolderName );
	}

	// Begin UDataprepContentProducer overrides
	virtual const FText& GetLabel() const override;
	virtual const FText& GetDescription() const override;
	virtual FString GetNamespace() const override;
	virtual bool Supersede(const UDataprepContentProducer* OtherProducer) const override;
	virtual bool CanAddToProducersArray(bool bIsAutomated) override;

protected:
	virtual bool Initialize() override;
	virtual bool Execute(TArray< TWeakObjectPtr< UObject > >& OutAssets) override;
	virtual void Reset() override;
	// End UDataprepContentProducer overrides

	void OnChangeImportSettings();
	void SetFileProducerSettings();

private:

	// The folder were datasmith will look for files to import
	UPROPERTY( EditAnywhere, Category = DatasmithDirProducer_Internal )
	FString FolderPath;

	// Semi-column separated string containing the extensions to consider. By default, set to * to get all extensions.
	UPROPERTY( EditAnywhere, Category = DatasmithDirProducer )
	FString ExtensionString;

	// If true the producer will look for the files in the sub-directories.
	UPROPERTY( EditAnywhere, Category = DatasmithDirProducer, meta = (ToolTip = "If checked, sub-directories will be traversed") )
	bool bRecursive;

	/** Called if the folder path has changed */
	void OnFolderPathChanged();

	/** Called if ExtensionString has changed */
	void OnExtensionsChanged();

	/** Called if bRecursive has changed */
	void OnRecursivityChanged();

	/** Helper function to extract set of extensions based on content of ExtensionString and supported formats */
	void UpdateExtensions();

	/** Helper function to get all matching files in FolderPath based on extensions set */
	TSet<FString> GetSetOfFiles() const;

	/** Update the name of the producer based on the directory name */
	void UpdateName();

	/** Try import files in the FolderPath as parts of constructed PLMXML file */
	bool ImportAsPlmXml(UPackage* RootPackage, TArray< TWeakObjectPtr< UObject > >& OutAssets, TArray<FString>& FilesCouldntBeProcessedWithPlmXml);

	/** 
	 * Make resulting scene hierarchy look the way it looks when each of files is imported individually
	 * Which means each separate CAD file should have DatasmithSceneActor as root actor
	 * In order to have this 
	 * 1. Root ImportSceneActor is removed 
	 * 2. Every child of this ImportSceneActor is converted into DatasmithSceneActor, filling RelatedChildren from this child's subtree
	*/
	void FixPlmXmlHierarchy();

	/** Indicates if ExtensionString contains "*.*" */
	bool bHasWildCardSearch;

	/** Set of extensions to look for */
	TSet< FString > FixedExtensionSet;

	/** Set of files matching folder and extensions */
	TSet< FString > FilesToProcess;

	UPROPERTY( Transient, DuplicateTransient )
	TObjectPtr<UDatasmithFileProducer> FileProducer;

	UPROPERTY()
	TObjectPtr<UDatasmithCommonTessellationOptions> TessellationOptions;

	static TSet< FString > SupportedFormats;

	friend class SDatasmithDirProducerFolderProperty;
	friend class FDatasmithDirProducerDetails;
};

// Customization of the details of the Datasmith Scene for the data prep editor.
class DATASMITHIMPORTER_API FDatasmithDirProducerDetails : public FDataprepContentProducerDetails
{
public:
	static TSharedRef< IDetailCustomization > MakeDetails() { return MakeShared<FDatasmithDirProducerDetails>(); };

	/** Called when details should be customized */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;
};
