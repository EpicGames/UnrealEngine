// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithImportContext.h"

#include "DatasmithActorImporter.h"
#include "DatasmithImportOptions.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithTranslator.h"
#include "DatasmithUtils.h"
#include "UI/DatasmithImportOptionsWindow.h"
#include "Utility/DatasmithImporterUtils.h"

#include "Dom/JsonObject.h"
#include "Engine/StaticMesh.h"
#include "EngineUtils.h"
#include "ExternalSource.h"
#include "ExternalSourceModule.h"
#include "FileHelpers.h"
#include "Interfaces/IMainFrameModule.h"
#include "IUriManager.h"
#include "Misc/FeedbackContext.h"
#include "Misc/Paths.h"
#include "Misc/SecureHash.h"
#include "PackageTools.h"
#include "SourceUri.h"


#define LOCTEXT_NAMESPACE "DatasmithImportPlugin"

namespace DatasmithImportContextImpl
{
	const TCHAR* GetUserOptionPath()
	{
		static FString UserDatasmithOptionsFile = FPaths::Combine(FPlatformProcess::UserSettingsDir(), TEXT("Unreal Engine/Enterprise/Datasmith/Config/UserOptions.ini"));
		return *UserDatasmithOptionsFile;
	}

	void LoadOptionConfig(UObject* InOption)
	{
		if (InOption)
		{
			InOption->LoadConfig(nullptr, GetUserOptionPath());
		}
	}

	void SaveOptionConfig(UObject* InOption)
	{
		if (InOption)
		{
			InOption->SaveConfig(CPF_Config, GetUserOptionPath());
		}
	}
}


class FDatasmithImportOptionHelper
{
public:
	/**
	 * Display options' dialog
	 *
	 * @param	ImportOptions	The options to display
	 * @param	DatasmithScene	The DatasmithScene for which we are displaying the import options.
	 *							if the scene is not valid, default values are used.
	 *
	 * @returns true if user accepted the import.
	 */
	static bool DisplayOptionsDialog(const TArray<UObject*>& ImportOptions, const TSharedPtr< IDatasmithScene >& DatasmithScene);

	/** Update options based on content in JSON object*/
	static void LoadOptions(const TArray<UObject*>& ImportOptions, const TSharedPtr<FJsonObject>& ImportSettingsJson);

	/** Work-around to name uniqueness for UObject class */
	static void CleanUpOptions(const TArray<UObject*>& ImportOptions);
};

bool FDatasmithImportOptionHelper::DisplayOptionsDialog(const TArray<UObject*>& ImportOptions, const TSharedPtr< IDatasmithScene >& DatasmithScene)
{
	TSharedPtr<SWindow> ParentWindow;

	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
		ParentWindow = MainFrame.GetParentWindow();
	}

	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(LOCTEXT("DatasmithImportSettingsTitle", "Datasmith Import Options"))
		.SizingRule(ESizingRule::Autosized);

	// First option object is always a UDatasmithImportOptions object
	UDatasmithImportOptions* MainOptions = Cast<UDatasmithImportOptions>(ImportOptions[0]);
	ensure(MainOptions->FileName.Len() > 0);
	ensure(MainOptions->FilePath.Len() > 0);

	float SceneVersion;
	FString FileSDKVersion;
	if (DatasmithScene)
	{
		LexFromString( SceneVersion, DatasmithScene->GetExporterVersion() );
		FileSDKVersion = DatasmithScene->GetExporterSDKVersion();
	}
	else
	{
		SceneVersion = FDatasmithUtils::GetDatasmithFormatVersionAsFloat();
		FileSDKVersion = FDatasmithUtils::GetEnterpriseVersionAsString();
	}

	TSharedPtr<SDatasmithOptionsWindow> OptionsWindow;
	Window->SetContent
	(
		SAssignNew(OptionsWindow, SDatasmithOptionsWindow)
		.ImportOptions(ImportOptions)
		.WidgetWindow(Window)
		// note: Spacing in text below is intentional for text alignment
		.FileNameText(FText::Format(LOCTEXT("DatasmithImportSettingsFileName", "  Import File  :    {0}"), FText::FromString(MainOptions->FileName)))
		.FilePathText(FText::FromString(MainOptions->FilePath))
		.FileFormatVersion(SceneVersion)
		.FileSDKVersion( FText::FromString(FileSDKVersion))
		.PackagePathText(FText::Format(LOCTEXT("DatasmithImportSettingsPackagePath", "  Import To   :    {0}"), FText::FromString(MainOptions->BaseOptions.AssetOptions.PackagePath.ToString())))
		.ProceedButtonLabel(LOCTEXT("DatasmithOptionWindow_ImportCurLevel", "Import"))
		.ProceedButtonTooltip(LOCTEXT("DatasmithOptionWindow_ImportCurLevel_ToolTip", "Import the file and add to the current Level"))
		.CancelButtonLabel(LOCTEXT("DatasmithOptionWindow_Cancel", "Cancel"))
		.CancelButtonTooltip(LOCTEXT("DatasmithOptionWindow_Cancel_ToolTip", "Cancel importing this file"))
		.MinDetailHeight(320.f)
		.MinDetailWidth(450.f)
	);

	FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);

	MainOptions->bUseSameOptions = OptionsWindow->UseSameOptions();

	return OptionsWindow->ShouldImport();
}

void FDatasmithImportOptionHelper::LoadOptions(const TArray<UObject*>& ImportOptions, const TSharedPtr<FJsonObject>& ImportSettingsJson)
{
	if (!ImportSettingsJson.IsValid() || ImportSettingsJson->Values.Num() == 0)
	{
		return;
	}

	for (UObject* Object : ImportOptions)
	{
		const TSharedPtr<FJsonObject>& OptionDataJsonObject = ImportSettingsJson->GetObjectField(Object->GetName());
		if (OptionDataJsonObject.IsValid())
		{
			for (TFieldIterator<FProperty> It(Object->GetClass()); It; ++It)
			{
				if (OptionDataJsonObject->HasField((*It)->GetNameCPP()))
				{
					FString PropertyValue = OptionDataJsonObject->GetStringField((*It)->GetNameCPP());
					uint8* PropertyAddr = (*It)->ContainerPtrToValuePtr<uint8>(Object);

					(*It)->ImportText_Direct(*PropertyValue, PropertyAddr, Object, 0);
				}
			}
		}
	}
}

void FDatasmithImportOptionHelper::CleanUpOptions(const TArray<UObject*>& ImportOptions)
{
	// The names of the option objects is just used for UI. And, the option objects are used once
	// But the names of UObjects have to be unique and those UObjects are garbaged collected instead of immediately freed.
	// So, the options are renamed to some generic names which remove any future potential name collision
	for (UObject* Option : ImportOptions)
	{
		Option->Rename(nullptr);
	}
}

FDatasmithImportContext::FDatasmithImportContext(const FString& InFileName, bool bLoadConfig, const FName& LoggerName, const FText& LoggerLabel, TSharedPtr<IDatasmithTranslator> InSceneTranslator)
	: SceneTranslator(InSceneTranslator)
	, Options(NewObject<UDatasmithImportOptions>(GetTransientPackage(), TEXT("Datasmith Import Settings")))
	, SceneAsset(nullptr)
	, bUserCancelled(false)
	, bIsAReimport(false)
	, bImportedViaScript(false)
	, FeedbackContext(nullptr)
	, AssetsContext(*this)
	, ContextExtension(nullptr)
	, Logger(LoggerName, LoggerLabel)
	, CurrentSceneActorIndex(0)
	, ReferenceCollector(this)
{
	if (SceneTranslator)
	{
		SceneTranslator->GetSceneImportOptions(AdditionalImportOptions);
	}

	if (bLoadConfig)
	{
		DatasmithImportContextImpl::LoadOptionConfig(Options.Get());
		for (const auto& AdditionalImportOption : AdditionalImportOptions)
		{
			DatasmithImportContextImpl::LoadOptionConfig(AdditionalImportOption.Get());
		}
	}

	// Force the SceneHandling to be on current level by default.
	// Note: This is done because this option was previously persisted and can get overwritten
	UE::DatasmithImporter::FSourceUri SourceUri(UE::DatasmithImporter::FSourceUri::FromFilePath(InFileName));
	Options->BaseOptions.SceneHandling = EDatasmithImportScene::CurrentLevel;
	Options->FileName = FPaths::GetCleanFilename(InFileName);
	Options->FilePath = FPaths::ConvertRelativePathToFull(InFileName);
	FileHash = FMD5Hash::HashFile(*Options->FilePath);

	Options->SourceUri = SourceUri.ToString();
	Options->SourceHash = FileHash;
}


FDatasmithImportContext::FDatasmithImportContext(const TSharedPtr<UE::DatasmithImporter::FExternalSource>& InExternalSource, bool bLoadConfig, const FName& LoggerName, const FText& LoggerLabel)
	: Options(NewObject<UDatasmithImportOptions>(GetTransientPackage(), TEXT("Datasmith Import Settings")))
	, SceneAsset(nullptr)
	, bUserCancelled(false)
	, bIsAReimport(false)
	, bImportedViaScript(false)
	, FeedbackContext(nullptr)
	, AssetsContext(*this)
	, ContextExtension(nullptr)
	, Logger(LoggerName, LoggerLabel)
	, CurrentSceneActorIndex(0)
	, ReferenceCollector(this)
{
	if (InExternalSource)
	{
		SceneTranslator = InExternalSource->GetAssetTranslator();
		if (SceneTranslator)
		{
			SceneTranslator->GetSceneImportOptions(AdditionalImportOptions);
		}

		const FString FileName = InExternalSource->GetFallbackFilepath();
		Options->FileName = FPaths::GetCleanFilename(FileName);
		Options->FilePath = FPaths::ConvertRelativePathToFull(FileName);
		Options->SourceUri = InExternalSource->GetSourceUri().ToString();
		Options->SourceHash = InExternalSource->GetSourceHash();
		SceneName = FDatasmithUtils::SanitizeObjectName(InExternalSource->GetSceneName());

		FileHash = FMD5Hash::HashFile(*Options->FilePath);
	}

	if (bLoadConfig)
	{
		DatasmithImportContextImpl::LoadOptionConfig(Options.Get());
		for (const auto& AdditionalImportOption : AdditionalImportOptions)
		{
			DatasmithImportContextImpl::LoadOptionConfig(AdditionalImportOption.Get());
		}
	}

	// Force the SceneHandling to be on current level by default.
	// Note: This is done because this option was previously persisted and can get overwritten
	Options->BaseOptions.SceneHandling = EDatasmithImportScene::CurrentLevel;
}

void FDatasmithImportContext::UpdateImportOption(UDatasmithOptionsBase* NewOption)
{
	if (NewOption)
	{
		auto SameClass = [=](const TObjectPtr<UDatasmithOptionsBase>& Option) { return Option->GetClass() == NewOption->GetClass(); };

		if (TObjectPtr<UDatasmithOptionsBase>* PreviousOptionPtr = AdditionalImportOptions.FindByPredicate(SameClass))
		{
			*PreviousOptionPtr = NewOption;
		}
		else
		{
			AdditionalImportOptions.Emplace(NewOption);
		}
	}
}

TSharedRef<FTokenizedMessage> FDatasmithImportContext::LogError(const FText& InErrorMessage)
{
	return Logger.Push(EMessageSeverity::Error, InErrorMessage);
}

TSharedRef<FTokenizedMessage> FDatasmithImportContext::LogWarning(const FText& InWarningMessage, bool bPerformance)
{
	return Logger.Push(bPerformance ? EMessageSeverity::PerformanceWarning : EMessageSeverity::Warning, InWarningMessage);
}

TSharedRef<FTokenizedMessage> FDatasmithImportContext::LogInfo(const FText& InInfoMessage)
{
	return Logger.Push(EMessageSeverity::Info, InInfoMessage);
}

bool FDatasmithImportContext::Init(TSharedRef< IDatasmithScene > InScene, const FString& InImportPath, EObjectFlags InFlags, FFeedbackContext* InWarn, const TSharedPtr<FJsonObject>& ImportSettingsJson, bool bSilent)
{
	InitScene(InScene);

	return InitOptions(ImportSettingsJson, InImportPath, bSilent)
		&& SetupDestination(InImportPath, InFlags, InWarn, bSilent);
}

bool FDatasmithImportContext::Init(const FString& InImportPath, EObjectFlags InFlags, FFeedbackContext* InWarn, const TSharedPtr<FJsonObject>& ImportSettingsJson, bool bSilent)
{
	return InitOptions(ImportSettingsJson, InImportPath, bSilent)
		&& SetupDestination(InImportPath, InFlags, InWarn, bSilent);
}

bool FDatasmithImportContext::InitOptions(TSharedRef< IDatasmithScene > InScene, const TSharedPtr<FJsonObject>& ImportSettingsJson, bool bSilent)
{
	InitScene(InScene);
	const FString ImportPath = Options->BaseOptions.AssetOptions.PackagePath.ToString();
	return InitOptions(ImportSettingsJson, ImportPath, bSilent);
}

bool FDatasmithImportContext::InitOptions(const TSharedPtr<FJsonObject>& ImportSettingsJson, const TOptional<FString>& InImportPath, bool bSilent)
{
	if (!FModuleManager::Get().IsModuleLoaded("AssetTools"))
	{
		UE_LOGF(LogDatasmithImport, Warning, "Import failed. The AssetTools module can't be loaded.");
		return false;
	}

	TArray<UObject*> ImportOptions;
	ImportOptions.Reserve(1 + AdditionalImportOptions.Num());

	ImportOptions.Add(Options.Get());
	for (const TObjectPtr<UDatasmithOptionsBase>& AdditionalOption : AdditionalImportOptions)
	{
		ImportOptions.Add(AdditionalOption);
	}

	if (bSilent)
	{
		// Update options based on given JSON object
		if (ImportSettingsJson.IsValid() && ImportSettingsJson->Values.Num() > 0)
		{
			FDatasmithImportOptionHelper::LoadOptions(ImportOptions, ImportSettingsJson);
		}
	}
	else
	{
		if (!InImportPath.IsSet())
		{
			UE_LOGF(LogDatasmithImport, Display, "Can't call FDatasmithImportContext::InitOptions(), with bSilent=false if InImportPath is not set.");
			return false;
		}

		Options->BaseOptions.AssetOptions.PackagePath = FName(*InImportPath.GetValue()); // Package path displayed in options window

		SetupBaseOptionsVisibility();

		bool bShouldImport = FDatasmithImportOptionHelper::DisplayOptionsDialog(ImportOptions, Scene);

		ResetBaseOptionsVisibility();

		if (!bShouldImport)
		{
			FDatasmithImportOptionHelper::CleanUpOptions(ImportOptions);
			UE_LOGF(LogDatasmithImport, Display, "Import canceled.");
			return false;
		}

		// Update config file with new values
		for (UObject* Option : ImportOptions)
		{
			DatasmithImportContextImpl::SaveOptionConfig(Option);
		}
	}

	if (SceneTranslator)
	{
		AdditionalImportOptions.Push(Options);
		SceneTranslator->SetSceneImportOptions(AdditionalImportOptions);
		AdditionalImportOptions.Pop();
	}

	Options->UpdateNotDisplayedConfig(bIsAReimport);

	FDatasmithImportOptionHelper::CleanUpOptions(ImportOptions);
	return true;
}

bool FDatasmithImportContext::SetupDestination(const FString& InImportPath, EObjectFlags InFlags, FFeedbackContext* InWarn, bool bSilent)
{
	if (SceneName.IsEmpty())
	{
		//Scene must be valid for initializing AssetsContext.
		return false;
	}

	Options->BaseOptions.AssetOptions.PackagePath = FName(*InImportPath);

	if (!ActorsContext.ImportWorld)
	{
		// User is asking to import model in a new level
		// Check to see if there is nothing to save and act according to user's selection
		// Note: The code below has been borrowed from UEditorEngine::CreateNewMapForEditing
		if (Options->BaseOptions.SceneHandling == EDatasmithImportScene::NewLevel)
		{
			if (!bSilent)
			{
				// Check to see if there are unsaved data and user wants to save them
				// Import will abort if user selects cancel on Save dialog
				bool bPromptUserToSave = true; // Ask user if they want to save the unsaved data
				bool bSaveMapPackages = true;
				bool bSaveContentPackages = true;
				if (FEditorFileUtils::SaveDirtyPackages(bPromptUserToSave, bSaveMapPackages, bSaveContentPackages) == false)
				{
					// The user pressed cancel. Abort the import so the user doesn't lose any changes
					return false;
				}
			}

			// Force the creation of a new level
			ActorsContext.ImportWorld = UEditorLoadingAndSavingUtils::NewBlankMap(false);
		}
		else if (Options->BaseOptions.SceneHandling == EDatasmithImportScene::CurrentLevel)
		{
			ActorsContext.ImportWorld = GWorld;
			if (ActorsContext.ImportWorld == nullptr)
			{
				UE_LOGF(LogDatasmithImport, Warning, "Import failed. There is no World/Map open in the Editor.");
				return false;
			}
		}
		else
		{
			ActorsContext.ImportWorld = nullptr;
		}
	}

	FeedbackContext = InWarn;

	bUserCancelled = false;

	ObjectFlags = InFlags | RF_Transactional;

	bool bResult = AssetsContext.Init();

	if (ShouldImportActors())
	{
		bResult = bResult && ActorsContext.Init();
	}

	return bResult;
}

void FDatasmithImportContext::InitScene(const TSharedRef<IDatasmithScene>& InScene)
{
	using namespace UE::DatasmithImporter;
	Scene = InScene;

	// Initialize the filtered scene as a copy of the original scene. We will use it to then filter out items to import.
	FilteredScene = FDatasmithSceneFactory::DuplicateScene(Scene.ToSharedRef());
	SceneName = FDatasmithUtils::SanitizeObjectName(Scene->GetName());

	// If the hash was not valid during initialization, try to get it again, now that the scene is loaded.
	if (!Options->SourceHash.IsValid())
	{
		const FSourceUri SourceUri(Options->SourceUri);

		if (const TSharedPtr<FExternalSource> ExternalSource = IExternalSourceModule::GetOrCreateExternalSource(SourceUri))
		{
			Options->SourceHash = ExternalSource->GetSourceHash();
		}
	}
}

void FDatasmithImportContext::DisplayMessages()
{
	Logger.Dump();
}

void FDatasmithImportContext::SetupBaseOptionsVisibility()
{
	if ( FProperty* ReimportOptionsProperty = FindFProperty< FProperty >( Options->GetClass(), GET_MEMBER_NAME_CHECKED( UDatasmithImportOptions, ReimportOptions ) ) )
	{
		if ( bIsAReimport )
		{
			ReimportOptionsProperty->SetMetaData( TEXT("Category"), TEXT("Reimport") );
			ReimportOptionsProperty->SetMetaData( TEXT("ShowOnlyInnerProperties"), TEXT("1") );
		}
		else
		{
			ReimportOptionsProperty->SetMetaData( TEXT("Category"), TEXT("NotVisible") );
			ReimportOptionsProperty->RemoveMetaData( TEXT("ShowOnlyInnerProperties") );
		}
	}
}

void FDatasmithImportContext::ResetBaseOptionsVisibility()
{
	FProperty* ReimportOptionsProperty = FindFProperty< FProperty >( Options->GetClass(), GET_MEMBER_NAME_CHECKED( UDatasmithImportOptions, ReimportOptions ) );

	if ( ReimportOptionsProperty )
	{
		ReimportOptionsProperty->SetMetaData( TEXT("ShowOnlyInnerProperties"), TEXT("1") );
		ReimportOptionsProperty->SetMetaData( TEXT("Category"), TEXT("NotVisible") );
	}
}

void FDatasmithImportContext::AddImportedActor(AActor* InActor)
{
	ImportedActorMap.Add(InActor->GetName(), InActor);
}

TArray<AActor*> FDatasmithImportContext::GetImportedActors() const
{
	TArray<AActor*> Result;
	if (ImportedActorMap.Num())
	{
		Result.Reserve(ImportedActorMap.Num());
		for (const auto& Itt : ImportedActorMap)
		{
			Result.Add(Itt.Value);
		}
	}

	return Result;
}

void FDatasmithImportContext::AddSceneComponent(const FString & InName, USceneComponent* InMeshComponent)
{
	ImportedSceneComponentMap.Add(InName, InMeshComponent);
}

bool FDatasmithImportContext::ShouldImportActors() const
{
	return ActorsContext.ImportWorld && Options->BaseOptions.SceneHandling != EDatasmithImportScene::AssetsOnly;
}

FDatasmithImportContext::FInternalReferenceCollector::FInternalReferenceCollector(FDatasmithImportContext* InImportContext)
	: ImportContext(InImportContext)
{
}

namespace DatasmithImportContextInternal
{
	template<typename T>
	void AddReferenceList(FReferenceCollector& Collector, T& List)
	{
		for (auto& Ref : List)
		{
			Collector.AddReferencedObject(Ref.Value);
		}
	}
}

void FDatasmithImportContext::FInternalReferenceCollector::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(ImportContext->ActorsContext.ImportSceneActor);
	Collector.AddReferencedObject(ImportContext->ActorsContext.ImportWorld);
	Collector.AddReferencedObject(ImportContext->ActorsContext.FinalWorld);

	Collector.AddReferencedObject(ImportContext->SceneAsset);

	for (auto& It : ImportContext->ImportedStaticMeshes)
	{
		Collector.AddReferencedObject( It.Value );
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	for (auto& Pair : ImportContext->ImportedClothes)  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	{
		Collector.AddReferencedObject(Pair.Value);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	Collector.AddReferencedObjects(ImportContext->ImportedMaterials);
	Collector.AddReferencedObjects(ImportContext->ImportedParentMaterials);

	for (auto& It : ImportContext->ImportedLevelSequences)
	{
		if ( It.Value )
		{
			Collector.AddReferencedObject(It.Value);
		}
	}

	for (auto& It : ImportContext->ImportedLevelVariantSets)
	{
		if ( It.Value )
		{
			Collector.AddReferencedObject(It.Value);
		}
	}

	DatasmithImportContextInternal::AddReferenceList(Collector, ImportContext->ImportedActorMap);
	DatasmithImportContextInternal::AddReferenceList(Collector, ImportContext->ImportedSceneComponentMap);

	Collector.AddReferencedObject(ImportContext->Options);
	for (TObjectPtr<UDatasmithOptionsBase>& Option : ImportContext->AdditionalImportOptions)
	{
		Collector.AddReferencedObject(Option);
	}
}

FDatasmithActorUniqueLabelProvider::FDatasmithActorUniqueLabelProvider(UWorld* World)
{
	PopulateLabelFrom(World);
}


void FDatasmithActorUniqueLabelProvider::PopulateLabelFrom(UWorld* World)
{
	if (World)
	{
		Clear();
		for (FActorIterator It(World); It; ++It)
		{
			AddExistingName(It->GetActorLabel());
		}
	}
}

FDatasmithActorImportContext::FDatasmithActorImportContext(UWorld* World)
	: ImportSceneActor(nullptr)
	, CurrentTargetedScene(nullptr)
	, ImportWorld(World)
	, FinalWorld(World)
{
}

bool FDatasmithActorImportContext::Init()
{
	UniqueNameProvider.PopulateLabelFrom( ImportWorld ); // TODO: Should probably populate based on FinalWorld
	return true;
}

FDatasmithAssetsImportContext::FDatasmithAssetsImportContext( FDatasmithImportContext& ImportContext )
	: ParentContext( &ImportContext )
{
}

bool FDatasmithAssetsImportContext::Init()
{
	FString NewRootFolder;
	if ( GetParentContext().SceneAsset )
	{
		NewRootFolder = FPackageName::GetLongPackagePath( GetParentContext().SceneAsset->GetOutermost()->GetName() );
	}
	else
	{
		NewRootFolder = FPaths::Combine( GetParentContext().Options->BaseOptions.AssetOptions.PackagePath.ToString(), GetParentContext().SceneName );
	}

	ReInit(NewRootFolder);

	return true;
}

void FDatasmithAssetsImportContext::ReInit(const FString& NewRootFolder)
{
	// Prevent re-initialize twice with the same root folder
	// This happens when reimporting specific assets or scene
	// This used to work but now crashes when creating the transient packages
	if (RootFolderPath == NewRootFolder && StaticMeshesFinalPackage.IsValid())
	{
		return;
	}

	RootFolderPath = UPackageTools::SanitizePackageName(NewRootFolder);

	StaticMeshesFinalPackage.Reset( CreatePackage( *FPaths::Combine( RootFolderPath, TEXT("Geometries") ) ) );
	MaterialsFinalPackage.Reset( CreatePackage( *FPaths::Combine( RootFolderPath, TEXT("Materials") ) ) );
	TexturesFinalPackage.Reset( CreatePackage( *FPaths::Combine( RootFolderPath, TEXT("Textures") ) ) );
	LightPackage.Reset( CreatePackage( *FPaths::Combine( RootFolderPath, TEXT("Lights") ) ) );
	LevelSequencesFinalPackage.Reset( CreatePackage( *FPaths::Combine( RootFolderPath, TEXT("Animations") ) ) );
	LevelVariantSetsFinalPackage.Reset( CreatePackage( *FPaths::Combine( RootFolderPath, TEXT("Variants") ) ) );

	// Use the engine's transient package path as initial root to create the Datasmith transient packages
	TransientFolderPath = FPaths::Combine(GetTransientPackage()->GetPathName(), FMD5::HashAnsiString(*RootFolderPath));

	StaticMeshesImportPackage.Reset( NewObject< UPackage >( nullptr, *FPaths::Combine( TransientFolderPath, TEXT("Geometries") ), RF_Transient ) );
	StaticMeshesImportPackage->FullyLoad();

	TexturesImportPackage.Reset( NewObject< UPackage >( nullptr, *FPaths::Combine( TransientFolderPath, TEXT("Textures") ), RF_Transient ) );
	TexturesImportPackage->FullyLoad();

	MaterialsImportPackage.Reset( NewObject< UPackage >( nullptr, *FPaths::Combine( TransientFolderPath, TEXT("Materials") ), RF_Transient ) );
	MaterialsImportPackage->FullyLoad();

	ReferenceMaterialsImportPackage.Reset(NewObject< UPackage >(nullptr, *FPaths::Combine(TransientFolderPath, TEXT("Materials/References")), RF_Transient));
	ReferenceMaterialsImportPackage->FullyLoad();

	MaterialFunctionsImportPackage.Reset( NewObject< UPackage >(nullptr, *FPaths::Combine(TransientFolderPath, TEXT("Materials/References/Functions")), RF_Transient));
	MaterialFunctionsImportPackage->FullyLoad();

	LevelSequencesImportPackage.Reset( NewObject< UPackage >( nullptr, *FPaths::Combine( TransientFolderPath, TEXT("Animations") ), RF_Transient ) );
	LevelSequencesImportPackage->FullyLoad();

	LevelVariantSetsImportPackage.Reset( NewObject< UPackage >( nullptr, *FPaths::Combine( TransientFolderPath, TEXT("Variants") ), RF_Transient ) );
	LevelVariantSetsImportPackage->FullyLoad();
}

#undef LOCTEXT_NAMESPACE
