// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithImportFactory.h"

#include "DatasmithAssetImportData.h"
#include "DatasmithAssetUserData.h"
#include "DatasmithContentBlueprintLibrary.h"
#include "DatasmithImporterHelper.h"
#include "DatasmithImporterModule.h"
#include "DatasmithScene.h"
#include "DatasmithSceneActor.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithStaticMeshImporter.h"
#include "DatasmithTranslatableSource.h"
#include "DatasmithTranslatorManager.h"
#include "DatasmithUtils.h"
#include "IDatasmithSceneElements.h"
#include "LayoutUV.h"
#include "Utility/DatasmithImporterUtils.h"
#include "Utility/DatasmithImportFactoryHelper.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "ComponentReregisterContext.h"
#include "Engine/StaticMesh.h"
#include "EngineAnalytics.h"
#include "ExternalSource.h"
#include "ExternalSourceModule.h"
#include "HAL/FileManager.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "IUriManager.h"
#include "Logging/LogMacros.h"
#include "Logging/MessageLog.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Misc/PackageName.h"
#include "Misc/ScopedSlowTask.h"
#include "PackageTools.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Subsystems/ImportSubsystem.h"
#include "Templates/UniquePtr.h"

#include "Editor/EditorEngine.h"
#include "MeshUtilities.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
	#include <psapi.h>
#include "Windows/HideWindowsPlatformTypes.h"
#endif

extern UNREALED_API UEditorEngine* GEditor;

#define LOCTEXT_NAMESPACE "DatasmithImportFactory"


namespace DatasmithImportFactoryImpl
{

	static UAssetImportData* GetImportData(UObject* Obj)
	{
		if (UClass* Class = Obj ? Obj->GetClass() : nullptr)
		{
			if (Class == UDatasmithScene::StaticClass())
			{
				// This factory handles both UDatasmithSceneImportData and UDatasmithTranslatedSceneImportData but not other children of UDatasmithSceneImportData
				UDatasmithSceneImportData* SceneAssetImportData = Cast< UDatasmithScene >(Obj)->AssetImportData;

				if ( SceneAssetImportData )
				{
					if ( SceneAssetImportData->GetClass() == UDatasmithSceneImportData::StaticClass() )
					{
						return SceneAssetImportData;
					}
					else
					{
						// UDatasmithTranslatedSceneImportData are associated with scenes imported through Translators system
						return ExactCast<UDatasmithTranslatedSceneImportData>(SceneAssetImportData);
					}
				}
			}
			if (Class == UDatasmithSceneImportData::StaticClass())
			{
				return Cast< UAssetImportData >(Obj);
			}
			if (Class == UStaticMesh::StaticClass())
			{
				return Cast< UDatasmithStaticMeshImportData >(Cast< UStaticMesh >(Obj)->GetAssetImportData());
			}
			if (Class->IsChildOf(UMaterialInterface::StaticClass()))
			{
				return Cast< UMaterialInterface >(Obj)->AssetImportData;
			}
		}

		return nullptr;
	}

	static void SetupSceneViewport(FDatasmithImportContext& InContext)
	{
		if ( !InContext.ShouldImportActors() || !InContext.SceneAsset || InContext.ActorsContext.FinalSceneActors.Num() == 0 || InContext.bIsAReimport)
		{
			return;
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(DatasmithImportFactoryImpl::SetupSceneViewport);

		// Use the first scene actor for the thumbnail
		ADatasmithSceneActor* SceneActor = *InContext.ActorsContext.FinalSceneActors.CreateIterator();

		TArray< FAssetData > AssetDataList;
		AssetDataList.Add( FAssetData( InContext.SceneAsset ) );
		DatasmithImportFactoryHelper::SetupSceneViewport(SceneActor, AssetDataList);
	}

	static bool CreateSceneAsset( FDatasmithImportContext& InContext )
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DatasmithImportFactoryImpl::CreateSceneAsset);

		// Reuse existing asset name or infer from file
		FString AssetName;
		if (InContext.bIsAReimport && InContext.SceneAsset)
		{
			AssetName = InContext.SceneAsset->GetName();
		}

		if (AssetName.IsEmpty())
		{
			AssetName = InContext.Scene->GetName();
		}

		AssetName = FDatasmithUtils::SanitizeObjectName(AssetName);

		FString PackageName = FPaths::Combine(InContext.AssetsContext.RootFolderPath, AssetName);
		PackageName = UPackageTools::SanitizePackageName(PackageName);

		FText CreateAssetFailure = LOCTEXT( "CreateSceneAsset_PackageFailure", "Failed to create the Datasmith Scene asset." );
		FText OutFailureReason;
		if ( !FDatasmithImporterUtils::CanCreateAsset< UDatasmithScene >( PackageName + "." + AssetName, OutFailureReason ) )
		{
			InContext.LogError( OutFailureReason );
			InContext.LogError( CreateAssetFailure );
			return false;
		}

		UDatasmithScene* SceneAsset = FDatasmithImporterUtils::FindObject< UDatasmithScene >( nullptr, PackageName );
		if ( !SceneAsset )
		{
			UPackage* Package = CreatePackage( *PackageName );
			if ( !ensure(Package) )
			{
				InContext.LogError( CreateAssetFailure );
				return false;
			}
			Package->FullyLoad();

			SceneAsset = NewObject< UDatasmithScene >( Package, FName(*AssetName), RF_Public | RF_Standalone );
		}

		UDatasmithTranslatedSceneImportData* ReImportSceneData = NewObject< UDatasmithTranslatedSceneImportData >( SceneAsset );
		SceneAsset->AssetImportData = ReImportSceneData;

		// Copy over the changes the user may have done on the options
		ReImportSceneData->BaseOptions = InContext.Options->BaseOptions;

		for (const TObjectPtr<UDatasmithOptionsBase>& Option : InContext.AdditionalImportOptions)
		{
			UDatasmithOptionsBase* OptionObj = Option.Get();
			OptionObj->Rename(nullptr, ReImportSceneData);
			ReImportSceneData->AdditionalOptions.Add(OptionObj);
		}
		ReImportSceneData->Update( InContext.Options->FilePath, InContext.FileHash.IsValid() ? &InContext.FileHash : nullptr );
		ReImportSceneData->DatasmithImportInfo = FDatasmithImportInfo(InContext.Options->SourceUri, InContext.Options->SourceHash);

		FAssetRegistryModule::AssetCreated(ReImportSceneData);

		InContext.SceneAsset = SceneAsset;

		FDatasmithImporterUtils::SaveDatasmithScene( InContext.Scene.ToSharedRef(), SceneAsset );

		return true;
	}

	bool ImportDatasmithScene(FDatasmithImportContext& InContext, bool& bOutOperationCancelled)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DatasmithImportFactoryImpl::ImportDatasmithScene);

		bOutOperationCancelled = false;

		// Return if the context is not valid
		if ( !InContext.Options )
		{
			return false;
		}

		TUniquePtr<FScopedSlowTask> ProgressPtr;
		FScopedSlowTask* Progress = nullptr;
		if ( InContext.FeedbackContext )
		{
			ProgressPtr = MakeUnique<FScopedSlowTask>(100.0f, LOCTEXT("StartWork", "Unreal Datasmith ..."), true, *InContext.FeedbackContext);
			Progress = ProgressPtr.Get();
			Progress->MakeDialog(true);
		}

		// Filter element that need to be imported depending on dirty state (or eventually depending on options)
		FDatasmithImporter::FilterElementsToImport(InContext);

		// TEXTURES
		// We need the textures before the materials
		if ( Progress )
		{
			Progress->EnterProgressFrame( 20.f );
		}
		FDatasmithImporter::ImportTextures(InContext);

		if ( InContext.bUserCancelled )
		{
			bOutOperationCancelled = true;
			return false;
		}

		// MATERIALS
		// We need to import the materials before the static meshes to know about the meshes build requirements that are driven by the materials
		if ( Progress )
		{
			Progress->EnterProgressFrame( 5.f );
		}
		FDatasmithImporter::ImportMaterials(InContext);

		if ( InContext.bUserCancelled )
		{
			bOutOperationCancelled = true;
			return false;
		}

		// SCENE ASSET
		if ( !DatasmithImportFactoryImpl::CreateSceneAsset( InContext ) )
		{
			return false;
		}

		// STATIC MESHES
		if ( Progress )
		{
			Progress->EnterProgressFrame( 25.f );
		}
		FDatasmithImporter::ImportStaticMeshes( InContext );

		if ( InContext.bUserCancelled )
		{
			bOutOperationCancelled = true;
			return false;
		}

		if ( Progress )
		{
			Progress->EnterProgressFrame( 10.f );
		}
		FDatasmithStaticMeshImporter::PreBuildStaticMeshes(InContext);

		if ( InContext.bUserCancelled )
		{
			bOutOperationCancelled = true;
			return false;
		}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// CLOTH
		FDatasmithImporter::ImportClothes( InContext );  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")

		if ( InContext.bUserCancelled )
		{
			bOutOperationCancelled = true;
			return false;
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		// ACTORS
		if( InContext.ShouldImportActors() )
		{
			if ( Progress )
			{
				Progress->EnterProgressFrame( 10.f );
			}

			FDatasmithImporter::ImportActors( InContext );

			if ( InContext.bUserCancelled )
			{
				bOutOperationCancelled = true;
				return false;
			}

			// Level sequences have to be imported after the actors to be able to bind the tracks to the actors to be animated
			FDatasmithImporter::ImportLevelSequences( InContext );

			// Level variant sets have to be imported after the actors and materials to be able to bind to them correctly
			FDatasmithImporter::ImportLevelVariantSets( InContext );
		}

		if ( InContext.bUserCancelled )
		{
			bOutOperationCancelled = true;
			return false;
		}

		if ( Progress )
		{
			Progress->EnterProgressFrame( 30.f );
		}
		FDatasmithImporter::FinalizeImport(InContext, TSet<UObject*>());

		// Must be called after the actors are spawned since we will compute the scene bounds
		DatasmithImportFactoryImpl::SetupSceneViewport( InContext );

		return true;
	}

	static void SendAnalytics(const FDatasmithImportContext& ImportContext, int32 ImportDurationInSeconds, bool bImportSuccess)
	{
		if ( FEngineAnalytics::IsAvailable() )
		{
			TArray<FAnalyticsEventAttribute> EventAttributes;

			EventAttributes.Emplace( TEXT("ImporterType"), ImportContext.SceneTranslator ? ImportContext.SceneTranslator->GetFName() : NAME_None );
			EventAttributes.Emplace( TEXT("ImportedWithViaScript"), ImportContext.bImportedViaScript );

			// Import info
			EventAttributes.Emplace( TEXT("ImporterVersion"), FDatasmithUtils::GetEnterpriseVersionAsString() );
			EventAttributes.Emplace( TEXT("ImportDuration"), ImportDurationInSeconds );

			// Imported objects count
			EventAttributes.Emplace( TEXT("MeshesCount"), ImportContext.Scene->GetMeshesCount() );
			EventAttributes.Emplace( TEXT("MeshActorsCount"), FDatasmithSceneUtils::GetAllMeshActorsFromScene( ImportContext.Scene ).Num() );
			EventAttributes.Emplace( TEXT("CamerasCount"),  FDatasmithSceneUtils::GetAllCameraActorsFromScene( ImportContext.Scene ).Num() );
			EventAttributes.Emplace( TEXT("LightsCount"), FDatasmithSceneUtils::GetAllLightActorsFromScene( ImportContext.Scene ).Num() );
			EventAttributes.Emplace( TEXT("MaterialsCount"), ImportContext.Scene->GetMaterialsCount() );
			EventAttributes.Emplace( TEXT("TexturesCount"), ImportContext.Scene->GetTexturesCount() );

			EventAttributes.Emplace( TEXT("ExporterVersion"), ImportContext.Scene->GetExporterSDKVersion() );
			EventAttributes.Emplace( TEXT("Vendor"), ImportContext.Scene->GetVendor() );
			EventAttributes.Emplace( TEXT("ProductName"), ImportContext.Scene->GetProductName() );
			EventAttributes.Emplace( TEXT("ProductVersion"), ImportContext.Scene->GetProductVersion() );
			EventAttributes.Emplace( TEXT("ExporterID"), ImportContext.Scene->GetUserID() );
			EventAttributes.Emplace( TEXT("ExporterOS"), ImportContext.Scene->GetUserOS() );
			EventAttributes.Emplace( TEXT("ExportDuration"), ImportContext.Scene->GetExportDuration() );

			if (ImportContext.SceneTranslator)
			{
				EventAttributes.Emplace(TEXT("SourceFileExtension"), ImportContext.SceneTranslator->GetSource().GetSourceFileExtension());

				// Log tessellator if CADKernel has been used
				bool bUseCADKernel = false;
				TArray<TObjectPtr<UDatasmithOptionsBase>> Options;
				ImportContext.SceneTranslator->GetSceneImportOptions(Options);

				for (const TObjectPtr<UDatasmithOptionsBase>& Option : Options)
				{
					if (UDatasmithCommonTessellationOptions* TessellationOptionsObject = Cast<UDatasmithCommonTessellationOptions>(Option))
					{
						bUseCADKernel = TessellationOptionsObject->Options.bUseCADKernel;
					}
				}

				if (bUseCADKernel)
				{
					EventAttributes.Emplace(TEXT("Tessellator"), TEXT("CADKernel"));
				}
			}
			else
			{
				EventAttributes.Emplace(TEXT("SourceFileExtension"), TEXT("Unknown"));
			}

			FString EventText = TEXT("Datasmith.");
			EventText += ImportContext.bIsAReimport ? TEXT("Reimport") : TEXT("Import");
			EventText += bImportSuccess             ? TEXT("")         : TEXT("Fail");
			FEngineAnalytics::GetProvider().RecordEvent( EventText, EventAttributes );
		}
	}

	static void ReportImportStats(const FDatasmithImportContext& ImportContext, uint64 StartTime)
	{
		FLayoutUV::LogStats();

		// Log time spent to import incoming file in minutes and seconds
		double ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);

		DatasmithImportFactoryImpl::SendAnalytics(ImportContext, FMath::RoundToInt(ElapsedSeconds), true);

		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

		FString MemoryStats;
#if PLATFORM_WINDOWS
		PROCESS_MEMORY_COUNTERS_EX MemoryInfo;
		GetProcessMemoryInfo(GetCurrentProcess(), (PROCESS_MEMORY_COUNTERS*)&MemoryInfo, sizeof(MemoryInfo));

		double PrivateBytesGB   = double(MemoryInfo.PrivateUsage) / (1024*1024*1024);
		double WorkingSetGB     = double(MemoryInfo.WorkingSetSize) / (1024*1024*1024);
		double PeakWorkingSetFB = double(MemoryInfo.PeakWorkingSetSize) / (1024*1024*1024);

		MemoryStats = FString::Printf(TEXT(" [Private Bytes: %.02f GB, Working Set %.02f GB, Peak Working Set %.02f GB]"), PrivateBytesGB, WorkingSetGB, PeakWorkingSetFB);
#endif

		int ElapsedMin = int(ElapsedSeconds / 60.0);
		ElapsedSeconds -= 60.0 * (double)ElapsedMin;
		UE_LOGF(LogDatasmithImport, Log, "%ls %ls in [%d min %.3f s]%ls", ImportContext.bIsAReimport ? TEXT("Reimported") : TEXT("Imported"), ImportContext.Scene->GetName(), ElapsedMin, ElapsedSeconds, *MemoryStats);
	}

} // ns DatasmithImportFactoryImpl


UDatasmithImportFactory::UDatasmithImportFactory()
{
	bCreateNew = false;
	bEditAfterNew = true;
	SupportedClass = UDatasmithScene::StaticClass();

	bEditorImport = true;
	bText = false;

	bShowOptions = true;
	bOperationCanceled = false;

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		Formats = FDatasmithTranslatorManager::Get().GetSupportedFormats();

		// FReimportManager automatically register factory on constructor.
		// We unregister all non-CDO in order to avoid n-plicated registered handler for the same factory.
		FReimportManager::Instance()->UnregisterHandler( *this );
	}
}

bool UDatasmithImportFactory::FactoryCanImport(const FString& Filename)
{
	return IsExtensionSupported(Filename);
}

FText UDatasmithImportFactory::GetDisplayName() const
{
	return LOCTEXT("DatasmithImportFactoryDescription", "Datasmith");
}

bool UDatasmithImportFactory::DoesSupportClass(UClass* InClass)
{
	return InClass == UMaterial::StaticClass()
		|| InClass == UMaterialInstanceConstant::StaticClass()
		|| InClass == UStaticMesh::StaticClass()
		|| InClass == UDatasmithScene::StaticClass();
}

UClass* UDatasmithImportFactory::ResolveSupportedClass()
{
	return UDatasmithScene::StaticClass();
}

void UDatasmithImportFactory::CleanUp()
{
	ImportSettingsJson.Reset();
	bOperationCanceled = false;
	bShowOptions = true;
	Super::CleanUp();
}

bool UDatasmithImportFactory::IsExtensionSupported(const FString& Filename)
{
	FString Extension;
	FString Name;
	FDatasmithUtils::GetCleanFilenameAndExtension(Filename, Name, Extension);
	auto ExtensionMatch = [&Extension](const FString& Format) { return Format.StartsWith(Extension); };
	return !Extension.IsEmpty() && Algo::FindByPredicate(Formats, ExtensionMatch) != nullptr;
}

void UDatasmithImportFactory::ValidateFilesForReimport(TArray<FString>& Filenames)
{
	FScopedLogger Logger(GetLoggerName(), GetDisplayName());

	TArray<FString> ValidFiles;

	for (const FString& SourceFilename : Filenames)
	{
		if (!SourceFilename.IsEmpty())
		{
			if (IFileManager::Get().FileSize(*SourceFilename) == INDEX_NONE)
			{
				FText Message = FText::Format(LOCTEXT("MissingSourceFile", "Could not find file \"{0}\" needed for reimport."), FText::FromString(SourceFilename));
				Logger.Push(EMessageSeverity::Warning, Message);
			}
			else
			{
				ValidFiles.Add(SourceFilename);
			}
		}
	}

	Filenames = ValidFiles;
}

UObject* UDatasmithImportFactory::FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, const FString& InFilename, const TCHAR* InParms, FFeedbackContext* InWarn, bool& bOutOperationCanceled)
{
	using namespace UE::DatasmithImporter;
	TRACE_CPUPROFILER_EVENT_SCOPE(UDatasmithImportFactory::FactoryCreateFile);

	const FSourceUri SourceUri = FSourceUri::FromFilePath(InFilename);
	TSharedPtr<FExternalSource> ExternalSource = IExternalSourceModule::GetOrCreateExternalSource(SourceUri);
	if ( !ExternalSource )
	{
		UE_LOGF(LogDatasmithImport, Warning, "Datasmith import error: no suitable external source found for this file path. Abort import.");
		return nullptr;
	}

	return CreateFromExternalSource(InClass, InParent, InName, InFlags, ExternalSource.ToSharedRef(), InParms, InWarn, bOutOperationCanceled);
}

UObject* UDatasmithImportFactory::CreateFromExternalSource(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, const TSharedRef<UE::DatasmithImporter::FExternalSource>& InExternalSource, const TCHAR* InParms, FFeedbackContext* InWarn, bool& bOutOperationCanceled)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UDatasmithImportFactory::CreateFromExternalSource);

	// Do not go any further if the user had canceled the import.
	// Happens when multiple files have been selected.
	if (bOperationCanceled)
	{
		bOutOperationCanceled = true;
		return nullptr;
	}

	FString PackageRoot;
	FString PackagePath;
	FString PackageName;

	FPackageName::SplitLongPackageName( InParent->GetName(), PackageRoot, PackagePath, PackageName );
	const FString ImportPath = PackageRoot / PackagePath;

	FDatasmithImportContext ImportContext(InExternalSource, !IsAutomatedImport(), GetLoggerName(), GetDisplayName());

	TFunction< UObject* ()> ExitReturn = [&InExternalSource, &ImportContext]() -> UObject*
		{
			if (InExternalSource->GetDatasmithScene())
			{
				if (TSharedPtr<IDatasmithTranslator> Translator = InExternalSource->GetAssetTranslator())
				{
					// Translators may be using global content. Make sure it is flushed before completing the import
					Translator->UnloadScene();
				}
			}

			return ImportContext.SceneAsset;
		};

	const bool bIsSilent = IsAutomatedImport() || !bShowOptions;
	if (!ImportContext.Init(ImportPath, InFlags, InWarn, ImportSettingsJson, bIsSilent))
	{
		bOperationCanceled = true;
		bOutOperationCanceled = true;
		return ExitReturn();
	}

	// Collect start time to log amount of time spent to import incoming file
	uint64 StartTime = FPlatformTime::Cycles64();

	if (TSharedPtr<IDatasmithScene> Scene = InExternalSource->TryLoad())
	{
		ImportContext.InitScene(Scene.ToSharedRef());
	}
	else
	{
		UE_LOGF(LogDatasmithImport, Warning, "Datasmith import error: Scene translation failure. Abort import.");
		return ExitReturn();
	}

	if (!Import( ImportContext ))
	{
		bOperationCanceled = true;
		bOutOperationCanceled = true;
		ImportContext.SceneAsset = nullptr;
		UE_LOGF(LogDatasmithImport, Warning, "Datasmith import error. Abort import.");
		return ExitReturn();
	}

	DatasmithImportFactoryImpl::ReportImportStats(ImportContext, StartTime);

	return ExitReturn();
}

bool UDatasmithImportFactory::Import( FDatasmithImportContext& ImportContext )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UDatasmithImportFactory::Import);

	// Avoid showing options if user asked to use same options for all files
	if (ImportContext.Options->bUseSameOptions)
	{
		bShowOptions = false;
	}

	FLayoutUV::ResetStats();

	if ( !ImportContext.bIsAReimport )
	{
		GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, UDatasmithScene::StaticClass(), nullptr, ImportContext.Scene->GetName(), nullptr);
	}
	else
	{
		GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetReimport(ImportContext.SceneAsset);
	}

	bool bOutOperationCanceled = false;
	const bool bImportSuccess = DatasmithImportFactoryImpl::ImportDatasmithScene( ImportContext, bOutOperationCanceled );

	GEditor->RedrawAllViewports();

	if ( !IsAutomatedImport() )
	{
		ImportContext.DisplayMessages();

		if ( bImportSuccess && !bOutOperationCanceled && !ImportContext.bIsAReimport )
		{
			IDatasmithContentEditorModule& ContentEditorModule = IDatasmithContentEditorModule::Get();
			const bool bIsAutoReimportAvailable = ContentEditorModule.IsAssetAutoReimportAvailable( ImportContext.SceneAsset ).Get( false );

			if ( bIsAutoReimportAvailable )
			{
				ContentEditorModule.SetAssetAutoReimport( ImportContext.SceneAsset, true );
			}
		}
	}

	if ( bOutOperationCanceled )
	{
		return false;
	}

	return true;
}

void UDatasmithImportFactory::ParseFromJson(TSharedRef<FJsonObject> InImportSettingsJson)
{
	ImportSettingsJson = InImportSettingsJson;
}

bool UDatasmithImportFactory::CanReimport( UObject* Obj, TArray<FString>& OutFilenames )
{
	// The CDO may be used to do that check. In that case, Formats are not necessarily initialized.
	if (Formats.IsEmpty())
	{
		Formats = FDatasmithTranslatorManager::Get().GetSupportedFormats();
	}

	UAssetImportData* ReImportData = DatasmithImportFactoryImpl::GetImportData(Obj);
	if (ReImportData == nullptr)
	{
		return false;
	}

	// Importers are only aware of one source file (the first one)
	const FAssetImportInfo::FSourceFile* FirstFileInfo = ReImportData->GetSourceData().SourceFiles.GetData();
	bool bHasSource = FirstFileInfo && !FirstFileInfo->RelativeFilename.IsEmpty();

	FScopedLogger Logger(GetLoggerName(), GetDisplayName());
	Logger.ClearLog();
	if (!bHasSource)
	{
		FText Message = FText::Format(LOCTEXT("MissingSourceFileInfo", "Missing source file information for reimport of asset \"{0}\"."), FText::FromString(ReImportData->GetFullGroupName(true)));
		Logger.Push(EMessageSeverity::Warning, Message);
		return false;
	}

	if (!FactoryCanImport(FirstFileInfo->RelativeFilename))
	{
		return false;
	}

	OutFilenames.Empty();
	ReImportData->ExtractFilenames(OutFilenames);
	ValidateFilesForReimport(OutFilenames);

	// Need to return false if there's no valid source file for StaticMesh to skip the file selection dialog normally shown for missing source file
	return OutFilenames.Num() > 0;
}

void UDatasmithImportFactory::SetReimportPaths( UObject* Obj, const TArray<FString>& NewReimportPaths )
{
	using namespace UE::DatasmithImporter;
	ensure(NewReimportPaths.Num() == 1);

	if (UAssetImportData* ReImportData = DatasmithImportFactoryImpl::GetImportData(Obj))
	{
		const FString PreviousImportPath = ReImportData->GetFirstFilename();
		ReImportData->UpdateFilenameOnly(NewReimportPaths[0]);

		FDatasmithImportInfo* DatasmithImportInfo = nullptr;
		if (UDatasmithAssetImportData* DatasmithAssetReImportData = Cast<UDatasmithAssetImportData>(ReImportData))
		{
			DatasmithImportInfo = &DatasmithAssetReImportData->DatasmithImportInfo;
		}
		else if (UDatasmithSceneImportData* DatasmithSceneReImportData = Cast<UDatasmithSceneImportData>(ReImportData))
		{
			DatasmithImportInfo = &DatasmithSceneReImportData->DatasmithImportInfo;
		}

		if (DatasmithImportInfo)
		{
			const bool bWasFileSource = FSourceUri(DatasmithImportInfo->SourceUri).GetScheme() == FSourceUri::GetFileScheme();
			const bool bChangedFilename = FPaths::ConvertRelativePathToFull(PreviousImportPath) != FPaths::ConvertRelativePathToFull(NewReimportPaths[0]);

			// Workaround to avoid clearing the import info on sources (directlink) that were not imported from a file.
			// #ueent_todo The ReimportFromNewFile command should be replaced by a ReimportFromNewSource, adding support for non-file sources.
			if (bWasFileSource || bChangedFilename)
			{
				const FString SourceUriString = FSourceUri::FromFilePath(NewReimportPaths[0]).ToString();
				*DatasmithImportInfo = FDatasmithImportInfo(SourceUriString);
			}
		}
	}
}

void UDatasmithImportFactory::OnObjectReimported(UObject* Object, UStaticMesh* StaticMesh)
{
	UStaticMesh* ImportedStaticMesh = Cast<UStaticMesh>(Object);
	if (StaticMesh != nullptr && ImportedStaticMesh == StaticMesh)
	{
		// Unregister since there is no need to listen anymore
		GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetReimport.RemoveAll(this);

		// Open static mesh editor on newly imported mesh
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset( StaticMesh );
	}
}

EReimportResult::Type UDatasmithImportFactory::ReimportStaticMesh(UStaticMesh* Mesh)
{
	using namespace UE::DatasmithImporter;
	UDatasmithStaticMeshImportData* MeshImportData = UDatasmithStaticMeshImportData::GetImportDataForStaticMesh(Mesh);

	if (MeshImportData == nullptr || MeshImportData->AssetImportOptions.PackagePath.IsNone())
	{
		return EReimportResult::Failed;
	}

	TSharedPtr<FExternalSource> ExternalSource = IExternalSourceModule::Get().GetManager().TryGetExternalSourceFromImportData(*MeshImportData);
	if (!ExternalSource)
	{
		UE_LOGF(LogDatasmithImport, Warning, "Datasmith ReimportStaticMesh error: cannot resolve the external source from Source file or URI. Abort import");
		bOperationCanceled = true;
		return EReimportResult::Failed;
	}

	const FString ImportPath = MeshImportData->AssetImportOptions.PackagePath.ToString();
	FDatasmithImportContext ImportContext(ExternalSource.ToSharedRef(), false, GetLoggerName(), GetDisplayName());

	// Restore static mesh options stored in mesh import data
	ImportContext.Options->BaseOptions.StaticMeshOptions = MeshImportData->ImportOptions;
	ImportContext.Options->BaseOptions.AssetOptions = MeshImportData->AssetImportOptions;

	ImportContext.SceneAsset = FDatasmithImporterUtils::FindDatasmithSceneForAsset( Mesh );
	if (ImportContext.SceneAsset == nullptr)
	{
		bOperationCanceled = true;
		UE_LOGF(LogDatasmithImport, Warning, "Datasmith ReimportStaticMesh error: no UDatasmithScene associated with asset %ls. Aborting reimport.", *Mesh->GetName());
		return EReimportResult::Failed;
	}

	// Restore additional import options
	UDatasmithTranslatedSceneImportData* SceneAssetImportData = ExactCast<UDatasmithTranslatedSceneImportData>(ImportContext.SceneAsset->AssetImportData);
	if (SceneAssetImportData)
	{
		ImportContext.AdditionalImportOptions.Empty();
		for (UDatasmithOptionsBase* AdditionalOption : SceneAssetImportData->AdditionalOptions)
		{
			ImportContext.AdditionalImportOptions.Emplace(AdditionalOption);
		}
	}

	const bool bIsSilent = true;
	if (!ImportContext.Init(ImportPath, Mesh->GetFlags(), GWarn, ImportSettingsJson, bIsSilent))
	{
		return EReimportResult::Cancelled;
	}

	if ( TSharedPtr<IDatasmithScene> LoadedScene = ExternalSource->TryLoad() )
	{
		ImportContext.InitScene( LoadedScene.ToSharedRef() );
	}
	else
	{
		bOperationCanceled = true;
		UE_LOGF(LogDatasmithImport, Warning, "Datasmith ReimportStaticMesh error: Scene translation failure. Abort import.");
		return EReimportResult::Failed;
	}

	UDatasmithContentBlueprintLibrary* DatasmithContentLibrary = Cast< UDatasmithContentBlueprintLibrary >( UDatasmithContentBlueprintLibrary::StaticClass()->GetDefaultObject() );
	FString StaticMeshUniqueId = FDatasmithImporterUtils::GetDatasmithElementIdString( Mesh );

	TSharedPtr< IDatasmithMeshElement > MeshElement;

	for (int32 MeshElementIndex = 0; MeshElementIndex < ImportContext.Scene->GetMeshesCount(); ++MeshElementIndex)
	{
		TSharedPtr< IDatasmithMeshElement > SceneMeshElement = ImportContext.Scene->GetMesh( MeshElementIndex );
		if ( SceneMeshElement->GetName() == StaticMeshUniqueId )
		{
			MeshElement = SceneMeshElement;
			break;
		}
	}

	if ( !MeshElement.IsValid() )
	{
		UE_LOGF(LogDatasmithImport, Warning, "Datasmith ReimportStaticMesh error: mesh not found in imported scene. Abort import.");
		return EReimportResult::Failed;
	}

	// Close the mesh editor to prevent crashing. Reopen it later if necessary.
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	IAssetEditorInstance* EditorInstance = AssetEditorSubsystem->FindEditorForAsset(Mesh, false);
	if (EditorInstance)
	{
		AssetEditorSubsystem->CloseAllEditorsForAsset(Mesh);
	}

	// Updates package paths to the content folder of the mesh's original imported scene
	// Necessary as we will search those for materials, textures and will use its Temp folder
	FString GeometryPackagePath = FPaths::GetPath(Mesh->GetOuter()->GetName());
	FString OldRootFolder = FPaths::GetPath(GeometryPackagePath);
	ImportContext.AssetsContext.ReInit(OldRootFolder);

	// We're not reimporting level sequences, materials or textures so their import packages must point to the real package
	ImportContext.AssetsContext.LevelSequencesImportPackage.Reset();
	ImportContext.AssetsContext.LevelVariantSetsImportPackage.Reset();
	ImportContext.AssetsContext.MaterialsImportPackage.Reset();
	ImportContext.AssetsContext.ReferenceMaterialsImportPackage.Reset();
	ImportContext.AssetsContext.MaterialFunctionsImportPackage.Reset();
	ImportContext.AssetsContext.TexturesImportPackage.Reset();

	UStaticMesh* ImportedStaticMesh = FDatasmithImporter::ImportStaticMesh( ImportContext, MeshElement.ToSharedRef(), Mesh );

	if ( !ImportedStaticMesh || ImportContext.ImportedStaticMeshes.Num() == 0 )
	{
		UE_LOGF(LogDatasmithImport, Warning, "Datasmith ReimportStaticMesh error: other. Abort import.");
		return EReimportResult::Failed;
	}

	TMap< TSharedRef< IDatasmithMeshElement >, float > LightmapWeights = FDatasmithStaticMeshImporter::CalculateMeshesLightmapWeights( ImportContext.Scene.ToSharedRef() );

	IMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked< IMeshUtilities >("MeshUtilities");

	FDatasmithStaticMeshImporter::SetupStaticMesh( ImportContext.AssetsContext, MeshElement.ToSharedRef(), ImportedStaticMesh, ImportContext.Options->BaseOptions.StaticMeshOptions, LightmapWeights[ MeshElement.ToSharedRef() ] );

	bool bIsMeshValid = FDatasmithStaticMeshImporter::PreBuildStaticMesh( ImportedStaticMesh );

	if ( bIsMeshValid )
	{
		FDatasmithImporter::FinalizeStaticMesh( ImportedStaticMesh, *Mesh->GetOutermost()->GetName(), Mesh );
	}

	if ( EditorInstance )
	{
		// Register to be notified when re-import is completed.
		GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetReimport.AddUObject( this, &UDatasmithImportFactory::OnObjectReimported, Mesh );
	}

	return EReimportResult::Succeeded;
}

EReimportResult::Type UDatasmithImportFactory::ReimportScene(UDatasmithScene* SceneAsset)
{
	using namespace UE::DatasmithImporter;
	// #ueent_todo: unify with import, BP, python, DP.
	if (!SceneAsset || !SceneAsset->AssetImportData)
	{
		return EReimportResult::Failed;
	}

	UDatasmithSceneImportData& AssetImportData = *SceneAsset->AssetImportData;

	TSharedPtr<FExternalSource> ExternalSource = IExternalSourceModule::Get().GetManager().TryGetExternalSourceFromImportData(AssetImportData);
	if (!ExternalSource)
	{
		UE_LOGF(LogDatasmithImport, Warning, "Datasmith ReimportScene error: Import source is not available. Abort import.");
		return EReimportResult::Failed;
	}

	ExternalSource->SetSceneName(*SceneAsset->GetName()); // keep initial name

	// Setup pipe for reimport
	const bool bLoadConfig = false;
	const FString ImportPath = AssetImportData.BaseOptions.AssetOptions.PackagePath.ToString();
	FDatasmithImportContext ImportContext(ExternalSource.ToSharedRef(), bLoadConfig, GetLoggerName(), GetDisplayName());
	ImportContext.SceneAsset = SceneAsset;
	ImportContext.Options->BaseOptions = AssetImportData.BaseOptions; // Restore options as used in original import
	if (UDatasmithTranslatedSceneImportData* TranslatedSceneReimportData = Cast<UDatasmithTranslatedSceneImportData>(SceneAsset->AssetImportData))
	{
		for (UDatasmithOptionsBase* Option : TranslatedSceneReimportData->AdditionalOptions)
		{
			ImportContext.UpdateImportOption(Option);
		}
	}
	ImportContext.bIsAReimport = true;


	const bool bIsSilent = IsAutomatedImport() || IsAutomatedReimport();
	if (!ImportContext.Init(ImportPath, ImportContext.SceneAsset->GetFlags(), GWarn, ImportSettingsJson, bIsSilent))
	{
		return EReimportResult::Cancelled;
	}

	// Collect start time to log amount of time spent to import incoming file
	uint64 StartTime = FPlatformTime::Cycles64();

	if (TSharedPtr<IDatasmithScene> LoadedScene = ExternalSource->TryLoad())
	{
		ImportContext.InitScene(LoadedScene.ToSharedRef());
	}
	else
	{
		UE_LOGF(LogDatasmithImport, Warning, "Datasmith import error: Scene translation failure. Abort import.");
		return EReimportResult::Failed;
	}

	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllAssetEditors();

	if (!Import( ImportContext ))
	{
		UE_LOGF(LogDatasmithImport, Warning, "Datasmith import error. Abort import.");
		return EReimportResult::Failed;
	}

	DatasmithImportFactoryImpl::ReportImportStats(ImportContext, StartTime);

	// Copy over the changes the user may have done on the options
	if (!SceneAsset->AssetImportData)
	{
		UE_LOGF(LogDatasmithImport, Warning, "Datasmith import error: Missing scene asset import data. Abort import.");
		return EReimportResult::Failed;
	}

	UDatasmithSceneImportData& NewReimportData = *SceneAsset->AssetImportData;
	NewReimportData.BaseOptions = ImportContext.Options->BaseOptions;

	NewReimportData.Modify();
	NewReimportData.PostEditChange();
	NewReimportData.MarkPackageDirty();

	return EReimportResult::Succeeded;
}

EReimportResult::Type UDatasmithImportFactory::ReimportMaterial( UMaterialInterface* Material )
{
	using namespace UE::DatasmithImporter;
	UDatasmithAssetImportData* MaterialImportData = Material ? Cast< UDatasmithAssetImportData >( Material->AssetImportData ) : nullptr;
	if ( MaterialImportData == nullptr )
	{
		UE_LOGF(LogDatasmithImport, Warning, "Datasmith ReimportMaterial error: missing import data.");
		return EReimportResult::Failed;
	}

	FString ImportPath = MaterialImportData->AssetImportOptions.PackagePath.ToString();
	if ( ImportPath.IsEmpty() )
	{
		UE_LOGF(LogDatasmithImport, Warning, "Datasmith ReimportMaterial error: missing import path.");
		return EReimportResult::Failed;
	}

	TSharedPtr<FExternalSource> ExternalSource = IExternalSourceModule::Get().GetManager().TryGetExternalSourceFromImportData(*MaterialImportData);
	if (!ExternalSource)
	{
		UE_LOGF(LogDatasmithImport, Warning, "Datasmith ReimportMaterial error: cannot resolve the external source from Source file or URI. Abort import");
		return EReimportResult::Failed;
	}

	// Reopen the material editor if it was opened for this material. Note that this will close all the tabs, even the ones for other materials.
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	IAssetEditorInstance* EditorInstance = AssetEditorSubsystem->FindEditorForAsset(Material, false);
	if (EditorInstance)
	{
		AssetEditorSubsystem->CloseAllEditorsForAsset(Material);
	}

	FDatasmithImportContext ImportContext(ExternalSource.ToSharedRef(), false, GetLoggerName(), GetDisplayName());

	ImportContext.Options->BaseOptions.AssetOptions = MaterialImportData->AssetImportOptions;

	ImportContext.SceneAsset = FDatasmithImporterUtils::FindDatasmithSceneForAsset( Material );

	const bool bIsSilent = true;
	if (!ImportContext.Init(ImportPath, Material->GetFlags(), GWarn, ImportSettingsJson, bIsSilent))
	{
		return EReimportResult::Cancelled;
	}

	if (TSharedPtr<IDatasmithScene> LoadedScene = ExternalSource->TryLoad())
	{
		ImportContext.InitScene(LoadedScene.ToSharedRef());
	}
	else
	{
		UE_LOGF(LogDatasmithImport, Warning, "Datasmith ReimportMaterial error: Scene load failure. Abort import.");
		return EReimportResult::Failed;
	}

	// We're not reimporting level sequences, static meshes or textures so clear their import packages
	ImportContext.AssetsContext.LevelSequencesImportPackage.Reset();
	ImportContext.AssetsContext.LevelVariantSetsImportPackage.Reset();
	ImportContext.AssetsContext.StaticMeshesImportPackage.Reset();
	ImportContext.AssetsContext.TexturesImportPackage.Reset();

	TSharedPtr< IDatasmithBaseMaterialElement > MaterialElement;
	const FString MaterialUniqueId = FDatasmithImporterUtils::GetDatasmithElementIdString(Material);

	for ( int32 MaterialElementIndex = 0; MaterialElementIndex < ImportContext.Scene->GetMaterialsCount(); ++MaterialElementIndex )
	{
		if (TSharedPtr< IDatasmithBaseMaterialElement > CandidateElement = ImportContext.Scene->GetMaterial( MaterialElementIndex ))
		{
			if ( CandidateElement->GetName() == MaterialUniqueId )
			{
				MaterialElement = CandidateElement;
				break;
			}
		}
	}

	if ( !MaterialElement.IsValid() )
	{
		UE_LOGF(LogDatasmithImport, Warning, "Datasmith ReimportMaterial error: Material '%ls' not found in the source", *Material->GetName());
		return EReimportResult::Failed;
	}

	FString MaterialPath = FPaths::GetPath(FSoftObjectPath(Material).GetLongPackageName()); // FPaths::GetPath(Material->GetOutermost()->GetName());

	Material->PreEditChange( nullptr );

	FDatasmithImporter::ImportMaterial( ImportContext, MaterialElement.ToSharedRef(), Material );

	const FString& RootFolderPath = ImportContext.AssetsContext.RootFolderPath;
	const FString& TransientFolderPath = ImportContext.AssetsContext.TransientFolderPath;

	UMaterialInterface* NewMaterial = ImportContext.ImportedMaterials.FindRef( MaterialElement.ToSharedRef() );
	FDatasmithImporter::FinalizeMaterial( NewMaterial, *MaterialPath, *TransientFolderPath, *RootFolderPath, Material );

	FGlobalComponentReregisterContext RecreateComponents;

	if ( EditorInstance )
	{
		AssetEditorSubsystem->OpenEditorForAsset( Material );
	}

	return EReimportResult::Succeeded;
}

EReimportResult::Type UDatasmithImportFactory::Reimport( UObject* Obj )
{
	EReimportResult::Type Result = EReimportResult::Failed;

	if ( UStaticMesh* Mesh = ExactCast< UStaticMesh >(Obj) )
	{
		Result = ReimportStaticMesh( Mesh );
	}
	else if ( UMaterialInterface* Material = Cast< UMaterialInterface >( Obj ) ) // We support UMaterialInterface (UMaterial and UMaterialInstance)
	{
		Result = ReimportMaterial( Material );
	}
	else if ( UDatasmithScene* SceneAsset = ExactCast< UDatasmithScene >(Obj) )
	{
		Result = ReimportScene( SceneAsset );
	}

	CleanUp();

	return Result;
}

int32 UDatasmithImportFactory::GetPriority() const
{
	return ImportPriority;
}

#undef LOCTEXT_NAMESPACE
