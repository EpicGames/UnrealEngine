// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithImporter.h"

#include "DatasmithActorImporter.h"
#include "DatasmithAdditionalData.h"
#include "DatasmithAnimationElements.h"
#include "DatasmithAssetImportData.h"
#include "DatasmithAssetUserData.h"
#include "DatasmithCameraImporter.h"
#include "DatasmithCloth.h"  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
#include "DatasmithImportContext.h"
#include "DatasmithImporterModule.h"
#include "DatasmithLevelSequenceImporter.h"
#include "DatasmithLevelVariantSetsImporter.h"
#include "DatasmithLightImporter.h"
#include "DatasmithMaterialExpressions.h"
#include "DatasmithMaterialImporter.h"
#include "DatasmithPayload.h"
#include "DatasmithPostProcessImporter.h"
#include "DatasmithScene.h"
#include "DatasmithSceneActor.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithStaticMeshImporter.h"
#include "DatasmithTextureImporter.h"
#include "DatasmithTranslator.h"
#include "DatasmithUtils.h"
#include "Engine/Texture2D.h"
#include "IDatasmithSceneElements.h"
#include "LevelVariantSets.h"
#include "ObjectTemplates/DatasmithObjectTemplate.h"
#include "Utility/DatasmithImporterImpl.h"
#include "Utility/DatasmithImporterUtils.h"
#include "Utility/DatasmithTextureResize.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "CineCameraComponent.h"
#include "ComponentReregisterContext.h"
#include "Components/HierarchicalInstancedStaticMeshComponent.h"
#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorLevelUtils.h"
#include "Engine/Engine.h"
#include "Engine/RendererSettings.h"
#include "Engine/SkinnedAsset.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshActor.h"
#include "Features/IModularFeatures.h"
#include "HAL/FileManager.h"
#include "IAssetTools.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Landscape.h"
#include "Layers/LayersSubsystem.h"
#include "LevelSequence.h"
#include "MaterialEditingLibrary.h"
#include "MaterialShared.h"
#include "Materials/Material.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialInstanceConstant.h"
#include "MaterialCachedData.h"
#include "Misc/FeedbackContext.h"
#include "Misc/FileHelper.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/UObjectToken.h"
#include "ObjectTools.h"
#include "PackageTools.h"
#include "Serialization/ArchiveReplaceObjectRef.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/ObjectReader.h"
#include "Serialization/ObjectWriter.h"
#include "Settings/EditorExperimentalSettings.h"
#include "SkinnedAssetCompiler.h"
#include "SourceControlOperations.h"
#include "Tasks/Task.h"
#include "Templates/UniquePtr.h"
#include "UObject/Package.h"
#include "UObject/UObjectHash.h"
#include "UnrealEdGlobals.h"

extern UNREALED_API UEditorEngine* GEditor;

#define LOCTEXT_NAMESPACE "DatasmithImporter"

namespace UE::DatasmithImporter::Private
{
	/**
	 * Make the necessary adjustment to the materials so that the functions have unique parameter names in the material where they are use.
	 */
	void MakeChangeToExpressionsNameForMaterialFunctions(const TArray<FDatasmithImporterUtils::FFunctionAndMaterialsThatUseIt>& FunctionMaterialAndTheirReferencer)
	{
		// Contains the name of the parameters for a material that isn't use as a function and the set of names used the by material functions it call
		TMap<TSharedPtr<IDatasmithUEPbrMaterialElement>,TArray<TSharedRef<TSet<TPair<FName, EDatasmithMaterialExpressionType>>>>> TopMaterialAndNamesUsedByType;
		TMap<EDatasmithMaterialExpressionType, TUniquePtr<FDatasmithUniqueNameProvider>> ExpressionTypeToUniqueNameProvider;
		using FFunctionAndMaterialsThatUseIt = FDatasmithImporterUtils::FFunctionAndMaterialsThatUseIt;

		TMap<TSharedPtr<IDatasmithUEPbrMaterialElement>, int32> MaterialToIndexInArray;
		MaterialToIndexInArray.Reserve( FunctionMaterialAndTheirReferencer.Num() );

		// Check if the name was already used in this function or in one of its referencers
		TFunction<bool (uint32, const TSharedPtr<IDatasmithUEPbrMaterialElement>&, uint32, FName, EDatasmithMaterialExpressionType)> CheckIfNameIsUsed;
		CheckIfNameIsUsed = [&TopMaterialAndNamesUsedByType, &MaterialToIndexInArray, &FunctionMaterialAndTheirReferencer, &CheckIfNameIsUsed] (uint32 MaterialHash, const TSharedPtr<IDatasmithUEPbrMaterialElement>& Material, uint32 ExpressionHash, FName Expression, EDatasmithMaterialExpressionType ExpressionType)
			{
				if ( int32* Index = MaterialToIndexInArray.FindByHash( MaterialHash, Material ) )
				{
					for ( const TSharedPtr<IDatasmithUEPbrMaterialElement>& Referencer : FunctionMaterialAndTheirReferencer[*Index].Value )
					{
						if ( CheckIfNameIsUsed( GetTypeHash( Referencer ), Referencer, ExpressionHash, Expression, ExpressionType ) )
						{
							return true;
						}
					}
				}
				else if ( const TArray<TSharedRef<TSet<TPair<FName, EDatasmithMaterialExpressionType>>>>* NameUsedArray = TopMaterialAndNamesUsedByType.FindByHash( MaterialHash, Material ) )
				// Only check if we are in a material that aren't referenced elsewhere
				{
					for ( const TSharedRef<TSet<TPair<FName, EDatasmithMaterialExpressionType>>>& NamesUsed : *NameUsedArray )
					{
						if ( NamesUsed->Contains( TPair<FName, EDatasmithMaterialExpressionType>( Expression, ExpressionType ) ) )
						{
							return true;
						}
					}
				}

				return false;
			};

		// Make sure that the parameter have a unique name in the materials were they are use
		auto HandleExpressionNameForMaterialFunction =
			[&ExpressionTypeToUniqueNameProvider, &CheckIfNameIsUsed] (uint32 MaterialHash, const TSharedPtr<IDatasmithUEPbrMaterialElement>& Material, FName Expression, EDatasmithMaterialExpressionType ExpressionType, int32 ExpressionIndex, TSet<TPair<FName, EDatasmithMaterialExpressionType>>& NameUsed)
			{
				TUniquePtr<FDatasmithUniqueNameProvider>& UniqueNameProvider = ExpressionTypeToUniqueNameProvider.FindOrAdd( ExpressionType );
				if ( !UniqueNameProvider )
				{
					UniqueNameProvider = MakeUnique<FDatasmithUniqueNameProvider>();
				}

				FString ChosenName;

				if ( CheckIfNameIsUsed( MaterialHash, Material, GetTypeHash( Expression ), Expression, ExpressionType ) )
				{
					// Make the name unique
					ChosenName = UniqueNameProvider->GenerateUniqueName( Expression.ToString() );
					Material->GetExpression( ExpressionIndex )->SetName( *ChosenName );
				}
				else
				{
					// Keep track of the used names
					ChosenName = Expression.ToString();
					UniqueNameProvider->AddExistingName( ChosenName );
				}

				NameUsed.Add( TPair<FName, EDatasmithMaterialExpressionType>( *ChosenName, ExpressionType ) );

				if ( ExpressionType != EDatasmithMaterialExpressionType::Generic )
				{
					// Always add to the generic, we don't know what type of parameter a generic expression would be
					TUniquePtr<FDatasmithUniqueNameProvider>& GenericUniqueNameProvider = ExpressionTypeToUniqueNameProvider.FindOrAdd( EDatasmithMaterialExpressionType::Generic );
					if ( !GenericUniqueNameProvider )
					{
						GenericUniqueNameProvider = MakeUnique<FDatasmithUniqueNameProvider>();
					}

					GenericUniqueNameProvider->AddExistingName( ChosenName );
					NameUsed.Add( TPair<FName, EDatasmithMaterialExpressionType>( *ChosenName, EDatasmithMaterialExpressionType::Generic ) );
				}
			};

		TFunction<void (const TSharedRef<TSet<TPair<FName, EDatasmithMaterialExpressionType>>>&, uint32, const TSharedPtr<IDatasmithUEPbrMaterialElement>&)> PushUsedNamesIntoTopLevelMaterial;
		PushUsedNamesIntoTopLevelMaterial = [&MaterialToIndexInArray, &TopMaterialAndNamesUsedByType, &FunctionMaterialAndTheirReferencer, &PushUsedNamesIntoTopLevelMaterial] (const TSharedRef<TSet<TPair<FName, EDatasmithMaterialExpressionType>>>& UsedNames, uint32 MaterialHash, const TSharedPtr<IDatasmithUEPbrMaterialElement>& Material)
			{
				if ( int32* Index = MaterialToIndexInArray.FindByHash( MaterialHash, Material ) )
				{
					for ( const TSharedPtr<IDatasmithUEPbrMaterialElement>& Referencer : FunctionMaterialAndTheirReferencer[*Index].Value )
					{
						PushUsedNamesIntoTopLevelMaterial( UsedNames, GetTypeHash( Referencer ), Referencer );
					}
				}
				else
				{
					TopMaterialAndNamesUsedByType.FindOrAddByHash( MaterialHash, Material ).Add( UsedNames );
				}
			};


		TSet<TSharedPtr<IDatasmithUEPbrMaterialElement>> VisitedMaterials;
		// Plus one because there is at least one top material that isn't used as a function
		VisitedMaterials.Reserve( FunctionMaterialAndTheirReferencer.Num() + 1 );

		/**
			* We expect the independents function to be at the start of the array
			* We want the top level function to conserve their name in case of conflicted naming of the parameters.
			*/
		for ( int32 Index = FunctionMaterialAndTheirReferencer.Num()-1; Index >= 0; Index-- )
		{
			const FFunctionAndMaterialsThatUseIt& FunctionAndReferencer = FunctionMaterialAndTheirReferencer[Index];
			const TSharedPtr<IDatasmithUEPbrMaterialElement>& CurrentElement = FunctionAndReferencer.Key;
			uint32 CurrentElementHash = GetTypeHash( CurrentElement );
			MaterialToIndexInArray.AddByHash( CurrentElementHash, CurrentElement, Index );

			// Populate the name used by the Referencers
			for ( const TSharedPtr<IDatasmithUEPbrMaterialElement>& Referencer : FunctionAndReferencer.Value )
			{
				uint32  ReferencerHash = GetTypeHash( Referencer );

				if ( !VisitedMaterials.ContainsByHash( ReferencerHash, Referencer ) )
				{
					TSharedRef<TSet<TPair<FName, EDatasmithMaterialExpressionType>>> UsedNames = MakeShared<TSet<TPair<FName, EDatasmithMaterialExpressionType>>>();

					FDatasmithMaterialExpressions::ForEachParamsNameInMaterial( Referencer,
						[ReferencerHash, &Referencer, &HandleExpressionNameForMaterialFunction, &UsedNames] (FName Expression, const EDatasmithMaterialExpressionType& ExpressionType, int32 Index)
						{
							HandleExpressionNameForMaterialFunction( ReferencerHash, Referencer, Expression, ExpressionType, Index, UsedNames.Get() );
						});
					VisitedMaterials.AddByHash( ReferencerHash, Referencer );
					PushUsedNamesIntoTopLevelMaterial( UsedNames, ReferencerHash, Referencer );
				}
			}

			// For the current element
			TSharedRef<TSet<TPair<FName, EDatasmithMaterialExpressionType>>> UsedNames = MakeShared<TSet<TPair<FName, EDatasmithMaterialExpressionType>>>();

			TFunction<void(FName Expression, const EDatasmithMaterialExpressionType& ExpressionType, int32 Index)> ForEachParameter =
				[CurrentElementHash, &CurrentElement, &UsedNames, &HandleExpressionNameForMaterialFunction](FName Expression, const EDatasmithMaterialExpressionType& ExpressionType, int32 Index)
				{
					HandleExpressionNameForMaterialFunction( CurrentElementHash, CurrentElement, Expression, ExpressionType, Index, UsedNames.Get() );
				};

			FDatasmithMaterialExpressions::ForEachParamsNameInMaterial( CurrentElement,
				[CurrentElementHash, &CurrentElement, &UsedNames, &HandleExpressionNameForMaterialFunction] (FName Expression, const EDatasmithMaterialExpressionType& ExpressionType, int32 Index)
				{
					HandleExpressionNameForMaterialFunction( CurrentElementHash, CurrentElement, Expression, ExpressionType, Index, UsedNames.Get() );
				});
			VisitedMaterials.AddByHash( CurrentElementHash, CurrentElement );
			PushUsedNamesIntoTopLevelMaterial( UsedNames, CurrentElementHash, CurrentElement );
		}
	}
} // namespace UE::DatasmithImporter::Private

void FDatasmithImporter::ImportStaticMeshes( FDatasmithImportContext& ImportContext )
{
	const int32 StaticMeshesCount = ImportContext.FilteredScene->GetMeshesCount();

	if ( !ImportContext.Options->BaseOptions.bIncludeGeometry || StaticMeshesCount == 0 )
	{
		return;
	}

	if (!ImportContext.AssetsContext.StaticMeshesFinalPackage || ImportContext.AssetsContext.StaticMeshesFinalPackage->GetFName() == NAME_None || ImportContext.SceneTranslator == nullptr)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithImporter::ImportStaticMeshes);

	TUniquePtr<FScopedSlowTask> ProgressPtr;

	if ( ImportContext.FeedbackContext )
	{
		ProgressPtr = MakeUnique<FScopedSlowTask>(StaticMeshesCount, LOCTEXT("ImportStaticMeshes", "Importing Static Meshes..."), true, *ImportContext.FeedbackContext );
		ProgressPtr->MakeDialog(true);
	}

	TMap<TSharedRef<IDatasmithMeshElement>, UE::Tasks::TTask<FDatasmithMeshElementPayload*>> MeshElementPayloads;

	FDatasmithTranslatorCapabilities TranslatorCapabilities;
	ImportContext.SceneTranslator->Initialize(TranslatorCapabilities);

	// Parallelize loading by doing a first pass to send translator loading into async task
	if (TranslatorCapabilities.bParallelLoadStaticMeshSupported)
	{
		for (int32 MeshIndex = 0; MeshIndex < StaticMeshesCount && !ImportContext.bUserCancelled; ++MeshIndex)
		{
			if (FDatasmithImporterImpl::HasUserCancelledTask(ImportContext.FeedbackContext))
			{
				ImportContext.bUserCancelled = true;
			}

			TSharedRef<IDatasmithMeshElement> MeshElement = ImportContext.FilteredScene->GetMesh( MeshIndex ).ToSharedRef();

			auto& ImportedStaticMesh = ImportContext.ImportedStaticMeshes.FindOrAdd( MeshElement );

			// We still have factories that are importing the UStaticMesh on their own, so check if it's already imported here
			if (ImportedStaticMesh == nullptr)
			{
				// Parallel loading from the translator using futures
				MeshElementPayloads.Add(
					MeshElement,
					UE::Tasks::Launch(UE_SOURCE_LOCATION,
						[&ImportContext, MeshElement]() -> FDatasmithMeshElementPayload*
						{
							if (ImportContext.bUserCancelled)
							{
								return nullptr;
							}

							TRACE_CPUPROFILER_EVENT_SCOPE(LoadStaticMesh);
							TUniquePtr<FDatasmithMeshElementPayload> MeshPayload = MakeUnique<FDatasmithMeshElementPayload>();
							return ImportContext.SceneTranslator->LoadStaticMesh(MeshElement, *MeshPayload) ? MeshPayload.Release() : nullptr;
						}
					)
				);
			}
		}
	}

	FScopedSlowTask* Progress = ProgressPtr.Get();

	// This pass will wait on the futures we got from the first pass async tasks
	for ( int32 MeshIndex = 0; MeshIndex < StaticMeshesCount && !ImportContext.bUserCancelled; ++MeshIndex )
	{
		if (FDatasmithImporterImpl::HasUserCancelledTask(ImportContext.FeedbackContext))
		{
			ImportContext.bUserCancelled = true;
		}

		TSharedRef< IDatasmithMeshElement > MeshElement = ImportContext.FilteredScene->GetMesh( MeshIndex ).ToSharedRef();

		FDatasmithImporterImpl::ReportProgress( Progress, 1.f, FText::FromString( FString::Printf( TEXT("Importing static mesh %d/%d (%s) ..."), MeshIndex + 1, StaticMeshesCount, MeshElement->GetLabel() ) ) );

		UStaticMesh* ExistingStaticMesh = nullptr;

		if (ImportContext.SceneAsset)
		{
			TSoftObjectPtr< UStaticMesh >* ExistingStaticMeshPtr = ImportContext.SceneAsset->StaticMeshes.Find( MeshElement->GetName() );

			if (ExistingStaticMeshPtr)
			{
				ExistingStaticMesh = ExistingStaticMeshPtr->LoadSynchronous();
			}
		}

		// #ueent_todo rewrite in N passes:
		//  - GetDestination (find or create StaticMesh, duplicate, flags and context etc)
		//  - Import (Import data in simple memory repr (eg. TArray<FMeshDescription>)
		//  - Set (fill UStaticMesh with imported data)
		UE::Tasks::TTask<FDatasmithMeshElementPayload*> MeshPayload;
		if (MeshElementPayloads.RemoveAndCopyValue(MeshElement, MeshPayload))
		{
			TUniquePtr<FDatasmithMeshElementPayload> MeshPayloadPtr(MeshPayload.GetResult());
			if (MeshPayloadPtr)
			{
				ImportStaticMesh(ImportContext, MeshElement, ExistingStaticMesh, MeshPayloadPtr.Get());
			}
		}
		else
		{
			ImportStaticMesh(ImportContext, MeshElement, ExistingStaticMesh, nullptr);
		}

		ImportContext.ImportedStaticMeshesByName.Add(MeshElement->GetName(), MeshElement);
	}

	//Just make sure there is no async task left running in case of a cancellation
	for (TPair<TSharedRef<IDatasmithMeshElement>, UE::Tasks::TTask<FDatasmithMeshElementPayload*>>& Kvp : MeshElementPayloads)
	{
		// Wait for the result and delete it when getting out of scope
		TUniquePtr<FDatasmithMeshElementPayload> MeshPayloadPtr(Kvp.Value.GetResult());
	}

	TMap< TSharedRef< IDatasmithMeshElement >, float > LightmapWeights = FDatasmithStaticMeshImporter::CalculateMeshesLightmapWeights( ImportContext.Scene.ToSharedRef() );

	for ( auto& ImportedStaticMeshPair : ImportContext.ImportedStaticMeshes )
	{
		FDatasmithStaticMeshImporter::SetupStaticMesh( ImportContext.AssetsContext, ImportedStaticMeshPair.Key, ImportedStaticMeshPair.Value, ImportContext.Options->BaseOptions.StaticMeshOptions, LightmapWeights[ ImportedStaticMeshPair.Key ] );
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FDatasmithImporter::ImportClothes(FDatasmithImportContext& ImportContext)  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
{
	const int32 ClothesCount = ImportContext.FilteredScene->GetClothesCount();

	if (!ImportContext.Options->BaseOptions.bIncludeGeometry
	 || ClothesCount == 0
	 || ImportContext.SceneTranslator == nullptr
	 || !ImportContext.AssetsContext.StaticMeshesFinalPackage
	 || ImportContext.AssetsContext.StaticMeshesFinalPackage->GetFName() == NAME_None)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithImporter::ImportClothes);

	// Locate a cloth asset provider
	const TArray<IDatasmithClothFactoryClassesProvider*> ClothFactoryClassesProviders = IModularFeatures::Get().GetModularFeatureImplementations<IDatasmithClothFactoryClassesProvider>(IDatasmithClothFactoryClassesProvider::FeatureName);
	if (!ClothFactoryClassesProviders.Num() || !ClothFactoryClassesProviders[0])
	{
		UE_LOGF(LogDatasmithImport, Warning, "No cloth factory class provider found, cloth import skipped.");
		return;
	}
	const UDatasmithClothAssetFactory* const ClothAssetFactory = ClothFactoryClassesProviders[0]->GetClothAssetFactoryClass().GetDefaultObject();
	if (!ClothAssetFactory)
	{
		UE_LOGF(LogDatasmithImport, Warning, "Invalid (null) cloth asset factory, cloth import skipped.");
		return;
	}
	if (ClothFactoryClassesProviders.Num() > 1)
	{
		UE_LOGF(LogDatasmithImport, Warning, "Multiple cloth asset providers found, only the cloth asset type from provider %ls will be imported.", *ClothFactoryClassesProviders[0]->GetName().ToString());
	}

	TUniquePtr<FScopedSlowTask> ProgressPtr;
	if (ImportContext.FeedbackContext)
	{
		ProgressPtr = MakeUnique<FScopedSlowTask>(ClothesCount, LOCTEXT("ImportClothes", "Importing Clothes..."), true, *ImportContext.FeedbackContext);
		ProgressPtr->MakeDialog(true);
	}

	FScopedSlowTask* Progress = ProgressPtr.Get();

	for (int32 ClothIndex = 0; ClothIndex < ClothesCount && !ImportContext.bUserCancelled; ++ClothIndex)
	{
		ImportContext.bUserCancelled = FDatasmithImporterImpl::HasUserCancelledTask(ImportContext.FeedbackContext);

		TSharedPtr<IDatasmithClothElement> ClothElementPtr = ImportContext.FilteredScene->GetCloth(ClothIndex);
		if (!ClothElementPtr)
		{
			continue;
		}
		TSharedRef<IDatasmithClothElement> ClothElement = ClothElementPtr.ToSharedRef();

		FDatasmithImporterImpl::ReportProgress(Progress, 1.f, FText::FromString(FString::Printf(TEXT("Importing cloth %d/%d (%s) ..."), ClothIndex + 1, ClothesCount, ClothElement->GetLabel())));

		int32 MaxNameCharCount = FDatasmithImporterUtils::GetAssetNameMaxCharCount(ImportContext.AssetsContext.StaticMeshesFinalPackage.Get());
		FString ClothName = ImportContext.AssetsContext.StaticMeshNameProvider.GenerateUniqueName(ClothElement->GetLabel(), MaxNameCharCount);
		ClothName = ObjectTools::SanitizeObjectName(ClothName);

		FDatasmithClothElementPayload Payload;
		// #ue_ds_cloth_todo: async parallel support
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithImporter::ImportClothes:TranslatorLoadCloth);
			if (!ImportContext.SceneTranslator->LoadCloth(ClothElement, Payload))
			{
				UE_LOGF(LogDatasmithImport, Warning, "Cloth %ls: load failed, translator issue.", ClothElement->GetName());
				return;
			}

			if (Payload.Cloth.Patterns.IsEmpty())
			{
				UE_LOGF(LogDatasmithImport, Warning, "Cloth %ls: no patterns, import skipped.", ClothElement->GetName());
				return;
			}
		}
		FDatasmithCloth DsCloth = MoveTemp(Payload.Cloth);

		UPackage* ImportPackage = ImportContext.AssetsContext.StaticMeshesImportPackage.Get();
		EObjectFlags ObjectFlags = ImportContext.ObjectFlags & ~RF_Public; // not RF_Public yet, the publicized asset will be.

		// TODO: Use FCollectionPropertyFacade instead, see \Engine\Source\Runtime\Experimental\Chaos\Public\Chaos\CollectionPropertyFacade.h
		//TArray<UObject*> ClothPresetAssets;

		//for (const FDatasmithClothPresetPropertySet& PropertySet : DsCloth.PropertySets)
		//{
		//	FString PresetAssetName = ObjectTools::SanitizeObjectName(PropertySet.SetName);
		//	if (UChaosClothPreset* PropertySetAsset = NewObject<UChaosClothPreset>(ImportPackage, *PresetAssetName, ObjectFlags))
		//	{
		//		ClothPresetAssets.Add(PropertySetAsset);
		//		for (const FDatasmithClothPresetProperty& Property : PropertySet.Properties)
		//		{
		//			// #ue_ds_cloth_todo fill property values, min/max,
		//			PropertySetAsset->AddProperty(Property.Name, EChaosClothPresetPropertyType::String);
		//		}
		//	}
		//}

		UObject* ExistingCloth = nullptr;
		if (ImportContext.SceneAsset)
		{
			if (TSoftObjectPtr<UObject>* ExistingAssetPtr = ImportContext.SceneAsset->Clothes.Find(ClothElement->GetName()))
			{
				ExistingCloth = ExistingAssetPtr->LoadSynchronous();
			}
		}
		UObject* ClothAsset = nullptr;
		if (ExistingCloth)
		{
			if (ExistingCloth->GetOuter() != ImportPackage)
			{
				// Temporary flag to skip PostLoad during DuplicateObject
				ExistingCloth->SetFlags(RF_ArchetypeObject);
				ClothAsset = ClothAssetFactory->DuplicateClothAsset(ExistingCloth, ImportPackage, *ClothName);
				ExistingCloth->ClearFlags(RF_ArchetypeObject);
				ClothAsset->ClearFlags(RF_ArchetypeObject);

				IDatasmithImporterModule::Get().ResetOverrides(ClothAsset);
			}
			else
			{
				ClothAsset = ExistingCloth;
			}

			ExistingCloth->SetFlags(ObjectFlags);
		}
		else
		{
			ClothAsset = ClothAssetFactory->CreateClothAsset(ImportPackage, *ClothName, ObjectFlags);
		}

		if (!ensure(ClothAsset))
		{
			continue;
		}

		ClothAssetFactory->InitializeClothAsset(ClothAsset, DsCloth);

		ImportContext.ImportedClothes.Add(ClothElement, ClothAsset);

		// TODO: Use FCollectionPropertyFacade instead, see \Engine\Source\Runtime\Experimental\Chaos\Public\Chaos\CollectionPropertyFacade.h
		//for (UObject* ClothPresetAsset : ClothPresetAssets)
		//{
		//	ImportContext.ImportedClothPresets.Add(ClothPresetAsset);
		//}
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

UStaticMesh* FDatasmithImporter::ImportStaticMesh( FDatasmithImportContext& ImportContext, TSharedRef< IDatasmithMeshElement > MeshElement, UStaticMesh* ExistingStaticMesh, FDatasmithMeshElementPayload* MeshPayload)
{
	if ( !ImportContext.AssetsContext.StaticMeshesFinalPackage || ImportContext.AssetsContext.StaticMeshesFinalPackage->GetFName() == NAME_None || ImportContext.SceneTranslator == nullptr)
	{
		return nullptr;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithImporter::ImportStaticMesh);

	auto& ImportedStaticMesh = ImportContext.ImportedStaticMeshes.FindOrAdd( MeshElement );

	TArray<UDatasmithAdditionalData*> AdditionalData;

	if ( ImportedStaticMesh == nullptr ) // We still have factories that are importing the UStaticMesh on their own, so check if it's already imported here
	{
		FDatasmithMeshElementPayload LocalMeshPayload;
		if (MeshPayload == nullptr)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(LoadStaticMesh);
			ImportContext.SceneTranslator->LoadStaticMesh(MeshElement, LocalMeshPayload);
			MeshPayload = &LocalMeshPayload;
		}

		ImportedStaticMesh = FDatasmithStaticMeshImporter::ImportStaticMesh( MeshElement, *MeshPayload, ImportContext.ObjectFlags & ~RF_Public, ImportContext.Options->BaseOptions.StaticMeshOptions, ImportContext.AssetsContext, ExistingStaticMesh );
		AdditionalData = MoveTemp(MeshPayload->AdditionalData);

		// Make sure the garbage collector can collect additional data allocated on other thread
		for (UDatasmithAdditionalData* Data : AdditionalData)
		{
			if (Data)
			{
				Data->ClearInternalFlags(EInternalObjectFlags::Async);
			}
		}

		// Creation of static mesh failed, remove it from the list of importer mesh elements
		if (ImportedStaticMesh == nullptr)
		{
			ImportContext.ImportedStaticMeshes.Remove(MeshElement);
			return nullptr;
		}
	}

	CreateStaticMeshAssetImportData( ImportContext, MeshElement, ImportedStaticMesh, AdditionalData );

	ImportMetaDataForObject( ImportContext, MeshElement, ImportedStaticMesh );

	if ( MeshElement->GetLightmapSourceUV() >= MAX_MESH_TEXTURE_COORDS_MD )
	{
		FFormatNamedArguments FormatArgs;
		FormatArgs.Add(TEXT("SourceUV"), FText::FromString(FString::FromInt(MeshElement->GetLightmapSourceUV())));
		FormatArgs.Add(TEXT("MeshName"), FText::FromName(ImportedStaticMesh->GetFName()));
		ImportContext.LogError(FText::Format(LOCTEXT("InvalidLightmapSourceUVError", "The lightmap source UV '{SourceUV}' used for the lightmap UV generation of the mesh '{MeshName}' is invalid."), FormatArgs));
	}

	return ImportedStaticMesh;
}


UStaticMesh* FDatasmithImporter::FinalizeStaticMesh( UStaticMesh* SourceStaticMesh, const TCHAR* StaticMeshesFolderPath, UStaticMesh* ExistingStaticMesh, TMap< UObject*, UObject* >* ReferencesToRemap, bool bBuild)
{
	UStaticMesh* DestinationStaticMesh = Cast< UStaticMesh >( FDatasmithImporterImpl::FinalizeAsset( SourceStaticMesh, StaticMeshesFolderPath, ExistingStaticMesh, ReferencesToRemap ) );

	if (bBuild)
	{
		FDatasmithStaticMeshImporter::BuildStaticMesh(DestinationStaticMesh);
	}

	return DestinationStaticMesh;
}

UObject* FDatasmithImporter::FinalizeCloth(UObject* SourceCloth, const TCHAR* FolderPath, UObject* ExistingCloth, TMap<UObject*, UObject*>* ReferencesToRemap)  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
{
	if (Cast<USkinnedAsset>(ExistingCloth))
	{
		// Delete the old asset completely. This fixes a crash with the render ressources being used while the asset is replaced.
		TArray<UObject*> ExistingClothAsArray{ExistingCloth};
		ObjectTools::ForceReplaceReferences(SourceCloth, ExistingClothAsArray);
		ObjectTools::DeleteAssets({ExistingCloth}, false);
		ExistingCloth = nullptr;
	}
	UObject* Asset = FDatasmithImporterImpl::FinalizeAsset(SourceCloth, FolderPath, ExistingCloth, ReferencesToRemap);

	return Asset;
}

void FDatasmithImporter::CreateStaticMeshAssetImportData(FDatasmithImportContext& InContext, TSharedRef< IDatasmithMeshElement > MeshElement, UStaticMesh* ImportedStaticMesh, TArray<UDatasmithAdditionalData*>& AdditionalData)
{
	UDatasmithStaticMeshImportData::DefaultOptionsPair ImportOptions = UDatasmithStaticMeshImportData::DefaultOptionsPair( InContext.Options->BaseOptions.StaticMeshOptions, InContext.Options->BaseOptions.AssetOptions );

	UDatasmithStaticMeshImportData* MeshImportData = UDatasmithStaticMeshImportData::GetImportDataForStaticMesh( ImportedStaticMesh, ImportOptions );

	if ( MeshImportData )
	{
		// Update the import data source file and set the mesh hash
		// #ueent_todo FH: piggybacking off of the SourceData file hash for now, until we have custom derived AssetImportData properly serialize to the AssetRegistry
		FMD5Hash Hash = MeshElement->CalculateElementHash( false );
		MeshImportData->Update( InContext.Options->FilePath, &Hash );
		MeshImportData->DatasmithImportInfo = FDatasmithImportInfo(InContext.Options->SourceUri, InContext.Options->SourceHash);

		// Set the final outer // #ueent_review: propagate flags of outer?
		for (UDatasmithAdditionalData* Data: AdditionalData)
		{
			Data->Rename(nullptr, MeshImportData);
		}
		MeshImportData->AdditionalData = MoveTemp(AdditionalData);
	}
}

void FDatasmithImporter::ImportTextures( FDatasmithImportContext& ImportContext )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithImporter::ImportTextures);

	FDatasmithImporterImpl::SetTexturesMode( ImportContext );

	const int32 TexturesCount = ImportContext.FilteredScene->GetTexturesCount();

	TUniquePtr<FScopedSlowTask> ProgressPtr;

	if ( ImportContext.FeedbackContext )
	{
		ProgressPtr = MakeUnique<FScopedSlowTask>( (float)TexturesCount, LOCTEXT("ImportingTextures", "Importing Textures..."), true, *ImportContext.FeedbackContext );
		ProgressPtr->MakeDialog(true);
	}

	if (ImportContext.Options->TextureConflictPolicy != EDatasmithImportAssetConflictPolicy::Ignore && TexturesCount > 0)
	{
		FDatasmithTextureImporter DatasmithTextureImporter(ImportContext);

		TArray<TSharedPtr< IDatasmithTextureElement >> FilteredTextureElements;
		for ( int32 i = 0; i < TexturesCount; i++ )
		{
			TSharedPtr< IDatasmithTextureElement > TextureElement = ImportContext.FilteredScene->GetTexture(i);

			if ( !TextureElement )
			{
				continue;
			}

			FilteredTextureElements.Add(TextureElement);
		}

		FDatasmithTextureResize::Initialize();

		// Try to import textures async first, this does not work with DatasmithImportFactory as it creates a deadlock on the main thread.
		const bool bInterchangeImport = false; /*GetDefault<UEditorExperimentalSettings>()->bEnableInterchangeFramework*/
		if ( bInterchangeImport )
		{
			for ( int32 TextureIndex = 0; TextureIndex < FilteredTextureElements.Num(); TextureIndex++ )
			{
				if (FDatasmithImporterImpl::HasUserCancelledTask(ImportContext.FeedbackContext))
				{
					ImportContext.bUserCancelled = true;
				}

				if (ImportContext.bUserCancelled)
				{
					break;
				}

				// Skip asynchronous creation for IES textures as it is not supported.
				// The creation of such textures will synchronously happen later in the call to FDatasmithTextureImporter::CreateTexture
				if (FilteredTextureElements[TextureIndex]->GetTextureMode() == EDatasmithTextureMode::Ies)
				{
					continue;
				}

				UE::Interchange::FAssetImportResultRef FutureTexture = DatasmithTextureImporter.CreateTextureAsync( FilteredTextureElements[TextureIndex] );

				if ( FutureTexture->IsValid() )
				{
					FutureTexture->OnDone(
						[ TextureElement = FilteredTextureElements[TextureIndex].ToSharedRef(), &ImportContext ]( UE::Interchange::FImportResult& AssetImportResults )
						{
							ImportMetaDataForObject( ImportContext, TextureElement, AssetImportResults.GetFirstAssetOfClass( UTexture::StaticClass() ) );
						}
					);

					ImportContext.ImportedTextures.Add( FilteredTextureElements[TextureIndex].ToSharedRef(), MoveTemp( FutureTexture ) );
				}
			}
		}

		struct FAsyncData
		{
			FString       Extension;
			TArray<uint8> TextureData;
			UE::Tasks::TTask<bool> Result;
		};
		TArray<FAsyncData> AsyncData;

		AsyncData.SetNum(FilteredTextureElements.Num());

		for ( int32 TextureIndex = 0; TextureIndex < FilteredTextureElements.Num(); TextureIndex++ )
		{
			if ( ImportContext.ImportedTextures.Contains( FilteredTextureElements[TextureIndex].ToSharedRef() ) )
			{
				continue;
			}

			if (FDatasmithImporterImpl::HasUserCancelledTask(ImportContext.FeedbackContext))
			{
				ImportContext.bUserCancelled = true;
			}

			AsyncData[TextureIndex].Result =
				UE::Tasks::Launch(UE_SOURCE_LOCATION,
					[&ImportContext, &AsyncData, &FilteredTextureElements, &DatasmithTextureImporter, TextureIndex]()
					{
						if (ImportContext.bUserCancelled)
						{
							return false;
						}

						if (FilteredTextureElements[TextureIndex]->GetTextureMode() == EDatasmithTextureMode::Ies)
						{
							return true;
						}

						return DatasmithTextureImporter.GetTextureData(FilteredTextureElements[TextureIndex], AsyncData[TextureIndex].TextureData, AsyncData[TextureIndex].Extension);
					}
				);
		}

		// Avoid a call to IsValid for each item
		FScopedSlowTask* Progress = ProgressPtr.Get();

		for ( int32 TextureIndex = 0; TextureIndex < FilteredTextureElements.Num(); TextureIndex++ )
		{
			if ( ImportContext.ImportedTextures.Contains( FilteredTextureElements[TextureIndex].ToSharedRef() ) )
			{
				continue;
			}

			if (FDatasmithImporterImpl::HasUserCancelledTask(ImportContext.FeedbackContext))
			{
				ImportContext.bUserCancelled = true;
			}

			if ( ImportContext.bUserCancelled )
			{
				// If operation has been canceled, just wait for other threads to also cancel
				AsyncData[TextureIndex].Result.Wait();
			}
			else
			{
				const TSharedPtr< IDatasmithTextureElement >& TextureElement = FilteredTextureElements[TextureIndex];

				FDatasmithImporterImpl::ReportProgress( Progress, 1.f, FText::FromString( FString::Printf( TEXT("Importing texture %d/%d (%s) ..."), TextureIndex + 1, FilteredTextureElements.Num(), TextureElement->GetLabel() ) ) );

				UTexture* ExistingTexture = nullptr;

				if ( ImportContext.SceneAsset )
				{
					TSoftObjectPtr< UTexture >* ExistingTexturePtr = ImportContext.SceneAsset->Textures.Find( TextureElement->GetName() );

					if ( ExistingTexturePtr )
					{
						ExistingTexture = ExistingTexturePtr->LoadSynchronous();
					}
				}

				if (AsyncData[TextureIndex].Result.GetResult())
				{
					ImportTexture( ImportContext, DatasmithTextureImporter, TextureElement.ToSharedRef(), ExistingTexture, AsyncData[TextureIndex].TextureData, AsyncData[TextureIndex].Extension );
				}
			}

			// Release memory as soon as possible
			AsyncData[TextureIndex].TextureData.Empty();
		}
	}
}

UTexture* FDatasmithImporter::ImportTexture( FDatasmithImportContext& ImportContext, FDatasmithTextureImporter& DatasmithTextureImporter, TSharedRef< IDatasmithTextureElement > TextureElement, UTexture* ExistingTexture, const TArray<uint8>& TextureData, const FString& Extension )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithImporter::ImportTexture);

	UTexture* ImportedTexture = DatasmithTextureImporter.CreateTexture( TextureElement, TextureData, Extension );

	if ( !ImportedTexture )
	{
		return nullptr;
	}

	UE::Interchange::FAssetImportResultRef& AssetImportResults = ImportContext.ImportedTextures.Add( TextureElement, MakeShared< UE::Interchange::FImportResult, ESPMode::ThreadSafe >() );
	AssetImportResults->AddImportedObject( ImportedTexture );
	AssetImportResults->SetDone();

	ImportMetaDataForObject( ImportContext, TextureElement, ImportedTexture );

	return Cast< UTexture >( ImportedTexture );
}

UTexture* FDatasmithImporter::FinalizeTexture( UTexture* SourceTexture, const TCHAR* TexturesFolderPath, UTexture* ExistingTexture, TMap< UObject*, UObject* >* ReferencesToRemap )
{
	return Cast< UTexture >( FDatasmithImporterImpl::FinalizeAsset( SourceTexture, TexturesFolderPath, ExistingTexture, ReferencesToRemap ) );
}

void FDatasmithImporter::ImportMaterials( FDatasmithImportContext& ImportContext )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithImporter::ImportMaterials);

	if ( ImportContext.Options->MaterialConflictPolicy != EDatasmithImportAssetConflictPolicy::Ignore && ImportContext.FilteredScene->GetMaterialsCount() > 0 )
	{
		IDatasmithShaderElement::bUseRealisticFresnel = ( ImportContext.Options->MaterialQuality == EDatasmithImportMaterialQuality::UseRealFresnelCurves );
		IDatasmithShaderElement::bDisableReflectionFresnel = ( ImportContext.Options->MaterialQuality == EDatasmithImportMaterialQuality::UseNoFresnelCurves );

		//Import referenced materials as MaterialFunctions first
		{
			using FFunctionAndMaterialsThatUseIt = FDatasmithImporterUtils::FFunctionAndMaterialsThatUseIt;
			TArray<FFunctionAndMaterialsThatUseIt> Functions = FDatasmithImporterUtils::GetOrderedListOfMaterialsReferencedByMaterials( ImportContext.FilteredScene );

			UE::DatasmithImporter::Private::MakeChangeToExpressionsNameForMaterialFunctions(Functions);

			for (const FFunctionAndMaterialsThatUseIt& FunctionAndMaterials : Functions)
			{
				ImportMaterialFunction(ImportContext, FunctionAndMaterials.Key.ToSharedRef());
			}
		}

		ImportContext.AssetsContext.MaterialsRequirements.Empty( ImportContext.FilteredScene->GetMaterialsCount() );

		for (FDatasmithImporterUtils::FDatasmithMaterialImportIterator It(ImportContext); It; ++It)
		{
			TSharedRef< IDatasmithBaseMaterialElement > MaterialElement = It.Value().ToSharedRef();

			UMaterialInterface* ExistingMaterial = nullptr;

			if ( ImportContext.SceneAsset )
			{
				TSoftObjectPtr< UMaterialInterface >* ExistingMaterialPtr = ImportContext.SceneAsset->Materials.Find( MaterialElement->GetName() );

				if ( ExistingMaterialPtr )
				{
					ExistingMaterial = ExistingMaterialPtr->LoadSynchronous();
				}
			}

			ImportMaterial( ImportContext, MaterialElement, ExistingMaterial );
		}

		// IMPORTANT: FGlobalComponentReregisterContext destructor will de-register and re-register all UActorComponent present in the world
		// Consequently, all static meshes will stop using the FMaterialResource of the original materials on de-registration
		// and will use the new FMaterialResource created on re-registration.
		// Otherwise, the editor will crash on redraw
		FGlobalComponentReregisterContext RecreateComponents;
	}
}

UMaterialFunction* FDatasmithImporter::ImportMaterialFunction(FDatasmithImportContext& ImportContext, TSharedRef< IDatasmithBaseMaterialElement > MaterialElement)
{
	UMaterialFunction* ImportedMaterialFunction = FDatasmithMaterialImporter::CreateMaterialFunction( ImportContext, MaterialElement );

	if (!ImportedMaterialFunction )
	{
		return nullptr;
	}

	ImportContext.ImportedMaterialFunctions.Add( MaterialElement ) = ImportedMaterialFunction;

	return ImportedMaterialFunction;
}

UMaterialFunction* FDatasmithImporter::FinalizeMaterialFunction(UObject* SourceMaterialFunction, const TCHAR* MaterialFunctionsFolderPath,
	UMaterialFunction* ExistingMaterialFunction, TMap< UObject*, UObject* >* ReferencesToRemap)
{
	UMaterialFunction* MaterialFunction = Cast< UMaterialFunction >( FDatasmithImporterImpl::FinalizeAsset( SourceMaterialFunction, MaterialFunctionsFolderPath, ExistingMaterialFunction, ReferencesToRemap ) );

	MaterialFunction->PreEditChange( nullptr );
	MaterialFunction->PostEditChange();

	return MaterialFunction;
}

UMaterialInterface* FDatasmithImporter::ImportMaterial( FDatasmithImportContext& ImportContext,
	TSharedRef< IDatasmithBaseMaterialElement > MaterialElement, UMaterialInterface* ExistingMaterial )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithImporter::ImportMaterial);

	UMaterialInterface* ImportedMaterial = FDatasmithMaterialImporter::CreateMaterial( ImportContext, MaterialElement, ExistingMaterial );

	if ( !ImportedMaterial )
	{
		return nullptr;
	}

	if (GetDefault<URendererSettings>()->bEnableVirtualTextureOpacityMask == false)
	{
		//Virtual textures are not supported in the OpacityMask slot, convert any textures back to a regular texture.
		TArray<UTexture*> OutOpacityMaskTextures;
		if (ImportedMaterial->GetTexturesInPropertyChain(MP_OpacityMask, OutOpacityMaskTextures, nullptr, nullptr))
		{
			for (UTexture* CurrentTexture : OutOpacityMaskTextures)
			{
				UTexture2D* Texture2D = Cast<UTexture2D>(CurrentTexture);
				if (Texture2D && Texture2D->VirtualTextureStreaming)
				{
					ImportContext.AssetsContext.VirtualTexturesToConvert.Add(Texture2D);
				}
			}
		}
	}

	UDatasmithAssetImportData* AssetImportData = Cast< UDatasmithAssetImportData >(ImportedMaterial->AssetImportData);

	if (!AssetImportData)
	{
		AssetImportData = NewObject< UDatasmithAssetImportData >(ImportedMaterial);
		ImportedMaterial->AssetImportData = AssetImportData;
	}

	AssetImportData->Update(ImportContext.Options->FilePath, ImportContext.FileHash.IsValid() ? &ImportContext.FileHash : nullptr);
	AssetImportData->AssetImportOptions = ImportContext.Options->BaseOptions.AssetOptions;
	AssetImportData->DatasmithImportInfo = FDatasmithImportInfo(ImportContext.Options->SourceUri, ImportContext.Options->SourceHash);

	// Record requirements on mesh building for this material
	ImportContext.AssetsContext.MaterialsRequirements.Add( MaterialElement->GetName(), FDatasmithMaterialImporter::GetMaterialRequirements( ImportedMaterial ) );
	ImportContext.ImportedMaterials.Add( MaterialElement ) = ImportedMaterial;

	ImportMetaDataForObject( ImportContext, MaterialElement, ImportedMaterial );

	return ImportedMaterial;
}

UObject* FDatasmithImporter::FinalizeMaterial( UObject* SourceMaterial, const TCHAR* MaterialFolderPath, const TCHAR* TransientPackagePath, const TCHAR* RootFolderPath, UMaterialInterface* ExistingMaterial, TMap< UObject*, UObject* >* ReferencesToRemap, FMaterialUpdateContext* MaterialUpdateContext)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithImporter::FinalizeMaterial);

	// Finalizing the material instance might add a remapping for the instance parent property so make sure we have a remapping map available
	TOptional< TMap< UObject*, UObject* > > ReferencesToRemapLocal;
	if ( !ReferencesToRemap )
	{
		ReferencesToRemapLocal.Emplace();
		ReferencesToRemap = &ReferencesToRemapLocal.GetValue();
	}

	if ( UMaterialInstance* SourceMaterialInstance = Cast< UMaterialInstance >( SourceMaterial ) )
	{
		if ( UMaterialInterface* SourceMaterialParent = SourceMaterialInstance->Parent )
		{
			// Do not finalize parent material more than once by verifying it is not already in ReferencesToRemap
			if (!ReferencesToRemap->Contains(SourceMaterialParent))
			{
				FString SourceMaterialPath = SourceMaterialInstance->GetOutermost()->GetName();
				FString SourceParentPath = SourceMaterialParent->GetOutermost()->GetName();

				if ( SourceParentPath.StartsWith( TransientPackagePath ) )
				{
					// Simply finalize the source parent material.
					// Note that the parent material will be overridden on the existing material instance
					const FString DestinationParentPath = SourceParentPath.Replace( TransientPackagePath, RootFolderPath, ESearchCase::CaseSensitive );

					FinalizeMaterial( SourceMaterialParent, *DestinationParentPath, TransientPackagePath, RootFolderPath, nullptr, ReferencesToRemap );
				}
			}
		}
	}

	UMaterialEditingLibrary::DeleteAllMaterialExpressions( Cast< UMaterial >( ExistingMaterial ) );

	UObject* DestinationMaterial = FDatasmithImporterImpl::FinalizeAsset( SourceMaterial, MaterialFolderPath, ExistingMaterial, ReferencesToRemap );

	FDatasmithImporterImpl::CompileMaterial( DestinationMaterial, MaterialUpdateContext);

	return DestinationMaterial;
}

void FDatasmithImporter::ImportActors( FDatasmithImportContext& ImportContext )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithImporter::ImportActors);

	/**
	 * Hot fix for reimport issues UE-71655. A temporary created actor might have the same object path as the previously deleted actor.
	 * This code below won't be needed when UE-71695 is fixed. This should be in 4.23.
	 */
	TArray< ADatasmithSceneActor* > SceneActors = FDatasmithImporterUtils::FindSceneActors( ImportContext.ActorsContext.FinalWorld, ImportContext.SceneAsset );
	for ( ADatasmithSceneActor* SceneActor : SceneActors )
	{
		if ( !SceneActor )
		{
			continue;
		}

		if ( ImportContext.SceneAsset == SceneActor->Scene && SceneActor->GetLevel() == ImportContext.ActorsContext.FinalWorld->GetCurrentLevel())
		{
			for ( TPair< FName, TSoftObjectPtr< AActor > >& Pair : SceneActor->RelatedActors )
			{
				// Try to load the actor. If we can't reset the soft object ptr
				if ( !Pair.Value.LoadSynchronous() )
				{
					Pair.Value.Reset();
				}
			}
		}
	}
	// end of the hotfix

	ADatasmithSceneActor* ImportSceneActor = ImportContext.ActorsContext.ImportSceneActor;

	// Create a scene actor to import with if we don't have one
	if ( !ImportSceneActor )
	{
		// Create a the import scene actor for the import context
		ImportSceneActor = FDatasmithImporterUtils::CreateImportSceneActor( ImportContext, FTransform::Identity );
	}

	const int32 ActorsCount = ImportContext.Scene->GetActorsCount();

	TUniquePtr<FScopedSlowTask> ProgressPtr;

	if ( ImportContext.FeedbackContext )
	{
		ProgressPtr = MakeUnique<FScopedSlowTask>( (float)ActorsCount, LOCTEXT("ImportActors", "Spawning actors..."), true, *ImportContext.FeedbackContext );
		ProgressPtr->MakeDialog(true);
	}

	FScopedSlowTask* Progress = ProgressPtr.Get();

	if ( ImportSceneActor )
	{
		ImportContext.Hierarchy.Push( ImportSceneActor->GetRootComponent() );

		FDatasmithActorUniqueLabelProvider UniqueNameProvider;

		for (int32 i = 0; i < ActorsCount && !ImportContext.bUserCancelled; ++i)
		{
			if (FDatasmithImporterImpl::HasUserCancelledTask(ImportContext.FeedbackContext))
			{
				ImportContext.bUserCancelled = true;
			}

			TSharedPtr< IDatasmithActorElement > ActorElement = ImportContext.Scene->GetActor(i);

			if ( ActorElement.IsValid() )
			{
				FDatasmithImporterImpl::ReportProgress( Progress, 1.f, FText::FromString( FString::Printf( TEXT("Spawning actor %d/%d (%s) ..."), ( i + 1 ), ActorsCount, ActorElement->GetLabel() ) ) );

				if ( ActorElement->IsAComponent() )
				{
					ImportActorAsComponent( ImportContext, ActorElement.ToSharedRef(), ImportSceneActor, UniqueNameProvider );
				}
				else
				{
					ImportActor( ImportContext, ActorElement.ToSharedRef() );
				}
			}
		}

		// After all actors were imported, perform a post import step so that any dependencies can be resolved
		for (int32 i = 0; i < ActorsCount && !ImportContext.bUserCancelled; ++i)
		{
			if (FDatasmithImporterImpl::HasUserCancelledTask(ImportContext.FeedbackContext))
			{
				ImportContext.bUserCancelled = true;
			}

			TSharedPtr< IDatasmithActorElement > ActorElement = ImportContext.Scene->GetActor(i);

			if ( ActorElement.IsValid() && ActorElement->IsA( EDatasmithElementType::Camera ) )
			{
				FDatasmithCameraImporter::PostImportCameraActor( StaticCastSharedRef< IDatasmithCameraActorElement >( ActorElement.ToSharedRef() ), ImportContext );
			}
		}

		ImportSceneActor->Scene = ImportContext.SceneAsset;

		ImportContext.Hierarchy.Pop();
	}

	// Sky
	if( ImportContext.Scene->GetUsePhysicalSky() )
	{
		AActor* SkyActor = FDatasmithLightImporter::CreatePhysicalSky(ImportContext);
	}

	if ( ImportContext.bUserCancelled )
	{
		FDatasmithImporterImpl::DeleteImportSceneActorIfNeeded(ImportContext.ActorsContext, true );
	}
}

AActor* FDatasmithImporter::ImportActor( FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithActorElement >& ActorElement )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithImporter::ImportActor);

	FDatasmithActorUniqueLabelProvider UniqueNameProvider;

	AActor* ImportedActor = nullptr;
	if (ActorElement->IsA(EDatasmithElementType::HierarchicalInstanceStaticMesh))
	{
		ImportedActor =  FDatasmithActorImporter::ImportHierarchicalInstancedStaticMeshAsActor( ImportContext, StaticCastSharedRef< IDatasmithHierarchicalInstancedStaticMeshActorElement >( ActorElement ), UniqueNameProvider );
	}
	else if (ActorElement->IsA(EDatasmithElementType::StaticMeshActor))
	{
		ImportedActor = FDatasmithActorImporter::ImportStaticMeshActor( ImportContext, StaticCastSharedRef< IDatasmithMeshActorElement >( ActorElement ) );
	}
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	else if (ActorElement->IsA(EDatasmithElementType::ClothActor))  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	{
		ImportedActor = FDatasmithActorImporter::ImportClothActor(ImportContext, StaticCastSharedRef<IDatasmithClothActorElement>( ActorElement ) );
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	else if (ActorElement->IsA(EDatasmithElementType::EnvironmentLight))
	{
		ImportedActor = FDatasmithActorImporter::ImportEnvironment( ImportContext, StaticCastSharedRef< IDatasmithEnvironmentElement >( ActorElement ) );
	}
	else if (ActorElement->IsA(EDatasmithElementType::Light))
	{
		ImportedActor = FDatasmithActorImporter::ImportLightActor( ImportContext, StaticCastSharedRef< IDatasmithLightActorElement >(ActorElement) );
	}
	else if (ActorElement->IsA(EDatasmithElementType::Camera))
	{
		ImportedActor = FDatasmithActorImporter::ImportCameraActor( ImportContext, StaticCastSharedRef< IDatasmithCameraActorElement >(ActorElement) );
	}
	else if (ActorElement->IsA(EDatasmithElementType::Decal))
	{
		ImportedActor = FDatasmithActorImporter::ImportDecalActor( ImportContext, StaticCastSharedRef< IDatasmithDecalActorElement >(ActorElement), UniqueNameProvider );
	}
	else if (ActorElement->IsA(EDatasmithElementType::CustomActor))
	{
		ImportedActor = FDatasmithActorImporter::ImportCustomActor( ImportContext, StaticCastSharedRef< IDatasmithCustomActorElement >(ActorElement), UniqueNameProvider );
	}
	else if (ActorElement->IsA(EDatasmithElementType::Landscape))
	{
		ImportedActor = FDatasmithActorImporter::ImportLandscapeActor( ImportContext, StaticCastSharedRef< IDatasmithLandscapeElement >(ActorElement) );
	}
	else if (ActorElement->IsA(EDatasmithElementType::PostProcessVolume))
	{
		ImportedActor = FDatasmithPostProcessImporter::ImportPostProcessVolume( StaticCastSharedRef< IDatasmithPostProcessVolumeElement >( ActorElement ), ImportContext, ImportContext.Options->OtherActorImportPolicy );
	}
	else
	{
		ImportedActor = FDatasmithActorImporter::ImportBaseActor( ImportContext, ActorElement );
	}

	if ( ImportedActor ) // It's possible that we didn't import an actor (ie: the user doesn't want to import the cameras), in that case, we'll skip it in the hierarchy
	{
		ImportContext.Hierarchy.Push( ImportedActor->GetRootComponent() );
		ImportMetaDataForObject(ImportContext, ActorElement, ImportedActor);

		ImportContext.AddImportedActor(ImportedActor);

		if (ADatasmithSceneActor* DatasmithSceneActor = ImportContext.ActorsContext.ImportSceneActor)
		{
			DatasmithSceneActor->RelatedActors.FindOrAdd(ActorElement->GetName()) = ImportedActor;
		}
	}
	else
	{
		ImportContext.ActorsContext.NonImportedDatasmithActors.Add( ActorElement->GetName() );
	}

	for (int32 i = 0; i < ActorElement->GetChildrenCount() && !ImportContext.bUserCancelled; ++i)
	{
		if (FDatasmithImporterImpl::HasUserCancelledTask(ImportContext.FeedbackContext))
		{
			ImportContext.bUserCancelled = true;
		}

		const TSharedPtr< IDatasmithActorElement >& ChildActorElement = ActorElement->GetChild(i);

		if ( ChildActorElement.IsValid() )
		{
			if (!ChildActorElement->IsAComponent() )
			{
				ImportActor( ImportContext, ChildActorElement.ToSharedRef() );
			}
			else if ( ImportedActor ) // Don't import the components of an actor that we didn't import
			{
				ImportActorAsComponent( ImportContext, ChildActorElement.ToSharedRef(), ImportedActor, UniqueNameProvider );
			}
		}
	}

	if ( ImportedActor )
	{
		ImportContext.Hierarchy.Pop();
	}

	return ImportedActor;
}

void FDatasmithImporter::ImportActorAsComponent(FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithActorElement >& ActorElement, AActor* InRootActor, FDatasmithActorUniqueLabelProvider& UniqueNameProvider)
{
	if (!InRootActor)
	{
		return;
	}

	USceneComponent* SceneComponent = nullptr;

	if (ActorElement->IsA(EDatasmithElementType::HierarchicalInstanceStaticMesh))
	{
		TSharedRef< IDatasmithHierarchicalInstancedStaticMeshActorElement > HierarchicalInstancedStaticMeshElement = StaticCastSharedRef< IDatasmithHierarchicalInstancedStaticMeshActorElement >(ActorElement);
		SceneComponent = FDatasmithActorImporter::ImportHierarchicalInstancedStaticMeshComponent(ImportContext, HierarchicalInstancedStaticMeshElement, InRootActor, UniqueNameProvider);
	}
	else if (ActorElement->IsA(EDatasmithElementType::StaticMeshActor))
	{
		TSharedRef< IDatasmithMeshActorElement > MeshActorElement = StaticCastSharedRef< IDatasmithMeshActorElement >(ActorElement);
		SceneComponent = FDatasmithActorImporter::ImportStaticMeshComponent(ImportContext, MeshActorElement, InRootActor, UniqueNameProvider);
	}
	else if (ActorElement->IsA(EDatasmithElementType::Light))
	{
		if (ImportContext.Options->LightImportPolicy == EDatasmithImportActorPolicy::Ignore)
		{
			return;
		}

		SceneComponent = FDatasmithLightImporter::ImportLightComponent(StaticCastSharedRef< IDatasmithLightActorElement >(ActorElement), ImportContext, InRootActor, UniqueNameProvider);
	}
	else if (ActorElement->IsA(EDatasmithElementType::Camera))
	{
		if (ImportContext.Options->CameraImportPolicy == EDatasmithImportActorPolicy::Ignore)
		{
			return;
		}

		SceneComponent = FDatasmithCameraImporter::ImportCineCameraComponent(StaticCastSharedRef< IDatasmithCameraActorElement >(ActorElement), ImportContext, InRootActor, UniqueNameProvider);
	}
	else
	{
		SceneComponent = FDatasmithActorImporter::ImportBaseActorAsComponent(ImportContext, ActorElement, InRootActor, UniqueNameProvider);
	}

	if (SceneComponent)
	{
		ImportContext.AddSceneComponent(SceneComponent->GetName(), SceneComponent);
		ImportMetaDataForObject(ImportContext, ActorElement, SceneComponent);
	}
	else
	{
		ImportContext.ActorsContext.NonImportedDatasmithActors.Add(ActorElement->GetName());
	}

	for (int32 i = 0; i < ActorElement->GetChildrenCount(); ++i)
	{
		if (SceneComponent) // If we didn't import the current component, skip it in the hierarchy
		{
			ImportContext.Hierarchy.Push(SceneComponent);
		}

		ImportActorAsComponent(ImportContext, ActorElement->GetChild(i).ToSharedRef(), InRootActor, UniqueNameProvider);

		if (SceneComponent)
		{
			ImportContext.Hierarchy.Pop();
		}
	}
}

void FDatasmithImporter::FinalizeActors( FDatasmithImportContext& ImportContext, TMap< UObject*, UObject* >* AssetReferencesToRemap )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithImporter::FinalizeActors);

	if ( !ImportContext.bUserCancelled )
	{
		// Ensure a proper setup for the finalize of the actors
		auto& ImportSceneActor = ImportContext.ActorsContext.ImportSceneActor;
		TSet< ADatasmithSceneActor* >& FinalSceneActors = ImportContext.ActorsContext.FinalSceneActors;

		if ( !ImportContext.ActorsContext.FinalWorld )
		{
			ImportContext.ActorsContext.FinalWorld = ImportContext.ActorsContext.ImportWorld;
		}
		else if ( !ImportContext.bIsAReimport && ImportSceneActor )
		{
				//Create a new datasmith scene actor in the final level
				FActorSpawnParameters SpawnParameters;
				SpawnParameters.Template = ImportSceneActor;
				ADatasmithSceneActor* DestinationSceneActor = ImportContext.ActorsContext.FinalWorld->SpawnActor< ADatasmithSceneActor >(SpawnParameters);

				// Name new destination ADatasmithSceneActor to the DatasmithScene's name
				DestinationSceneActor->SetActorLabel( ImportContext.Scene->GetName() );
				DestinationSceneActor->MarkPackageDirty();
				DestinationSceneActor->RelatedActors.Reset();

				// Workaround for UE-94255. We should be able to remove this when UE-76028 is fixed
				TArray<UObject*> SubObjects;
				GetObjectsWithOuter( DestinationSceneActor, SubObjects );
				for ( UObject* SubObject : SubObjects )
				{
					if ( UAssetUserData* AssetUserData = Cast<UAssetUserData>( SubObject ) )
					{
						AssetUserData->SetFlags( RF_Public );
					}
				}

				FinalSceneActors.Empty( 1 );
				FinalSceneActors.Add( DestinationSceneActor );
		}

		if( FinalSceneActors.Num() == 0 )
		{
			if ( ImportContext.bIsAReimport )
			{
				FinalSceneActors.Append( FDatasmithImporterUtils::FindSceneActors( ImportContext.ActorsContext.FinalWorld, ImportContext.SceneAsset ) );
				FinalSceneActors.Remove( ImportSceneActor );
			}
			else
			{
				FinalSceneActors.Add( ImportSceneActor );
			}
		}

		for ( AActor* Actor : FinalSceneActors )
		{
			check(Actor->GetWorld() == ImportContext.ActorsContext.FinalWorld);
		}


		// Do the finalization for each actor from each FinalSceneActor
		TMap< FSoftObjectPath, FSoftObjectPath > RenamedActorsMap;
		TSet< FName > LayersUsedByActors;
		const bool bShouldSpawnNonExistingActors = !ImportContext.bIsAReimport || ImportContext.Options->ReimportOptions.bRespawnDeletedActors;

		TArray<uint8> ReusableBuffer;
		for ( ADatasmithSceneActor* DestinationSceneActor : FinalSceneActors )
		{
			if ( !DestinationSceneActor )
			{
				continue;
			}

			if ( ImportSceneActor->Scene != DestinationSceneActor->Scene  || DestinationSceneActor->GetLevel() != ImportContext.ActorsContext.FinalWorld->GetCurrentLevel())
			{
				continue;
			}

			// In order to allow modification on components owned by DestinationSceneActor, unregister all of them
			DestinationSceneActor->UnregisterAllComponents( /* bForReregister = */true);

			ImportContext.ActorsContext.CurrentTargetedScene = DestinationSceneActor;

			if ( ImportSceneActor != DestinationSceneActor )
			{
				// Before we delete the non imported actors, remove the old actor labels from the unique name provider
				// as we don't care if the source labels clash with labels from actors that will be deleted or replaced on reimport
				for (const TPair< FName, TSoftObjectPtr< AActor > >& ActorPair : DestinationSceneActor->RelatedActors)
				{
					if ( AActor* DestActor = ActorPair.Value.Get() )
					{
						ImportContext.ActorsContext.UniqueNameProvider.RemoveExistingName(DestActor->GetActorLabel());
					}
				}

				FDatasmithImporterUtils::DeleteNonImportedDatasmithElementFromSceneActor( *ImportSceneActor, *DestinationSceneActor, ImportContext.ActorsContext.NonImportedDatasmithActors );


				// Update destination actor's metadata
				if (IInterface_AssetUserData* SourceAssetUserData = Cast<IInterface_AssetUserData>(ImportSceneActor->GetRootComponent()))
				{
					if (UDatasmithAssetUserData* SourceDatasmithUserData = SourceAssetUserData->GetAssetUserData<UDatasmithAssetUserData>())
					{
						USceneComponent* SceneComponent = DestinationSceneActor->GetRootComponent();

						if (IInterface_AssetUserData* AssetUserData = Cast<IInterface_AssetUserData>(SceneComponent))
						{
							UDatasmithAssetUserData* DatasmithUserData = AssetUserData->GetAssetUserData<UDatasmithAssetUserData>();

							if (!DatasmithUserData)
							{
								DatasmithUserData = NewObject<UDatasmithAssetUserData>(SceneComponent, NAME_None, RF_Public | RF_Transactional);
								AssetUserData->AddAssetUserData(DatasmithUserData);
							}

							DatasmithUserData->MetaData.Append(SourceDatasmithUserData->MetaData);
						}
					}
				}
			}

			// Add Actor info to the remap info
			TMap< UObject*, UObject* > PerSceneActorReferencesToRemap = AssetReferencesToRemap ? *AssetReferencesToRemap : TMap< UObject*, UObject* >();
			PerSceneActorReferencesToRemap.Add( ImportSceneActor ) = DestinationSceneActor;
			PerSceneActorReferencesToRemap.Add( ImportSceneActor->GetRootComponent() ) = DestinationSceneActor->GetRootComponent();

			// #ueent_todo order of actors matters for ReferencesFix + re-parenting
			for ( const TPair< FName, TSoftObjectPtr< AActor > >& SourceActorPair : ImportSceneActor->RelatedActors )
			{
				AActor* SourceActor = SourceActorPair.Value.Get();
				if ( SourceActor == nullptr )
				{
					continue;
				}

				const bool bActorIsRelatedToDestinationScene = DestinationSceneActor->RelatedActors.Contains( SourceActorPair.Key );
				TSoftObjectPtr< AActor >& ExistingActorPtr = DestinationSceneActor->RelatedActors.FindOrAdd( SourceActorPair.Key );
				const bool bShouldFinalizeActor = bShouldSpawnNonExistingActors || !bActorIsRelatedToDestinationScene || ( ExistingActorPtr.Get() && !ExistingActorPtr.Get()->IsPendingKillPending() );

				if ( bShouldFinalizeActor )
				{
					// Remember the original source path as FinalizeActor may set SourceActor's label, which apparently can also change its Name and package path
					FSoftObjectPath OriginalSourcePath = FSoftObjectPath(SourceActor);
					AActor* DestinationActor = FinalizeActor( ImportContext, *SourceActor, ExistingActorPtr.Get(), PerSceneActorReferencesToRemap, ReusableBuffer );
					RenamedActorsMap.Add(OriginalSourcePath, FSoftObjectPath(DestinationActor));
					LayersUsedByActors.Append(DestinationActor->Layers);
					ExistingActorPtr = DestinationActor;
				}
			}

			for (  const TPair< FName, TSoftObjectPtr< AActor > >& DestinationActorPair : DestinationSceneActor->RelatedActors )
			{
				if ( AActor* Actor = DestinationActorPair.Value.Get() )
				{
					FDatasmithImporterImpl::FixReferencesForObject( Actor, PerSceneActorReferencesToRemap );
				}
			}

			// Modification is completed, re-register all components owned by DestinationSceneActor
			DestinationSceneActor->RegisterAllComponents();
		}

		// Add the missing layers to the final world
		FDatasmithImporterUtils::AddUniqueLayersToWorld( ImportContext.ActorsContext.FinalWorld, LayersUsedByActors );

		// Fixed the soft object paths that were pointing to our pre-finalized actors.
		TArray< UPackage* > PackagesToFix;

		for ( const TPair< FName, TSoftObjectPtr< ULevelSequence > >& LevelSequencePair : ImportContext.SceneAsset->LevelSequences )
		{
			if (LevelSequencePair.Value)
			{
				PackagesToFix.Add( LevelSequencePair.Value->GetOutermost() );
			}
		}

		for ( const TPair< FName, TSoftObjectPtr< ULevelVariantSets > >& LevelVariantSetsPair : ImportContext.SceneAsset->LevelVariantSets )
		{
			if (LevelVariantSetsPair.Value)
			{
				PackagesToFix.Add( LevelVariantSetsPair.Value->GetOutermost() );
			}
		}

		FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked< FAssetToolsModule >(TEXT("AssetTools"));
		AssetToolsModule.Get().RenameReferencingSoftObjectPaths( PackagesToFix, RenamedActorsMap );
	}

	FDatasmithImporterImpl::DeleteImportSceneActorIfNeeded( ImportContext.ActorsContext );

	// Ensure layer visibility is properly updated for new actors associated with existing layers
	ULayersSubsystem* LayersSubsystem = GEditor->GetEditorSubsystem<ULayersSubsystem>();
	LayersSubsystem->UpdateAllActorsVisibility( false, true);

	GEngine->BroadcastLevelActorListChanged();
}

AActor* FDatasmithImporter::FinalizeActor( FDatasmithImportContext& ImportContext, AActor& SourceActor, AActor* ExistingActor, TMap< UObject*, UObject* >& ReferencesToRemap, TArray<uint8>& ReusableBuffer )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithImporter::FinalizeActor);

	// If the existing actor is not of the same class we destroy it
	if ( ExistingActor && ExistingActor->GetClass() != SourceActor.GetClass() )
	{
		FDatasmithImporterUtils::DeleteActor( *ExistingActor );
		ExistingActor = nullptr;
	}

	TArray< AActor* > Children;
	AActor* DestinationActor = ExistingActor;
	if ( !DestinationActor )
	{
		DestinationActor = ImportContext.ActorsContext.FinalWorld->SpawnActor( SourceActor.GetClass() );
	}
	else
	{
		// Backup hierarchy
		ExistingActor->GetAttachedActors( Children );
	}

	check( DestinationActor );

	{
		// Setup the actor to allow modifications.
		FDatasmithImporterImpl::FScopedFinalizeActorChanges ScopedFinalizedActorChanges(DestinationActor, ImportContext);


		ReferencesToRemap.Add(&SourceActor, DestinationActor);


		TArray< FDatasmithImporterImpl::FMigratedTemplatePairType > MigratedTemplates = FDatasmithImporterImpl::MigrateTemplates(
			&SourceActor,
			ExistingActor,
			&ReferencesToRemap,
			true
			);

		TArray<TPair<USceneComponent&, TArray<FDatasmithImporterImpl::FMigratedTemplatePairType>>> ComponentsToApplyMigratedTemplate;

		FDatasmithImporterImpl::FinalizeComponents( ImportContext, SourceActor, *DestinationActor, ReferencesToRemap, ReusableBuffer, ComponentsToApplyMigratedTemplate );

		FDatasmithImporterImpl::CopyObject( SourceActor, *DestinationActor, ReusableBuffer );

		FDatasmithImporterImpl::PublicizeSubObjects( SourceActor, *DestinationActor, ReferencesToRemap , ReusableBuffer );

		FDatasmithImporterImpl::FixReferencesForObject( DestinationActor, ReferencesToRemap );

		ForEachObjectWithOuter( DestinationActor, [&ReferencesToRemap](UObject* InObject)
			{
				FDatasmithImporterImpl::FixReferencesForObject( InObject, ReferencesToRemap );

				InObject->PostEditImport();
				InObject->PostEditChange();
			});


		for ( TPair<USceneComponent&, TArray<FDatasmithImporterImpl::FMigratedTemplatePairType>>& Pair : ComponentsToApplyMigratedTemplate )
		{
			// Put back the components in a proper state (Unfortunately without this the set relative transform might not work)
			Pair.Key.UpdateComponentToWorld();
			FDatasmithImporterImpl::ApplyMigratedTemplates( Pair.Value, &Pair.Key );
			Pair.Key.PostEditChange();
		}


		// The templates for the actor need to be applied after the components were created.
		FDatasmithImporterImpl::ApplyMigratedTemplates( MigratedTemplates, DestinationActor );
	}

	// Update label to match the source actor's
	DestinationActor->SetActorLabel( ImportContext.ActorsContext.UniqueNameProvider.GenerateUniqueName( SourceActor.GetActorLabel() ) );


	return DestinationActor;
}

void FDatasmithImporter::ImportLevelSequences( FDatasmithImportContext& ImportContext )
{
	const int32 SequencesCount = ImportContext.FilteredScene->GetLevelSequencesCount();
	if ( !ImportContext.Options->BaseOptions.CanIncludeAnimation() || !ImportContext.Options->BaseOptions.bIncludeAnimation || SequencesCount == 0 )
	{
		return;
	}

	TUniquePtr<FScopedSlowTask> ProgressPtr;
	if ( ImportContext.FeedbackContext )
	{
		ProgressPtr = MakeUnique<FScopedSlowTask>( (float)SequencesCount, LOCTEXT("ImportingLevelSequences", "Importing Level Sequences..."), true, *ImportContext.FeedbackContext );
		ProgressPtr->MakeDialog(true);
	}

	// We can only parse a IDatasmithLevelSequenceElement with IDatasmithSubsequenceAnimationElements if their target
	// subsequences' LevelSequenceElement have been parsed. We solve that with a structure we can repeatedly loop over,
	// iteratively resolving all dependencies
	TArray<TSharedPtr<IDatasmithLevelSequenceElement>> SequencesToImport;
	SequencesToImport.Reserve(SequencesCount);
	for ( int32 SequenceIndex = 0; SequenceIndex < SequencesCount && !ImportContext.bUserCancelled; ++SequenceIndex )
	{
		if (FDatasmithImporterImpl::HasUserCancelledTask(ImportContext.FeedbackContext))
		{
			ImportContext.bUserCancelled = true;
		}

		TSharedPtr< IDatasmithLevelSequenceElement > SequenceElement = ImportContext.FilteredScene->GetLevelSequence( SequenceIndex );
		if ( !SequenceElement )
		{
			continue;
		}

		SequencesToImport.Add(SequenceElement);
	}


	FScopedSlowTask* Progress = ProgressPtr.Get();

	// If the scene is ok we will do at most HardLoopCounter passes
	int32 HardLoopCounter = SequencesToImport.Num();
	int32 NumImported = 0;
	int32 LastNumImported = -1;
	for (int32 IterationCounter = 0; IterationCounter < HardLoopCounter && !ImportContext.bUserCancelled; ++IterationCounter)
	{
		// Scan remaining sequences and import the ones we can, removing from this array
		for ( int32 SequenceIndex = SequencesToImport.Num() - 1; SequenceIndex >= 0 && !ImportContext.bUserCancelled; --SequenceIndex )
		{
			if (FDatasmithImporterImpl::HasUserCancelledTask(ImportContext.FeedbackContext))
			{
				ImportContext.bUserCancelled = true;
			}

			TSharedPtr<IDatasmithLevelSequenceElement>& SequenceElement = SequencesToImport[SequenceIndex];

			if (!FDatasmithLevelSequenceImporter::CanImportLevelSequence(SequenceElement.ToSharedRef(), ImportContext))
			{
				continue;
			}

			ULevelSequence* ExistingLevelSequence = nullptr;
			if ( ImportContext.SceneAsset )
			{
				TSoftObjectPtr< ULevelSequence >* ExistingLevelSequencePtr = ImportContext.SceneAsset->LevelSequences.Find( SequenceElement->GetName() );

				if ( ExistingLevelSequencePtr )
				{
					ExistingLevelSequence = ExistingLevelSequencePtr->LoadSynchronous();
				}
			}

			FString SequenceName = ObjectTools::SanitizeObjectName(SequenceElement->GetName());
			FDatasmithImporterImpl::ReportProgress( Progress, 1.f, FText::FromString( FString::Printf(TEXT("Importing level sequence %d/%d (%s) ..."), NumImported + 1, HardLoopCounter, *SequenceName ) ) );

			if (ImportContext.SceneTranslator)
			{
				FDatasmithLevelSequencePayload LevelSequencePayload;
				if (!ImportContext.SceneTranslator->LoadLevelSequence(SequenceElement.ToSharedRef(), LevelSequencePayload))
				{
					FString ErrorMessage = LOCTEXT("FailedToLoad", "Failed to load some animation sequences: ").ToString() + SequenceElement->GetName() + TEXT("\n");
					ImportContext.LogError(FText::FromString(ErrorMessage));
				}
			}

			auto& ImportedLevelSequence = ImportContext.ImportedLevelSequences.FindOrAdd( SequenceElement.ToSharedRef() );
			ImportedLevelSequence = FDatasmithLevelSequenceImporter::ImportLevelSequence( SequenceElement.ToSharedRef(), ImportContext, ExistingLevelSequence );

			SequencesToImport.RemoveAt(SequenceIndex);
			++NumImported;
		}

		// If we do a full loop and haven't managed to parse at least one IDatasmithLevelSequenceElement, we'll assume something
		// went wrong and step out.
		if (NumImported == LastNumImported)
		{
			break;
		}
		LastNumImported = NumImported;
	}

	if (SequencesToImport.Num() > 0)
	{
		FString ErrorMessage = LOCTEXT("FailedToImport", "Failed to import some animation sequences:\n").ToString();
		for (TSharedPtr<IDatasmithLevelSequenceElement>& Sequence: SequencesToImport)
		{
			ErrorMessage += FString(TEXT("\t")) + Sequence->GetName() + TEXT("\n");
		}
		ImportContext.LogError(FText::FromString(ErrorMessage));
	}

	// Assets have been imported and moved out of their import packages, clear them so that we don't look for them in there anymore
	ImportContext.AssetsContext.LevelSequencesImportPackage.Reset();
}

ULevelSequence* FDatasmithImporter::FinalizeLevelSequence( ULevelSequence* SourceLevelSequence, const TCHAR* AnimationsFolderPath, ULevelSequence* ExistingLevelSequence )
{
	return Cast< ULevelSequence >( FDatasmithImporterImpl::PublicizeAsset( SourceLevelSequence, AnimationsFolderPath, ExistingLevelSequence ) );
}

void FDatasmithImporter::ImportLevelVariantSets( FDatasmithImportContext& ImportContext )
{
	const int32 LevelVariantSetsCount = ImportContext.FilteredScene->GetLevelVariantSetsCount();
	if ( LevelVariantSetsCount == 0 )
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithImporter::ImportLevelVariantSets);

	TUniquePtr<FScopedSlowTask> ProgressPtr;
	if ( ImportContext.FeedbackContext )
	{
		ProgressPtr = MakeUnique<FScopedSlowTask>( (float)LevelVariantSetsCount, LOCTEXT("ImportingLevelVariantSets", "Importing Level Variant Sets..."), true, *ImportContext.FeedbackContext );
		ProgressPtr->MakeDialog(true);
	}

	FScopedSlowTask* Progress = ProgressPtr.Get();

	for ( int32 LevelVariantSetIndex = 0; LevelVariantSetIndex < LevelVariantSetsCount && !ImportContext.bUserCancelled; ++LevelVariantSetIndex )
	{
		if (FDatasmithImporterImpl::HasUserCancelledTask(ImportContext.FeedbackContext))
		{
			ImportContext.bUserCancelled = true;
		}

		TSharedPtr< IDatasmithLevelVariantSetsElement > LevelVariantSetsElement = ImportContext.FilteredScene->GetLevelVariantSets( LevelVariantSetIndex );
		if ( !LevelVariantSetsElement )
		{
			continue;
		}

		ULevelVariantSets* ExistingLevelVariantSets = nullptr;
		if ( ImportContext.SceneAsset )
		{
			TSoftObjectPtr< ULevelVariantSets >* ExistingLevelVariantSetsPtr = ImportContext.SceneAsset->LevelVariantSets.Find( LevelVariantSetsElement->GetName() );

			if ( ExistingLevelVariantSetsPtr )
			{
				ExistingLevelVariantSets = ExistingLevelVariantSetsPtr->LoadSynchronous();
			}
		}


		FString LevelVariantSetsName = ObjectTools::SanitizeObjectName(LevelVariantSetsElement->GetName());
		FDatasmithImporterImpl::ReportProgress(Progress, 1.f, FText::FromString(FString::Printf(TEXT("Importing level variant sets %d/%d (%s) ..."), LevelVariantSetIndex + 1, LevelVariantSetsCount, *LevelVariantSetsName)));

		auto& ImportedLevelVariantSets = ImportContext.ImportedLevelVariantSets.FindOrAdd( LevelVariantSetsElement.ToSharedRef() );
		ImportedLevelVariantSets = FDatasmithLevelVariantSetsImporter::ImportLevelVariantSets( LevelVariantSetsElement.ToSharedRef(), ImportContext, ExistingLevelVariantSets );
	}

	// Assets have been imported and moved out of their import packages, clear them so that we don't look for them in there anymore
	ImportContext.AssetsContext.LevelVariantSetsImportPackage.Reset();
}

ULevelVariantSets* FDatasmithImporter::FinalizeLevelVariantSets( ULevelVariantSets* SourceLevelVariantSets, const TCHAR* VariantsFolderPath, ULevelVariantSets* ExistingLevelVariantSets )
{
	return Cast< ULevelVariantSets >( FDatasmithImporterImpl::PublicizeAsset( SourceLevelVariantSets, VariantsFolderPath, ExistingLevelVariantSets ) );
}

void FDatasmithImporter::ImportMetaDataForObject(FDatasmithImportContext& ImportContext, const TSharedRef<IDatasmithElement>& DatasmithElement, UObject* Object)
{
	if ( !Object )
	{
		return;
	}

	UDatasmithAssetUserData::FMetaDataContainer MetaData;

	// Add Datasmith meta data
	MetaData.Add( UDatasmithAssetUserData::UniqueIdMetaDataKey, DatasmithElement->GetName() );

	// Check if there's metadata associated with the given element
	const TSharedPtr<IDatasmithMetaDataElement>& MetaDataElement = ImportContext.Scene->GetMetaData( DatasmithElement );
	if ( MetaDataElement.IsValid() )
	{
		int32 PropertiesCount = MetaDataElement->GetPropertiesCount();
		MetaData.Reserve( PropertiesCount );
		for ( int32 PropertyIndex = 0; PropertyIndex < PropertiesCount; ++PropertyIndex )
		{
			const TSharedPtr<IDatasmithKeyValueProperty>& Property = MetaDataElement->GetProperty( PropertyIndex );
			MetaData.Add( Property->GetName(), Property->GetValue() );
		}

		MetaData.KeySort(FNameLexicalLess());
	}

	if ( MetaData.Num() > 0 )
	{
		// For AActor, the interface is actually implemented by the ActorComponent
		AActor* Actor = Cast<AActor>( Object );
		if ( Actor )
		{
			UActorComponent* ActorComponent = Actor->GetRootComponent();
			if ( ActorComponent )
			{
				Object = ActorComponent;
			}
		}

		if ( Object->GetClass()->ImplementsInterface( UInterface_AssetUserData::StaticClass() ) )
		{
			IInterface_AssetUserData* AssetUserData = Cast< IInterface_AssetUserData >( Object );

			UDatasmithAssetUserData* DatasmithUserData = AssetUserData->GetAssetUserData< UDatasmithAssetUserData >();

			if ( !DatasmithUserData )
			{
				DatasmithUserData = NewObject<UDatasmithAssetUserData>( Object, NAME_None, RF_Public | RF_Transactional );
				AssetUserData->AddAssetUserData( DatasmithUserData );
			}

			DatasmithUserData->MetaData = MoveTemp( MetaData );
		}
	}
}

void FDatasmithImporter::FilterElementsToImport( FDatasmithImportContext& ImportContext )
{
	ImportContext.FilteredScene = FDatasmithSceneFactory::DuplicateScene( ImportContext.Scene.ToSharedRef() );

	FDatasmithSceneUtils::CleanUpScene(ImportContext.FilteredScene.ToSharedRef(), false);

	// Filter meshes to import by consulting the AssetRegistry to see if that asset already exist
	// or if it changed at all, if deemed identical filter the mesh out of the current import
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();

	// No Scene asset yet, all assets of the scene must be imported
	if (!ImportContext.SceneAsset || !ImportContext.SceneAsset->AssetImportData)
	{
		return;
	}

	auto ElementNeedsReimport = [&](const FString& FullyQualifiedName, TSharedRef<IDatasmithElement> Element, const FString& SourcePath) -> bool
	{
		const FAssetData AssetData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(FullyQualifiedName));
		const FAssetDataTagMapSharedView::FFindTagResult ImportDataStr = AssetData.TagsAndValues.FindTag(TEXT("AssetImportData"));
		FString CurrentRelativeFileName;

		// Filter out Element only if it has valid and up to date import info
		bool bImportThisElement = !ImportDataStr.IsSet();
		if (!bImportThisElement)
		{
			TOptional<FAssetImportInfo> AssetImportInfo = FAssetImportInfo::FromJson(ImportDataStr.GetValue());
			if (AssetImportInfo && AssetImportInfo->SourceFiles.Num() > 0)
			{
				FAssetImportInfo::FSourceFile ExistingSourceFile = AssetImportInfo->SourceFiles[0];
				FMD5Hash ElementHash = Element->CalculateElementHash(false);
				bImportThisElement = ExistingSourceFile.FileHash != ElementHash;
				CurrentRelativeFileName = ExistingSourceFile.RelativeFilename;
			}
		}

		// Sync import data now for skipped elements
		if (!bImportThisElement && !SourcePath.IsEmpty())
		{
			FString ImportRelativeFileName = UAssetImportData::SanitizeImportFilename(SourcePath, AssetData.PackageName.ToString());
			if (CurrentRelativeFileName != ImportRelativeFileName)
			{
				UObject* Asset = AssetData.GetAsset();
				if (UAssetImportData* AssetImportData = Datasmith::GetAssetImportData(Asset))
				{
					AssetImportData->UpdateFilenameOnly(ImportRelativeFileName);
				}
			}
		}

		return bImportThisElement;
	};

	// Meshes part
	ImportContext.FilteredScene->EmptyMeshes();
	const bool bDifferentStaticMeshOptions = ImportContext.Options->BaseOptions.StaticMeshOptions != ImportContext.SceneAsset->AssetImportData->BaseOptions.StaticMeshOptions;
	const TMap< FName, TSoftObjectPtr< UStaticMesh > >& StaticMeshes = ImportContext.SceneAsset->StaticMeshes;
	for (int32 MeshIndex = 0; MeshIndex < ImportContext.Scene->GetMeshesCount(); ++MeshIndex)
	{
		TSharedRef< IDatasmithMeshElement > MeshElement = ImportContext.Scene->GetMesh( MeshIndex ).ToSharedRef();
		bool bNeedsReimport = true;
		FString AssetName = MeshElement->GetName();

		if ( StaticMeshes.Contains( MeshElement->GetName() ) )
		{
			AssetName = StaticMeshes[ MeshElement->GetName() ].ToString();
			// If we are reimporting with different options we should not try to skip the reimport.
			bNeedsReimport = bDifferentStaticMeshOptions || ElementNeedsReimport(AssetName, MeshElement, ImportContext.Options->FilePath );
		}

		if ( bNeedsReimport )
		{
			ImportContext.FilteredScene->AddMesh( MeshElement );
		}
		// If the mesh element does not need to be re-imported, register its name
		else
		{
			const FAssetData AssetData = AssetRegistry.GetAssetByObjectPath( FSoftObjectPath(AssetName) );
			ImportContext.AssetsContext.StaticMeshNameProvider.AddExistingName( FPaths::GetBaseFilename( AssetData.PackageName.ToString() ) );
		}
	}

	// Texture part
	ImportContext.FilteredScene->EmptyTextures();
	const TMap< FName, TSoftObjectPtr< UTexture > >& Textures = ImportContext.SceneAsset->Textures;
	for (int32 TexIndex = 0; TexIndex < ImportContext.Scene->GetTexturesCount(); ++TexIndex)
	{
		TSharedRef< IDatasmithTextureElement > TextureElement = ImportContext.Scene->GetTexture(TexIndex).ToSharedRef();

		bool bNeedsReimport = true;
		FString AssetName = TextureElement->GetName();
		if ( Textures.Contains( TextureElement->GetName() ) )
		{
			AssetName = Textures[TextureElement->GetName()].ToString();
			bNeedsReimport = ElementNeedsReimport(AssetName, TextureElement, ImportContext.Options->FilePath );
		}

		if ( bNeedsReimport )
		{
			ImportContext.FilteredScene->AddTexture( TextureElement );
		}
		// If the texture element does not need to be re-imported, register its name
		else
		{
			const FAssetData AssetData = AssetRegistry.GetAssetByObjectPath( FSoftObjectPath(AssetName) );
			ImportContext.AssetsContext.TextureNameProvider.AddExistingName( FPaths::GetBaseFilename( AssetData.PackageName.ToString() ) );
		}
	}
}

void FDatasmithImporter::FinalizeImport(FDatasmithImportContext& ImportContext, const TSet<UObject*>& ValidAssets)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithImporter::FinalizeImport);

	const int32 NumImportedObjects = ImportContext.ImportedStaticMeshes.Num() +
PRAGMA_DISABLE_DEPRECATION_WARNINGS
									 ImportContext.ImportedClothes.Num() +       // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
									 ImportContext.ImportedClothPresets.Num() +  //
PRAGMA_ENABLE_DEPRECATION_WARNINGS
									 ImportContext.ImportedTextures.Num() +
									 ImportContext.ImportedMaterialFunctions.Num() +
									 ImportContext.ImportedMaterials.Num() +
									 ImportContext.ImportedLevelSequences.Num() +
									 ImportContext.ImportedLevelVariantSets.Num();
	const int32 NumAssetsToFinalize = ValidAssets.Num() == 0 ? NumImportedObjects : ValidAssets.Num() + ImportContext.ImportedLevelSequences.Num() + ImportContext.ImportedLevelVariantSets.Num();
	const int32 NumStaticMeshToBuild = ImportContext.ImportedStaticMeshes.Num();

	TUniquePtr<FScopedSlowTask> ProgressPtr;

	if ( ImportContext.FeedbackContext )
	{
		ProgressPtr = MakeUnique<FScopedSlowTask>((float)NumAssetsToFinalize + NumStaticMeshToBuild, LOCTEXT("FinalizingAssets", "Finalizing Assets"), true, *ImportContext.FeedbackContext);
		ProgressPtr->MakeDialog(true);
	}

	TMap<UObject*, UObject*> ReferencesToRemap;

	// Array of packages containing templates which are referring to assets as TSoftObjectPtr or FSoftObjectPath
	TArray<UPackage*> PackagesToCheck;

	int32 AssetIndex = 0;

	const FString& RootFolderPath = ImportContext.AssetsContext.RootFolderPath;
	const FString& TransientFolderPath = ImportContext.AssetsContext.TransientFolderPath;

	FScopedSlowTask* Progress = ProgressPtr.Get();

	// Needs to be done in dependencies order (textures -> materials -> static meshes)
	for (const TPair< TSharedRef< IDatasmithTextureElement >, UE::Interchange::FAssetImportResultRef >& ImportedTexturePair : ImportContext.ImportedTextures)
	{
		if (ImportContext.bUserCancelled)
		{
			break;
		}

		ImportedTexturePair.Value->WaitUntilDone();
		UTexture* SourceTexture = Cast<UTexture>(ImportedTexturePair.Value->GetFirstAssetOfClass(UTexture::StaticClass()));

		if (!SourceTexture || (ValidAssets.Num() > 0 && !ValidAssets.Contains(Cast<UObject>(SourceTexture))))
		{
			continue;
		}

		FName TextureId = ImportedTexturePair.Key->GetName();

		FDatasmithImporterImpl::ReportProgress(Progress, 1.f, FText::FromString(FString::Printf(TEXT("Finalizing assets %d/%d (Texture %s) ..."), ++AssetIndex, NumAssetsToFinalize, *SourceTexture->GetName())));

		UTexture* ExistingTexture = ImportContext.SceneAsset ? ImportContext.SceneAsset->Textures.FindOrAdd(TextureId).Get() : nullptr;

		FString SourcePackagePath = SourceTexture->GetOutermost()->GetName();
		FString DestinationPackagePath = SourcePackagePath.Replace(*TransientFolderPath, *RootFolderPath, ESearchCase::CaseSensitive);

		ExistingTexture = FinalizeTexture(SourceTexture, *DestinationPackagePath, ExistingTexture, &ReferencesToRemap);
		if (ImportContext.SceneAsset)
		{
			*(ImportContext.SceneAsset->Textures.Find(TextureId)) = ExistingTexture;
		}
		FDatasmithImporterImpl::CheckAssetPersistenceValidity(ExistingTexture, ImportContext);
	}

	{
		// Destroy render states of all actors to avoid excessive refresh in the renderer while updating materials.
		// Also batch all the material update together to improve performance.
		FMaterialUpdateContext MaterialUpdateContext;

		for (const TPair< TSharedRef< IDatasmithBaseMaterialElement >, UMaterialFunction* >& ImportedMaterialFunctionPair : ImportContext.ImportedMaterialFunctions)
		{
			if (ImportContext.bUserCancelled)
			{
				break;
			}

			UMaterialFunction* SourceMaterialFunction = ImportedMaterialFunctionPair.Value;

			if (!SourceMaterialFunction || (ValidAssets.Num() > 0 && !ValidAssets.Contains(Cast<UObject>(SourceMaterialFunction))))
			{
				continue;
			}

			FName MaterialFunctionId = ImportedMaterialFunctionPair.Key->GetName();
			FDatasmithImporterImpl::ReportProgress(Progress, 1.f, FText::FromString(FString::Printf(TEXT("Finalizing assets %d/%d (Material Function %s) ..."), ++AssetIndex, NumAssetsToFinalize, *SourceMaterialFunction->GetName())));

			UMaterialFunction* ExistingMaterialFunction = ImportContext.SceneAsset ? ImportContext.SceneAsset->MaterialFunctions.FindOrAdd(MaterialFunctionId).Get() : nullptr;

			FString SourcePackagePath = SourceMaterialFunction->GetOutermost()->GetName();
			FString DestinationPackagePath = SourcePackagePath.Replace(*TransientFolderPath, *RootFolderPath, ESearchCase::CaseSensitive);

			ExistingMaterialFunction = FinalizeMaterialFunction(SourceMaterialFunction, *DestinationPackagePath, ExistingMaterialFunction, &ReferencesToRemap);
			if (ImportContext.SceneAsset)
			{
				ImportContext.SceneAsset->MaterialFunctions[MaterialFunctionId] = ExistingMaterialFunction;
			}

			FDatasmithImporterImpl::CheckAssetPersistenceValidity(ExistingMaterialFunction, ImportContext);
		}

		for (const auto& ImportedMaterialPair : ImportContext.ImportedMaterials)
		{
			if (ImportContext.bUserCancelled)
			{
				break;
			}

			UMaterialInterface* SourceMaterialInterface = ImportedMaterialPair.Value;

			if (!SourceMaterialInterface || (ValidAssets.Num() > 0 && !ValidAssets.Contains(Cast<UObject>(SourceMaterialInterface))))
			{
				continue;
			}

			FName MaterialId = ImportedMaterialPair.Key->GetName();

			FDatasmithImporterImpl::ReportProgress(Progress, 1.f, FText::FromString(FString::Printf(TEXT("Finalizing assets %d/%d (Material %s) ..."), ++AssetIndex, NumAssetsToFinalize, *SourceMaterialInterface->GetName())));

			UMaterialInterface* ExistingMaterial = ImportContext.SceneAsset ? ImportContext.SceneAsset->Materials.FindOrAdd(MaterialId).Get() : NULL;

			FString SourcePackagePath = SourceMaterialInterface->GetOutermost()->GetName();
			FString DestinationPackagePath = SourcePackagePath.Replace( *TransientFolderPath, *RootFolderPath, ESearchCase::CaseSensitive );

			if (UMaterial* SourceMaterial = Cast< UMaterial >(SourceMaterialInterface))
			{
				SourceMaterial->UpdateCachedExpressionData();

				TArray<UMaterialFunctionInterface*> FinalizableFunctions;
				for (const FMaterialFunctionInfo& MaterialFunctionInfo : SourceMaterial->GetCachedExpressionData().FunctionInfos)
				{
					if (MaterialFunctionInfo.Function && MaterialFunctionInfo.Function->GetOutermost() == SourceMaterial->GetOutermost())
					{
						FinalizableFunctions.Add(MaterialFunctionInfo.Function);
					}
				}

				for (UMaterialFunctionInterface* Function : FinalizableFunctions)
				{
					FinalizeMaterial(Function, *DestinationPackagePath, *TransientFolderPath, *RootFolderPath, nullptr, &ReferencesToRemap, &MaterialUpdateContext);
				}
			}

			ExistingMaterial = Cast<UMaterialInterface>( FinalizeMaterial(SourceMaterialInterface, *DestinationPackagePath, *TransientFolderPath, *RootFolderPath, ExistingMaterial, &ReferencesToRemap, &MaterialUpdateContext) );
			if (ImportContext.SceneAsset)
			{
				ImportContext.SceneAsset->Materials[MaterialId] = ExistingMaterial;
			}

			// Add material to array of packages to apply soft object path redirection to
			if (ExistingMaterial)
			{
				PackagesToCheck.Add(ExistingMaterial->GetOutermost());
				FDatasmithImporterImpl::CheckAssetPersistenceValidity(ExistingMaterial, ImportContext);
			}
		}
	}

	FDatasmithImporterImpl::ConvertUnsupportedVirtualTexture(ImportContext, ImportContext.AssetsContext.VirtualTexturesToConvert, ReferencesToRemap);

	// Sometimes, the data is invalid and we get the same UStaticMesh multiple times
	TSet< UStaticMesh* > StaticMeshes;
	for (auto& ImportedStaticMeshPair : ImportContext.ImportedStaticMeshes)
	{
		if (ImportContext.bUserCancelled)
		{
			break;
		}

		UStaticMesh* SourceStaticMesh = ImportedStaticMeshPair.Value;

		if (!SourceStaticMesh || (ValidAssets.Num() > 0 && !ValidAssets.Contains(Cast<UObject>(SourceStaticMesh))))
		{
			continue;
		}

		FName StaticMeshId = ImportedStaticMeshPair.Key->GetName();

		FDatasmithImporterImpl::ReportProgress(Progress, 1.f, FText::FromString(FString::Printf(TEXT("Finalizing assets %d/%d (Static Mesh %s) ..."), ++AssetIndex, NumAssetsToFinalize, *SourceStaticMesh->GetName())));

		UStaticMesh* ExistingStaticMesh = ImportContext.SceneAsset ? ImportContext.SceneAsset->StaticMeshes.FindOrAdd(StaticMeshId).Get() : nullptr;

		FString SourcePackagePath = SourceStaticMesh->GetOutermost()->GetName();
		FString DestinationPackagePath = SourcePackagePath.Replace( *TransientFolderPath, *RootFolderPath, ESearchCase::CaseSensitive );

		ExistingStaticMesh = FinalizeStaticMesh(SourceStaticMesh, *DestinationPackagePath, ExistingStaticMesh, &ReferencesToRemap, false);
		if (ImportContext.SceneAsset)
		{
			ImportContext.SceneAsset->StaticMeshes[StaticMeshId] = ExistingStaticMesh;
		}
		FDatasmithImporterImpl::CheckAssetPersistenceValidity(ExistingStaticMesh, ImportContext);

		ImportedStaticMeshPair.Value = ExistingStaticMesh;
		StaticMeshes.Add(ExistingStaticMesh);
	}

	int32 StaticMeshIndex = 0;
	auto ProgressFunction = [&](UStaticMesh* StaticMesh)
	{
		FDatasmithImporterImpl::ReportProgress(Progress, 1.f, FText::FromString(FString::Printf(TEXT("Building Static Mesh %d/%d (%s) ..."), ++StaticMeshIndex, StaticMeshes.Num(), *StaticMesh->GetName())));
		return !ImportContext.bUserCancelled;
	};

	FDatasmithStaticMeshImporter::BuildStaticMeshes(StaticMeshes.Array(), ProgressFunction);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	for (auto& SourcePresetObj : ImportContext.ImportedClothPresets)  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	{
		if (!SourcePresetObj || (ValidAssets.Num() > 0 && !ValidAssets.Contains(SourcePresetObj)))
		{
			continue;
		}

		FDatasmithImporterImpl::ReportProgress(Progress, 1.f, FText::FromString(FString::Printf(TEXT("Finalizing assets %d/%d (Cloth Preset %s) ..."), ++AssetIndex, NumAssetsToFinalize, *SourcePresetObj->GetName())));

		UObject* ExistingPreset = nullptr; // Cloth = ImportContext.SceneAsset ? ImportContext.SceneAsset->Clothes.FindOrAdd(StaticMeshId).Get() : nullptr;

		FString SourcePackagePath = SourcePresetObj->GetOutermost()->GetName();
		FString DestinationPackagePath = SourcePackagePath.Replace( *TransientFolderPath, *RootFolderPath, ESearchCase::CaseSensitive );

		UObject* FinalizedPreset = FDatasmithImporterImpl::FinalizeAsset(SourcePresetObj, *DestinationPackagePath, ExistingPreset, &ReferencesToRemap);

		FDatasmithImporterImpl::CheckAssetPersistenceValidity(FinalizedPreset, ImportContext);

		SourcePresetObj = FinalizedPreset;
	}

	for (auto& ImportedPair : ImportContext.ImportedClothes)  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	{
		UObject* SourceClothObj = ImportedPair.Value;

		if (!SourceClothObj || (ValidAssets.Num() > 0 && !ValidAssets.Contains(SourceClothObj)))
		{
			continue;
		}

		FDatasmithImporterImpl::ReportProgress(Progress, 1.f, FText::FromString(FString::Printf(TEXT("Finalizing assets %d/%d (Cloth %s) ..."), ++AssetIndex, NumAssetsToFinalize, *SourceClothObj->GetName())));

		FName ClothId = ImportedPair.Key->GetName();
		UObject* ExistingCloth = ImportContext.SceneAsset ? ImportContext.SceneAsset->Clothes.FindOrAdd(ClothId).Get() : nullptr;

		FString SourcePackagePath = SourceClothObj->GetOutermost()->GetName();
		FString DestinationPackagePath = SourcePackagePath.Replace( *TransientFolderPath, *RootFolderPath, ESearchCase::CaseSensitive );

		UObject* FinalizedCloth = FinalizeCloth(SourceClothObj, *DestinationPackagePath, ExistingCloth, &ReferencesToRemap);
		if (ImportContext.SceneAsset)
		{
			ImportContext.SceneAsset->Clothes[ClothId] = FinalizedCloth;
		}
		FDatasmithImporterImpl::CheckAssetPersistenceValidity(FinalizedCloth, ImportContext);

		ImportedPair.Value = FinalizedCloth;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	for (const auto& ImportedLevelSequencePair : ImportContext.ImportedLevelSequences)
	{
		if (ImportContext.bUserCancelled)
		{
			break;
		}

		ULevelSequence* SourceLevelSequence = ImportedLevelSequencePair.Value;

		if (!SourceLevelSequence)
		{
			continue;
		}

		FName LevelSequenceId = ImportedLevelSequencePair.Key->GetName();

		FDatasmithImporterImpl::ReportProgress(Progress, 1.f, FText::FromString(FString::Printf(TEXT("Finalizing assets %d/%d (Level Sequence %s) ..."), ++AssetIndex, NumAssetsToFinalize, *SourceLevelSequence->GetName())));

		ULevelSequence* ExistingLevelSequence = ImportContext.SceneAsset ? ImportContext.SceneAsset->LevelSequences.FindOrAdd(LevelSequenceId).Get() : nullptr;

		FString SourcePackagePath = SourceLevelSequence->GetOutermost()->GetName();
		FString DestinationPackagePath = SourcePackagePath.Replace( *TransientFolderPath, *RootFolderPath, ESearchCase::CaseSensitive );

		ExistingLevelSequence = FinalizeLevelSequence(SourceLevelSequence, *DestinationPackagePath, ExistingLevelSequence);
		if (ImportContext.SceneAsset)
		{
			ImportContext.SceneAsset->LevelSequences[LevelSequenceId] = ExistingLevelSequence;
			ImportContext.SceneAsset->RegisterPreWorldRenameCallback();
		}
	}

	for (const auto& ImportedLevelVariantSetsPair : ImportContext.ImportedLevelVariantSets)
	{
		if (ImportContext.bUserCancelled)
		{
			break;
		}

		ULevelVariantSets* SourceLevelVariantSets = ImportedLevelVariantSetsPair.Value;

		if (!SourceLevelVariantSets)
		{
			continue;
		}

		FName LevelVariantSetsId = ImportedLevelVariantSetsPair.Key->GetName();

		FDatasmithImporterImpl::ReportProgress(Progress, 1.f, FText::FromString(FString::Printf(TEXT("Finalizing assets %d/%d (Level Variant Sets %s) ..."), ++AssetIndex, NumAssetsToFinalize, *SourceLevelVariantSets->GetName())));

		ULevelVariantSets* ExistingLevelVariantSets = ImportContext.SceneAsset ? ImportContext.SceneAsset->LevelVariantSets.FindOrAdd(LevelVariantSetsId).Get() : nullptr;

		FString SourcePackagePath = SourceLevelVariantSets->GetOutermost()->GetName();
		FString DestinationPackagePath = SourcePackagePath.Replace( *TransientFolderPath, *RootFolderPath, ESearchCase::CaseSensitive );

		ExistingLevelVariantSets = FinalizeLevelVariantSets(SourceLevelVariantSets, *DestinationPackagePath, ExistingLevelVariantSets);

		if (ImportContext.SceneAsset)
		{
			ImportContext.SceneAsset->LevelVariantSets[LevelVariantSetsId] = ExistingLevelVariantSets;
			ImportContext.SceneAsset->RegisterPreWorldRenameCallback();
		}
	}

	// Apply soft object path redirection to identified packages
	if (PackagesToCheck.Num() > 0 && ReferencesToRemap.Num() > 0)
	{
		TMap<FSoftObjectPath, FSoftObjectPath> AssetRedirectorMap;

		for ( const TPair< UObject*, UObject* >& ReferenceToRemap : ReferencesToRemap )
		{
			AssetRedirectorMap.Emplace( ReferenceToRemap.Key ) = ReferenceToRemap.Value;
		}

		IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
		AssetTools.RenameReferencingSoftObjectPaths(PackagesToCheck, AssetRedirectorMap);
	}

	if ( ImportContext.ShouldImportActors() )
	{
		FinalizeActors( ImportContext, &ReferencesToRemap );
	}

	// Everything has been finalized, make sure the UDatasmithScene is set to dirty
	if ( ImportContext.SceneAsset )
	{
		ImportContext.SceneAsset->MarkPackageDirty();
	}

	FGlobalComponentReregisterContext RecreateComponents;

	// Flush the transaction buffer (eg. avoid corrupted hierarchies after repeated undo actions)
	// This is an aggressive workaround while we don't properly support undo history.
	GEditor->ResetTransaction(LOCTEXT("Reset Transaction Buffer", "Datasmith Import Finalization"));
}

#undef LOCTEXT_NAMESPACE
