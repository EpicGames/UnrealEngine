// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithImporterHelper.h"
#include "DatasmithImportFactory.h"
#include "Utility/DatasmithImporterUtils.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "BusyCursor.h"
#include "ContentBrowserModule.h"
#include "DesktopPlatformModule.h"
#include "Dialogs/DlgPickPath.h"
#include "Editor.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorDirectories.h"
#include "ExternalSource.h"
#include "ExternalSourceModule.h"
#include "Factories/Factory.h"
#include "FileHelpers.h"
#include "Framework/Application/SlateApplication.h"
#include "IContentBrowserSingleton.h"
#include "ISourceControlModule.h"
#include "IUriManager.h"
#include "Logging/LogMacros.h"
#include "Logging/MessageLog.h"
#include "Misc/MessageDialog.h"
#include "ObjectTools.h"
#include "SourceUri.h"
#include "UnrealEdGlobals.h"

#define LOCTEXT_NAMESPACE "DatasmithImporter"

namespace DatasmithImporterHelperInternal
{
	// Patterned on UAssetToolsImpl::ImportAssets/ImportAssetsInternal, but simplified to handle only the given Factory
	TArray<UObject*> ImportAssets(const TArray<TSharedRef<UE::DatasmithImporter::FExternalSource>>& Sources, const FString& DestinationPath, UFactory* Factory)
	{
		TArray<UObject*> ReturnObjects;

		if (Sources.Num() > 0)
		{
			const bool bAutomatedImport = Factory->IsAutomatedImport();

			// Reset the 'Do you want to overwrite the existing object?' Yes to All / No to All prompt, to make sure the
			// user gets a chance to select something when the factory is first used during this import
			Factory->ResetState();

			bool bImportSucceeded = false;
			bool bImportWasCancelled = false;

			// Make sure the incoming path is correct
			FString CorrectPath = DestinationPath;
			CorrectPath.ReplaceInline(TEXT("\\"), TEXT("/"), ESearchCase::CaseSensitive);
			if (CorrectPath[CorrectPath.Len() - 1] == TEXT('/'))
			{
				CorrectPath.RemoveAt(CorrectPath.Len() - 1);
			}

			for (const TSharedRef<UE::DatasmithImporter::FExternalSource>& Source : Sources)
			{
				FString Name = ObjectTools::SanitizeObjectName(Source->GetSourceName());
				FString PackageName = ObjectTools::SanitizeInvalidChars(FPaths::Combine( *CorrectPath, *Name ), INVALID_LONGPACKAGE_CHARACTERS);

				TStrongObjectPtr<UPackage> Pkg(CreatePackage(*PackageName));
				if (!ensure(Pkg))
				{
					// Failed to create the package to hold this asset for some reason
					continue;
				}

				UClass* ImportAssetType = Factory->ResolveSupportedClass();
				UObject* Result;
				if (UDatasmithImportFactory* DatasmithFactory = Cast<UDatasmithImportFactory>(Factory))
				{
					// Only UDatasmithImportFactory can directly import FExternalSource objects.
					Result = DatasmithFactory->CreateFromExternalSource(ImportAssetType, Pkg.Get(), FName(*Name), RF_Public | RF_Standalone | RF_Transactional, Source, nullptr, GWarn, bImportWasCancelled);
				}
				else
				{
					FString FilePath = Source->GetFallbackFilepath();
					Result = Factory->ImportObject(ImportAssetType, Pkg.Get(), FName(*Name), RF_Public | RF_Standalone | RF_Transactional, FilePath, nullptr, bImportWasCancelled);
				}

				// Do not report any error if the operation was canceled.
				if (!bImportWasCancelled)
				{
					if (Result)
					{
						ReturnObjects.Add(Result);

						// Notify the asset registry
						FAssetRegistryModule::AssetCreated(Result);
						GEditor->BroadcastObjectReimported(Result);

						bImportSucceeded = true;
					}
					else
					{
						FString SourceLocation = Source->GetSourceUri().ToString();
						const FText Message = FText::Format(LOCTEXT("ImportFailed_Generic", "Failed to import '{0}'. Failed to create asset '{1}'.\nPlease see Output Log for details."), FText::FromString(SourceLocation), FText::FromString(Name));
						if (!bAutomatedImport)
						{
							FMessageDialog::Open(EAppMsgType::Ok, Message);
						}
						UE_LOGF(LogDatasmithImport, Warning, "%ls", *Message.ToString());
					}
				}
			}

			// Sync content browser to the newly created assets
			if (ReturnObjects.Num())
			{
				FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
				ContentBrowserModule.Get().SyncBrowserToAssets(ReturnObjects, /*bAllowLockedBrowsers=*/true);
			}
		}
		return ReturnObjects;
	}

	// Patterned on FEditorFileUtils::Import, but using the given Factory instead of deducing the factory from the file extension
	void Import(const TArray<TSharedRef<UE::DatasmithImporter::FExternalSource>>& InExternalSources, UFactory* Factory)
	{
		const FScopedBusyCursor BusyCursor;

		if (Factory)
		{
			static FString Path = "/Game/"; // Trailing '/' is needed to set the default path

			//Ask the user for the root path where they want any content to be placed
			{
				TSharedRef<SDlgPickPath> PickContentPathDlg =
					SNew(SDlgPickPath)
					.Title(LOCTEXT("ChooseImportRootContentPath", "Choose Location for importing the Datasmith content"))
					.DefaultPath(FText::FromString(Path));

				if (PickContentPathDlg->ShowModal() == EAppReturnType::Cancel)
				{
					return;
				}

				Path = PickContentPathDlg->GetPath().ToString() + "/";
			}

			ImportAssets(InExternalSources, Path, Factory);
		}

		if(GUnrealEd != nullptr)
		{
			// We just finished a potentially long operation and we don't want auto save triggered as soon as we finish the import
			GUnrealEd->GetPackageAutoSaver().ResetAutoSaveTimer();

			GUnrealEd->RedrawLevelEditingViewports();
		}


		FEditorDelegates::RefreshAllBrowsers.Broadcast();
	}

	// FileDialogHelpers::OpenFiles, which should be exposed in the header file
	bool OpenFiles(const FString& Title, const FString& FileTypes, FString& InOutLastPath, EFileDialogFlags::Type DialogMode, TArray<FString>& OutOpenFilenames)
	{
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
		bool bOpened = false;
		if (DesktopPlatform)
		{
			bOpened = DesktopPlatform->OpenFileDialog(
				FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
				Title,
				InOutLastPath,
				TEXT(""),
				FileTypes,
				DialogMode,
				OutOpenFilenames
			);
		}

		bOpened = (OutOpenFilenames.Num() > 0);

		if (bOpened)
		{
			// User successfully chose a file; remember the path for the next time the dialog opens.
			InOutLastPath = OutOpenFilenames[0];
		}

		return bOpened;
	}
}

// Patterned on FEditorFileUtils::Import(), but using the given Factory instead of deducing the factory from the file extension
void FDatasmithImporterHelper::ImportInternal(UFactory* FactoryCDO)
{
	using namespace UE::DatasmithImporter;

	if (!FactoryCDO)
	{
		return;
	}

	TStrongObjectPtr<UFactory> Factory(NewObject< UFactory >((UObject*)GetTransientPackage(), FactoryCDO->GetClass()));
	Factory->ConfigureProperties();
	TArray<FString> OpenedFiles;
	FString DefaultLocation(FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_IMPORT));

	if (DatasmithImporterHelperInternal::OpenFiles(LOCTEXT("ImportDatasmithTitle", "Import Datasmith").ToString(), GetFilterStringInternal(Factory.Get()), DefaultLocation, EFileDialogFlags::Multiple, OpenedFiles))
	{
		TArray<TSharedRef<FExternalSource>> Sources;
		Sources.Reserve(OpenedFiles.Num());
		IUriManager& UriManager = IExternalSourceModule::Get().GetManager();

		for (const FString& OpenedFile : OpenedFiles)
		{
			FSourceUri SourceUri = FSourceUri::FromFilePath(OpenedFile);
			if (TSharedPtr<FExternalSource> FileExternalSource = UriManager.GetOrCreateExternalSource(SourceUri))
			{
				Sources.Add(FileExternalSource.ToSharedRef());
			}
		}

		if (Sources.Num() > 0)
		{
			FEditorDirectories::Get().SetLastDirectory(ELastDirectory::GENERIC_IMPORT, FPaths::GetPath(Sources[0]->GetFallbackFilepath())); // Save path as default for next time.
			DatasmithImporterHelperInternal::Import(Sources, Factory.Get());
		}
	}

	Factory->CleanUp();
}

// Patterned on FEditorFileUtils::Import(), but using the given Factory instead of deducing the factory from the file extension
void FDatasmithImporterHelper::ImportInternal(UFactory* FactoryCDO, const TSharedRef<UE::DatasmithImporter::FExternalSource>& ExternalSource)
{
	if (!FactoryCDO)
	{
		return;
	}

	checkf(FactoryCDO->IsA(UDatasmithImportFactory::StaticClass()), TEXT("While prototyping we should only use ExternalSource with UDatasmithImportFactory."));
	TStrongObjectPtr<UFactory> Factory(NewObject< UFactory >( (UObject*)GetTransientPackage(), FactoryCDO->GetClass() ) );
	Factory->ConfigureProperties();

	TArray<TSharedRef<UE::DatasmithImporter::FExternalSource>> Sources{ ExternalSource };
	DatasmithImporterHelperInternal::Import(Sources, Factory.Get());

	Factory->CleanUp();
}

// Patterned on FEditorFileUtils::GetFilterString, but without the interaction type and using the given factory to determine the supported file extensions
FString FDatasmithImporterHelper::GetFilterStringInternal(UFactory* Factory)
{
	TArray<UFactory*> Factories;
	Factories.Add(Factory);

	FString FileTypes;
	FString AllExtensions;
	TMultiMap<uint32, UFactory*> FilterIndexToFactory;

	ObjectTools::GenerateFactoryFileExtensions(Factories, FileTypes, AllExtensions, FilterIndexToFactory);

	return FString::Printf(TEXT("All Files (%s)|%s|%s"), *AllExtensions, *AllExtensions, *FileTypes);
}

#undef LOCTEXT_NAMESPACE
