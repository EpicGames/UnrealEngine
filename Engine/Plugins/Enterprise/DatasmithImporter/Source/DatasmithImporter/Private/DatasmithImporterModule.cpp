// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithImporterModule.h"

#include "ActorFactoryDatasmithScene.h"
#include "DatasmithAssetImportData.h"
#include "DatasmithAssetUserData.h"
#include "DatasmithConsumer.h"
#include "DatasmithContentEditorModule.h"
#include "DatasmithContentEditorStyle.h"
#include "DatasmithCustomAction.h"
#include "DatasmithFileProducer.h"
#include "DatasmithImporterEditorSettings.h"
#include "DatasmithImporterHelper.h"
#include "DatasmithImportFactory.h"
#include "DatasmithScene.h"
#include "DatasmithStaticMeshImporter.h"
#include "DatasmithUtils.h"
#include "DirectLinkExternalSource.h"
#include "ObjectTemplates/DatasmithObjectTemplate.h"
#include "ObjectTemplates/DatasmithStaticMeshTemplate.h"
#include "UI/DatasmithConsumerDetails.h"
#include "UI/DatasmithStyle.h"

#include "Async/Async.h"
#include "AssetToolsModule.h"
#include "ContentBrowserDelegates.h"
#include "ContentBrowserMenuContexts.h"
#include "ContentBrowserModule.h"
#include "DatasmithContentModule.h"
#include "DataprepAssetInterface.h"
#include "DataprepAssetUserData.h"
#include "DataprepCoreUtils.h"
#include "DirectLinkExtensionEditorModule.h"
#include "DirectLinkUriResolver.h"
#include "Editor.h"
#include "EditorFramework/AssetImportData.h"
#include "Styling/AppStyle.h"
#include "Engine/StaticMesh.h"
#include "ExternalSourceModule.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IAssetTools.h"
#include "IDirectLinkManager.h"
#include "IUriManager.h"
#include "LevelEditor.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Misc/App.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "ToolMenu.h"
#include "ToolMenuDelegates.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "UObject/StrongObjectPtr.h"

#define LOCTEXT_NAMESPACE "DatasmithImporter"

/**
 * DatasmithImporter module implementation (private)
 */
class FDatasmithImporterModule : public IDatasmithImporterModule
{
public:
	virtual void StartupModule() override
	{
		UDatasmithFileProducer::LoadDefaultSettings();

		// Disable any UI feature if running in command mode
		if (UToolMenus::IsToolMenuUIEnabled())
		{
			FDatasmithStyle::Initialize();
			FDatasmithStyle::SetIcon(TEXT("Import"), TEXT("DatasmithImporter/Content/Icons/DatasmithImporterIcon40"));

			SetupMenuEntry();
			SetupContentBrowserContextMenuExtender();
			SetupLevelEditorContextMenuExtender();
			SetupDatasmithContentDelegates();

			// Register the details customizer
			FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked< FPropertyEditorModule >( TEXT("PropertyEditor") );
			PropertyModule.RegisterCustomClassLayout( TEXT("DatasmithFileProducer"), FOnGetDetailCustomizationInstance::CreateStatic( &FDatasmithFileProducerDetails::MakeDetails ) );
			PropertyModule.RegisterCustomClassLayout( TEXT("DatasmithDirProducer"), FOnGetDetailCustomizationInstance::CreateStatic( &FDatasmithDirProducerDetails::MakeDetails ) );
			PropertyModule.RegisterCustomClassLayout( TEXT("DatasmithConsumer"), FOnGetDetailCustomizationInstance::CreateStatic( &FDatasmithConsumerDetails::MakeDetails ) );

			AddDataprepMenuEntryForDatasmithSceneAsset();
		}
	}

	virtual void ShutdownModule() override
	{
		// Disable any UI feature if running in command mode
		if (UToolMenus::IsToolMenuUIEnabled())
		{
			RemoveDataprepMenuEntryForDatasmithSceneAsset();
			RemoveDatasmithContentDelegates();
			RemoveLevelEditorContextMenuExtender();
			RemoveContentBrowserContextMenuExtender();

			FDatasmithStyle::Shutdown();

			// Register the details customizer
			FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked< FPropertyEditorModule >( TEXT("PropertyEditor") );
			PropertyModule.UnregisterCustomClassLayout( TEXT("DatasmithFileProducer") );
			PropertyModule.UnregisterCustomClassLayout( TEXT("DatasmithDirProducer") );
			PropertyModule.UnregisterCustomClassLayout( TEXT("DatasmithConsumer") );
		}
	}

	virtual void ResetOverrides( UObject* Object ) override
	{
		ResetFromTemplates( Object );
	}

	virtual FOnGenerateDatasmithImportMenu& OnGenerateDatasmithImportMenu() override
	{
		return GenerateDatasmithImportMenuEvent;
	}

	bool IsInOfflineMode() const
	{
		return GetDefault< UDatasmithImporterEditorSettings >() && GetDefault< UDatasmithImporterEditorSettings >()->bOfflineImporter;
	}

private:
	static TSharedRef<FExtender> OnExtendContentBrowserAssetSelectionMenu(const TArray<FAssetData>& SelectedAssets);
	static TSharedRef<FExtender> OnExtendLevelEditorActorSelectionMenu(const TSharedRef<FUICommandList> CommandList, TArray<AActor*> SelectedActors);

	static void PopulateDatasmithActionsMenu( FMenuBuilder& MenuBuilder, TArray<FAssetData> SelectedAssets );
	static void PopulateDatasmithActorsMenu( FMenuBuilder& MenuBuilder, TArray< AActor*> SelectedActors );
	static void ExecuteReimportDatasmithMaterials(TArray<FAssetData> AssetData);

	static void DiffAgainstTemplates( UObject* Outer );
	static void ResetFromTemplates( UObject* Outer );

	static void DiffAssetAgainstTemplate( TArray< FAssetData > SelectedAssets );
	static void ResetAssetFromTemplate( TArray< FAssetData > SelectedAssets );

	static void DiffActorAgainstTemplate( TArray< AActor* > SelectedActors );
	static void ResetActorFromTemplate( TArray< AActor* > SelectedActors );

	static void ApplyCustomActionOnAssets(TArray< FAssetData > SelectedAssets, IDatasmithCustomAction* Action);

	void SetupMenuEntry();
	void OnClickedImportFileMenuEntry();
	void OnClickedImportDirectLinkMenuEntry();

	// Add the menu entry for a datasmith asset generated from a dataprep asset
	void AddDataprepMenuEntryForDatasmithSceneAsset();
	void RemoveDataprepMenuEntryForDatasmithSceneAsset();

	void SetupContentBrowserContextMenuExtender();
	void RemoveContentBrowserContextMenuExtender();

	void SetupLevelEditorContextMenuExtender();
	void RemoveLevelEditorContextMenuExtender();

	void SetupDatasmithContentDelegates();
	void RemoveDatasmithContentDelegates();

	static TSharedPtr<IDataprepImporterInterface> CreateDatasmithImportHandler();

	FDelegateHandle ContentBrowserExtenderDelegateHandle;
	FDelegateHandle LevelEditorExtenderDelegateHandle;
	FDelegateHandle SpawnSceneActorsDelegateHandle;
	FDelegateHandle CreateDatasmithImportHandlerDelegateHandle;

	FDelegateHandle SetAssetAutoReimportHandle;
	FDelegateHandle IsAssetAutoReimportSupportedHandle;
	FDelegateHandle IsAssetAutoReimportEnabledHandle;
	FDelegateHandle BrowseExternalSourceUriHandle;
	FDelegateHandle GetSupporedUriSchemeHandle;

	FOnGenerateDatasmithImportMenu GenerateDatasmithImportMenuEvent;

};

void FDatasmithImporterModule::SetupMenuEntry()
{
	if (!IsRunningCommandlet())
	{
		UToolMenu* ContentMenu = UToolMenus::Get()->ExtendMenu( "LevelEditor.LevelEditorToolBar.AddQuickMenu" );
		check( ContentMenu );

		FToolMenuSection& Section = ContentMenu->FindOrAddSection( "ImportAssets" );
		Section.InitSection( "ImportAssets", LOCTEXT( "ImportAssets_Label", "Import Assets" ), FToolMenuInsert() );

		const bool bOpenMenuOnClick = false; //default value;
		Section.AddSubMenu(
			TEXT( "DatasmithImport" ),
			LOCTEXT( "DatasmithImport", "Datasmith" ), // label
			LOCTEXT( "DatasmithImportTooltip", "Import Unreal Datasmith file" ), // description
			FNewToolMenuDelegate::CreateLambda( [&]( UToolMenu* Menu ) {
				FToolMenuSection& SubSection = Menu->FindOrAddSection( TEXT( "DatasmithImportMenu" ) );
				SubSection.InitSection( TEXT( "DatasmithImportMenu" ), LOCTEXT( "DatasmithImport_Label", "Datasmith Import" ), FToolMenuInsert() );

				SubSection.AddMenuEntry(
					TEXT( "ImportFile" ),
					LOCTEXT( "DatasmithFileImport", "File Import..." ), // label
					LOCTEXT( "DatasmithFileImportTooltip", "Import Unreal Datasmith file" ), // description
					FSlateIcon( FDatasmithStyle::GetStyleSetName(), TEXT("Datasmith.Import") ),
					FUIAction( FExecuteAction::CreateRaw( this, &FDatasmithImporterModule::OnClickedImportFileMenuEntry ) )
				);

				SubSection.AddMenuEntry(
					TEXT( "ImportDirectLink" ),
					LOCTEXT( "DatasmithDirectLinkImport", "Direct Link Import..." ), // label
					LOCTEXT( "DatasmithDirectLinkImportTooltip", "Import Unreal Datasmith with Direct Link" ), // description
					FSlateIcon( FDatasmithStyle::GetStyleSetName(), TEXT( "Datasmith.Import" ) ),
					FUIAction( FExecuteAction::CreateRaw( this, &FDatasmithImporterModule::OnClickedImportDirectLinkMenuEntry ) )
				);

				GenerateDatasmithImportMenuEvent.Broadcast(SubSection);
			}),
			bOpenMenuOnClick,
			FSlateIcon( FDatasmithStyle::GetStyleSetName(), TEXT( "Datasmith.Import" ) )
		);
	}
}

void FDatasmithImporterModule::OnClickedImportFileMenuEntry()
{
	if ( !IsRunningCommandlet() )
	{
		FDatasmithImporterHelper::Import< UDatasmithImportFactory >();
	}
}

void FDatasmithImporterModule::OnClickedImportDirectLinkMenuEntry()
{
	using namespace UE::DatasmithImporter;

	TSharedPtr<FDirectLinkExternalSource> ExternalSource = IDirectLinkExtensionEditorModule::Get().DisplayDirectLinkSourcesDialog();
	if ( ExternalSource )
	{
		TFuture<TSharedPtr<IDatasmithScene>> DatasmithSceneFuture = ExternalSource->AsyncLoad();

		// Even though DatasmithSceneFuture will be destructed when exiting this scope, its shared state will still be referenced by the TPromise.
		// This is where the delegate set by Next() is kept, and so the callback will always be called.
		DatasmithSceneFuture.Next([ExternalSource = MoveTemp(ExternalSource)](const TSharedPtr<IDatasmithScene>& LoadedScene) {
			if (LoadedScene)
			{
				// Import can only be triggered from main thread.
				Async(EAsyncExecution::TaskGraphMainThread, [ExternalSource]() {
					FDatasmithImporterHelper::Import< UDatasmithImportFactory >(ExternalSource.ToSharedRef());
				});
			}
		});
	}
}

void FDatasmithImporterModule::AddDataprepMenuEntryForDatasmithSceneAsset()
{
	if (UToolMenu* Menu = UToolMenus::Get()->ExtendMenu(TEXT("ContentBrowser.AssetContextMenu.DatasmithScene")))
	{

		FNewToolMenuDelegate DataprepSectionConstructor;
		DataprepSectionConstructor.BindLambda( [](UToolMenu* ToolMenu)
			{
				if ( ToolMenu )
				{
					TArray<TStrongObjectPtr<UDataprepAssetInterface>> DataprepAssetInterfacesPtr;
					if (UContentBrowserAssetContextMenuContext* ContentBrowserMenuContext = ToolMenu->FindContext<UContentBrowserAssetContextMenuContext>())
					{
						if (ContentBrowserMenuContext->CommonClass == UDatasmithScene::StaticClass())
						{
							TArray<UDatasmithScene*> SelectedObjects = ContentBrowserMenuContext->LoadSelectedObjects<UDatasmithScene>({});
							DataprepAssetInterfacesPtr.Reserve( SelectedObjects.Num() );
							for (UDatasmithScene* SelectedDatasmithScene : SelectedObjects)
							{
								if (UDataprepAssetUserData* DataprepAssetUserData = SelectedDatasmithScene->GetAssetUserData<UDataprepAssetUserData>())
								{
									if (UDataprepAssetInterface* DataprepAsset = DataprepAssetUserData->DataprepAssetPtr.LoadSynchronous())
									{
										if (UDatasmithConsumer* DatasmithConsumer = Cast<UDatasmithConsumer>(DataprepAsset->GetConsumer()))
										{
											if (DatasmithConsumer->GetDatasmithScene() == SelectedDatasmithScene)
											{
												// A Dataprep asset was found and it will regenerate this scene
												DataprepAssetInterfacesPtr.Emplace( DataprepAsset );
												continue;
											}
										}
									}
								}

								// Couldn't find the dataprep asset for this scene
								return;
							}
						}
					}
					else
					{
						return;
					}


					FToolUIAction UIAction;
					UIAction.ExecuteAction.BindLambda( [DataprepAssetInterfacesPtr](const FToolMenuContext&)
						{
							for ( const TStrongObjectPtr<UDataprepAssetInterface>& DataprepAssetInterfacePtr : DataprepAssetInterfacesPtr )
							{
								FDataprepCoreUtils::ExecuteDataprep( DataprepAssetInterfacePtr.Get()
									, MakeShared<FDataprepCoreUtils::FDataprepLogger>()
									, MakeShared<FDataprepCoreUtils::FDataprepProgressUIReporter>() );
							}
						});

					FToolMenuInsert MenuInsert;
					MenuInsert.Position = EToolMenuInsertType::First;
					FToolMenuSection& Section = ToolMenu->AddSection( TEXT("Dataprep"), LOCTEXT("Dataprep", "Dataprep"), MenuInsert );

					Section.AddMenuEntry(
						TEXT("UpdateDataprepGeneratedAsset"),
						LOCTEXT("UpdateDataprepGeneratedAsset", "Update Datasmith Scene(s)"),
						LOCTEXT("UpdateDataprepGeneratedAssetTooltip", "Update the asset(s) by executing the Dataprep asset(s) that created it."),
						FSlateIcon(),
						UIAction
					);

				}
			});


		FToolMenuSection& Section = Menu->AddDynamicSection(TEXT("Dataprep"), DataprepSectionConstructor);
	}
}

void FDatasmithImporterModule::RemoveDataprepMenuEntryForDatasmithSceneAsset()
{
	if (UToolMenus* Singlethon = UToolMenus::Get())
	{
		if (UToolMenu* Menu = Singlethon->ExtendMenu(TEXT("ContentBrowser.AssetContextMenu.DatasmithScene")))
		{
			Menu->RemoveSection(TEXT("Dataprep"));
		}
	}
}

void FDatasmithImporterModule::SetupContentBrowserContextMenuExtender()
{
	if ( !IsRunningCommandlet() )
	{
		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>( "ContentBrowser" );
		TArray< FContentBrowserMenuExtender_SelectedAssets >& CBMenuExtenderDelegates = ContentBrowserModule.GetAllAssetViewContextMenuExtenders();

		CBMenuExtenderDelegates.Add( FContentBrowserMenuExtender_SelectedAssets::CreateStatic( &FDatasmithImporterModule::OnExtendContentBrowserAssetSelectionMenu ) );
		ContentBrowserExtenderDelegateHandle = CBMenuExtenderDelegates.Last().GetHandle();
	}
}

void FDatasmithImporterModule::RemoveContentBrowserContextMenuExtender()
{
	if ( ContentBrowserExtenderDelegateHandle.IsValid() && FModuleManager::Get().IsModuleLoaded( "ContentBrowser" ) )
	{
		FContentBrowserModule& ContentBrowserModule = FModuleManager::GetModuleChecked<FContentBrowserModule>( "ContentBrowser" );
		TArray<FContentBrowserMenuExtender_SelectedAssets>& CBMenuExtenderDelegates = ContentBrowserModule.GetAllAssetViewContextMenuExtenders();
		CBMenuExtenderDelegates.RemoveAll(
			[ this ]( const FContentBrowserMenuExtender_SelectedAssets& Delegate )
			{
				return Delegate.GetHandle() == ContentBrowserExtenderDelegateHandle;
			}
		);
	}
}

void FDatasmithImporterModule::SetupLevelEditorContextMenuExtender()
{
	if ( !IsRunningCommandlet() )
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked< FLevelEditorModule >( "LevelEditor" );
		TArray< FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors >& CBMenuExtenderDelegates = LevelEditorModule.GetAllLevelViewportContextMenuExtenders();

		CBMenuExtenderDelegates.Add( FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors::CreateStatic( &FDatasmithImporterModule::OnExtendLevelEditorActorSelectionMenu ) );
		LevelEditorExtenderDelegateHandle = CBMenuExtenderDelegates.Last().GetHandle();
	}
}

void FDatasmithImporterModule::RemoveLevelEditorContextMenuExtender()
{
	if ( ContentBrowserExtenderDelegateHandle.IsValid() && FModuleManager::Get().IsModuleLoaded( "LevelEditor" ) )
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked< FLevelEditorModule >( "LevelEditor" );
		TArray< FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors >& CBMenuExtenderDelegates = LevelEditorModule.GetAllLevelViewportContextMenuExtenders();
		CBMenuExtenderDelegates.RemoveAll(
			[ this ]( const FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors& Delegate )
		{
			return Delegate.GetHandle() == LevelEditorExtenderDelegateHandle;
		}
		);
	}
}

TSharedRef<FExtender> FDatasmithImporterModule::OnExtendContentBrowserAssetSelectionMenu(const TArray<FAssetData>& SelectedAssets)
{
	TSharedRef<FExtender> Extender = MakeShared<FExtender>();

	// Run through the assets to determine if any meet our criteria
	bool bShouldExtendAssetActions = false;
	for ( const FAssetData& Asset : SelectedAssets )
	{
		if ( Asset.AssetClassPath == UMaterial::StaticClass()->GetClassPathName() || Asset.AssetClassPath == UMaterialInstance::StaticClass()->GetClassPathName() ||
			 Asset.AssetClassPath == UMaterialInstanceConstant::StaticClass()->GetClassPathName() )
		{
			UMaterialInterface* MaterialInterface = Cast< UMaterialInterface >( Asset.GetAsset() ); // Need to load the asset at this point to figure out the type of the AssetImportData

			if ( ( MaterialInterface && MaterialInterface->AssetImportData && MaterialInterface->AssetImportData->IsA< UDatasmithAssetImportData >() ) ||
				 FDatasmithObjectTemplateUtils::HasObjectTemplates( MaterialInterface ) )
			{
				bShouldExtendAssetActions = true;
				break;
			}
		}
		else if ( Asset.AssetClassPath == UStaticMesh::StaticClass()->GetClassPathName() )
		{
			UStaticMesh* StaticMesh = Cast< UStaticMesh >( Asset.GetAsset() ); // Need to load the asset at this point to figure out the type of the AssetImportData

			if ( StaticMesh && StaticMesh->GetAssetImportData() && StaticMesh->GetAssetImportData()->IsA< UDatasmithAssetImportData >())
			{
				bShouldExtendAssetActions = true;
				break;
			}
		}
	}

	if ( bShouldExtendAssetActions )
	{
		// Add the Datasmith actions sub-menu extender
		Extender->AddMenuExtension("GetAssetActions", EExtensionHook::After, nullptr, FMenuExtensionDelegate::CreateLambda(
			[SelectedAssets](FMenuBuilder& MenuBuilder)
			{
				MenuBuilder.AddSubMenu(
					NSLOCTEXT("DatasmithActions", "ObjectContext_Datasmith", "Datasmith"),
					NSLOCTEXT("DatasmithActions", "ObjectContext_Datasmith", "Datasmith"),
					FNewMenuDelegate::CreateStatic( &FDatasmithImporterModule::PopulateDatasmithActionsMenu, SelectedAssets ),
					false,
					FSlateIcon());
			}));
	}

	return Extender;
}

TSharedRef<FExtender> FDatasmithImporterModule::OnExtendLevelEditorActorSelectionMenu( const TSharedRef<FUICommandList> CommandList, TArray<AActor*> SelectedActors )
{
	TSharedRef<FExtender> Extender = MakeShared<FExtender>();

	bool bShouldExtendActorActions = false;

	for ( AActor* Actor : SelectedActors )
	{
		for ( UActorComponent* Component : Actor->GetComponents() )
		{
			if ( FDatasmithObjectTemplateUtils::HasObjectTemplates( Component ) )
			{
				bShouldExtendActorActions = true;
				break;
			}
		}

		if ( bShouldExtendActorActions )
		{
			break;
		}
	}

	if ( bShouldExtendActorActions )
	{
		// Add the Datasmith actions sub-menu extender
		Extender->AddMenuExtension("ActorTypeTools", EExtensionHook::After, nullptr, FMenuExtensionDelegate::CreateLambda(
			[SelectedActors](FMenuBuilder& MenuBuilder)
		{
			MenuBuilder.BeginSection("Datasmith", LOCTEXT("DatasmithMenuSection", "Datasmith"));
			MenuBuilder.AddSubMenu(
				NSLOCTEXT("DatasmithActions", "ObjectContext_Datasmith", "Datasmith"),
				NSLOCTEXT("DatasmithActions", "ObjectContext_Datasmith", "Datasmith"),
				FNewMenuDelegate::CreateStatic( &FDatasmithImporterModule::PopulateDatasmithActorsMenu, SelectedActors ),
				false,
				FSlateIcon());
			MenuBuilder.EndSection();
		}));
	}

	return Extender;
}

void FDatasmithImporterModule::PopulateDatasmithActionsMenu( FMenuBuilder& MenuBuilder, TArray<FAssetData> SelectedAssets )
{
	bool bCanResetOverrides = false;
	bool bCanReimportMaterial = false;

	for ( const FAssetData& Asset : SelectedAssets )
	{
		if ( Asset.AssetClassPath == UMaterial::StaticClass()->GetClassPathName() || Asset.AssetClassPath == UMaterialInstance::StaticClass()->GetClassPathName() ||
			 Asset.AssetClassPath == UMaterialInstanceConstant::StaticClass()->GetClassPathName() )
		{
			bCanResetOverrides = true;

			UMaterialInterface* MaterialInterface = Cast< UMaterialInterface >( Asset.GetAsset() );
			bCanReimportMaterial = ( MaterialInterface && MaterialInterface->AssetImportData && MaterialInterface->AssetImportData->IsA< UDatasmithAssetImportData >() );
		}
		else if ( Asset.AssetClassPath == UStaticMesh::StaticClass()->GetClassPathName() )
		{
			bCanResetOverrides = true;
		}
	}

	if ( bCanResetOverrides )
	{
		// Add the Datasmith diff sub-menu extender (disabled until we have a proper UI)
		/*MenuBuilder.AddMenuEntry(
			NSLOCTEXT("DatasmithActions", "ObjectContext_DiffDatasmith", "Show Overrides"),
			NSLOCTEXT("DatasmithActions", "ObjectContext_DiffDatasmithTooltip", "Displays which values are currently overriden"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "SourceControl.Actions.Diff"),
			FUIAction( FExecuteAction::CreateStatic( &FDatasmithImporterModule::DiffAssetAgainstTemplate, SelectedAssets ), FCanExecuteAction() ));*/

		// Add the Datasmith reset sub-menu extender
		MenuBuilder.AddMenuEntry(
			NSLOCTEXT("DatasmithActions", "ObjectContext_ResetDatasmith", "Reset Overrides"),
			NSLOCTEXT("DatasmithActions", "ObjectContext_ResetDatasmithTooltip", "Resets overriden values with the values from Datasmith"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "SourceControl.Actions.Refresh"),
			FUIAction( FExecuteAction::CreateStatic( &FDatasmithImporterModule::ResetAssetFromTemplate, SelectedAssets ), FCanExecuteAction() ));
	}

	if ( bCanReimportMaterial )
	{
		// Add the reimport Datasmith material sub-menu extender
		MenuBuilder.AddMenuEntry(
			NSLOCTEXT("AssetTypeActions_Material", "ObjectContext_ReimportDatasmithMaterial", "Reimport Material"),
			NSLOCTEXT("AssetTypeActions_Material", "ObjectContext_ReimportDatasmithMaterialTooltip", "Reimports a material using Datasmith"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.AssetActions.ReimportAsset"),
			FUIAction( FExecuteAction::CreateStatic( &FDatasmithImporterModule::ExecuteReimportDatasmithMaterials, SelectedAssets ), FCanExecuteAction() ));
	}

	// Add an entry for each applicable custom action
	FDatasmithCustomActionManager ActionsManager;
	for (UDatasmithCustomActionBase* Action : ActionsManager.GetApplicableActions(SelectedAssets))
	{
		if (ensure(Action))
		{
			MenuBuilder.AddMenuEntry(
				Action->GetLabel(),
				Action->GetTooltip(),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.AssetActions.ReimportAsset"),
				FUIAction( FExecuteAction::CreateLambda( [=](){ FDatasmithImporterModule::ApplyCustomActionOnAssets(SelectedAssets, Action);} ), FCanExecuteAction() )
			);
		}
	}
}

void FDatasmithImporterModule::PopulateDatasmithActorsMenu( FMenuBuilder& MenuBuilder, TArray< AActor*> SelectedActors )
{
	// Add the Datasmith diff sub-menu extender (disabled until we have a proper UI)
	/*MenuBuilder.AddMenuEntry(
		NSLOCTEXT("DatasmithActions", "ObjectContext_DiffDatasmith", "Show Overrides"),
		NSLOCTEXT("DatasmithActions", "ObjectContext_DiffDatasmithTooltip", "Displays which values are currently overriden"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "SourceControl.Actions.Diff"),
		FUIAction( FExecuteAction::CreateStatic( &FDatasmithImporterModule::DiffActorAgainstTemplate, SelectedActors ), FCanExecuteAction() ));*/

	// Add the Datasmith reset sub-menu extender
	MenuBuilder.AddMenuEntry(
		NSLOCTEXT("DatasmithActions", "ObjectContext_ResetDatasmith", "Reset Overrides"),
		NSLOCTEXT("DatasmithActions", "ObjectContext_ResetDatasmithTooltip", "Resets overriden values with the values from Datasmith"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "SourceControl.Actions.Refresh"),
		FUIAction( FExecuteAction::CreateStatic( &FDatasmithImporterModule::ResetActorFromTemplate, SelectedActors ), FCanExecuteAction() ));

	// Add an entry for each applicable custom action
	FDatasmithCustomActionManager ActionsManager;
	for (UDatasmithCustomActionBase* Action : ActionsManager.GetApplicableActions(SelectedActors))
	{
		if (ensure(Action))
		{
			MenuBuilder.AddMenuEntry(
				Action->GetLabel(),
				Action->GetTooltip(),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.AssetActions.ReimportAsset"),
				FUIAction( FExecuteAction::CreateLambda( [=]() { Action->ApplyOnActors(SelectedActors); } ) , FCanExecuteAction() )
			);
		}
	}
}

void FDatasmithImporterModule::ExecuteReimportDatasmithMaterials( TArray<FAssetData> SelectedAssets )
{
	if ( UDatasmithImportFactory* DatasmithImportFactory = UDatasmithImportFactory::StaticClass()->GetDefaultObject< UDatasmithImportFactory >() )
	{
		for ( const FAssetData& AssetData : SelectedAssets )
		{
			if ( AssetData.AssetClassPath == UMaterial::StaticClass()->GetClassPathName()
			  || AssetData.AssetClassPath == UMaterialInstance::StaticClass()->GetClassPathName()
			  || AssetData.AssetClassPath == UMaterialInstanceConstant::StaticClass()->GetClassPathName() )
			{
				if ( UObject* AssetToReimport = AssetData.GetAsset() )
				{
					TArray<FString> OutFilenames;
					if (DatasmithImportFactory->CanReimport(AssetToReimport, OutFilenames))
					{
						DatasmithImportFactory->Reimport( AssetToReimport );
					}
				}
			}
		}
	}
}

void FDatasmithImporterModule::DiffAgainstTemplates( UObject* Outer )
{
	TMap< TSubclassOf< UDatasmithObjectTemplate >, TObjectPtr<UDatasmithObjectTemplate> >* ObjectTemplates = FDatasmithObjectTemplateUtils::FindOrCreateObjectTemplates( Outer );

	if ( !ObjectTemplates )
	{
		return;
	}

	for ( auto It = ObjectTemplates->CreateConstIterator(); It; ++It )
	{
		UDatasmithObjectTemplate* OldTemplate = FDatasmithObjectTemplateUtils::GetObjectTemplate( Outer, It->Key );

		UDatasmithObjectTemplate* NewTemplate = NewObject< UDatasmithObjectTemplate >( GetTransientPackage(), It->Key, NAME_None, RF_Transient );
		NewTemplate->Load( Outer );

		check( OldTemplate != nullptr );
		check( NewTemplate != nullptr );

		// Dump assets to temp text files
		FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");

		FString OldTextFilename = AssetToolsModule.Get().DumpAssetToTempFile( OldTemplate );
		FString NewTextFilename = AssetToolsModule.Get().DumpAssetToTempFile( NewTemplate );
		FString DiffCommand = GetDefault< UEditorLoadingSavingSettings >()->TextDiffToolPath.FilePath;

		AssetToolsModule.Get().CreateDiffProcess( DiffCommand, OldTextFilename, NewTextFilename );
	}
}

void FDatasmithImporterModule::ResetFromTemplates( UObject* Outer )
{
	if ( TMap< TSubclassOf< UDatasmithObjectTemplate >, TObjectPtr<UDatasmithObjectTemplate> >* ObjectTemplates = FDatasmithObjectTemplateUtils::FindOrCreateObjectTemplates( Outer ) )
	{
		for ( auto It = ObjectTemplates->CreateConstIterator(); It; ++It )
		{
			It->Value->Apply( Outer, true );
		}
	}
}

void FDatasmithImporterModule::DiffAssetAgainstTemplate( TArray< FAssetData > SelectedAssets )
{
	for ( const FAssetData& AssetData : SelectedAssets )
	{
		UStaticMesh* StaticMesh = Cast< UStaticMesh >( AssetData.GetAsset() );

		if ( !StaticMesh )
		{
			continue;
		}

		DiffAgainstTemplates( StaticMesh );
	}
}

void FDatasmithImporterModule::ResetAssetFromTemplate( TArray< FAssetData > SelectedAssets )
{
	for ( const FAssetData& AssetData : SelectedAssets )
	{
		UObject* Asset = AssetData.GetAsset();

		if ( !Asset )
		{
			continue;
		}

		Asset->PreEditChange( nullptr );
		ResetFromTemplates( Asset );
		Asset->PostEditChange();
	}

	if ( GEditor )
	{
		GEditor->RedrawAllViewports();
	}
}

void FDatasmithImporterModule::DiffActorAgainstTemplate( TArray< AActor*> SelectedActors )
{
	for ( AActor* Actor : SelectedActors )
	{
		if ( !Actor )
		{
			continue;
		}

		for ( UActorComponent* Component : Actor->GetComponents() )
		{
			DiffAgainstTemplates( Component );
		}
	}
}

void FDatasmithImporterModule::ResetActorFromTemplate( TArray< AActor*> SelectedActors )
{
	for ( AActor* Actor : SelectedActors )
	{
		if ( !Actor )
		{
			continue;
		}

		Actor->UnregisterAllComponents( true );

		for ( UActorComponent* Component : Actor->GetComponents() )
		{
			ResetFromTemplates( Component );
		}

		Actor->RerunConstructionScripts();
		Actor->RegisterAllComponents();

		GEditor->RedrawAllViewports();
	}
}

void FDatasmithImporterModule::ApplyCustomActionOnAssets(TArray< FAssetData > SelectedAssets, IDatasmithCustomAction* Action)
{
	if (ensure(Action))
	{
		Action->ApplyOnAssets(SelectedAssets);
	}
}

TSharedPtr< IDataprepImporterInterface > FDatasmithImporterModule::CreateDatasmithImportHandler()
{
	return nullptr;
}

void FDatasmithImporterModule::SetupDatasmithContentDelegates()
{
	IDatasmithContentEditorModule& DatasmithContentEditorModule = FModuleManager::LoadModuleChecked< IDatasmithContentEditorModule >(TEXT("DatasmithContentEditor"));

	FOnSpawnDatasmithSceneActors SpawnSceneActorsDelegate = FOnSpawnDatasmithSceneActors::CreateStatic(UActorFactoryDatasmithScene::SpawnRelatedActors);
	SpawnSceneActorsDelegateHandle = SpawnSceneActorsDelegate.GetHandle();
	DatasmithContentEditorModule.RegisterSpawnDatasmithSceneActorsHandler(SpawnSceneActorsDelegate);

	{
		FOnSetAssetAutoReimport SetAssetAutoReimport = FOnSetAssetAutoReimport::CreateLambda(
			[](UObject* Asset, bool bEnabled)
			{
				using namespace UE::DatasmithImporter;
				const FAssetData AssetData(Asset);
				const FSourceUri SourceUri = FSourceUri::FromAssetData(AssetData);

				if (bEnabled && !SourceUri.HasScheme(FDirectLinkUriResolver::GetDirectLinkScheme()))
				{
					return false;
				}

				return IDirectLinkExtensionEditorModule::Get().GetManager().SetAssetAutoReimport(Asset, bEnabled);
			});
		SetAssetAutoReimportHandle = SetAssetAutoReimport.GetHandle();
		DatasmithContentEditorModule.RegisterSetAssetAutoReimportHandler(MoveTemp(SetAssetAutoReimport));
	}

	{
		FOnIsAssetAutoReimportAvailable IsAssetAutoReimportAvailable = FOnIsAssetAutoReimportAvailable::CreateLambda(
			[](UObject* Asset)
			{
				using namespace UE::DatasmithImporter;
				const FAssetData AssetData(Asset);
				const FSourceUri SourceUri = FSourceUri::FromAssetData(AssetData);

				return SourceUri.HasScheme(FDirectLinkUriResolver::GetDirectLinkScheme());
			});
		IsAssetAutoReimportSupportedHandle = IsAssetAutoReimportAvailable.GetHandle();
		DatasmithContentEditorModule.RegisterIsAssetAutoReimportAvailableHandler(MoveTemp(IsAssetAutoReimportAvailable));
	}

	{
		FOnIsAssetAutoReimportEnabled IsAssetAutoReimportEnabled = FOnIsAssetAutoReimportEnabled::CreateLambda(
			[](UObject* Asset)
			{
				using namespace UE::DatasmithImporter;
				const FAssetData AssetData(Asset);
				const FSourceUri SourceUri = FSourceUri::FromAssetData(AssetData);

				if (SourceUri.HasScheme(FDirectLinkUriResolver::GetDirectLinkScheme()))
				{
					return IDirectLinkExtensionEditorModule::Get().GetManager().IsAssetAutoReimportEnabled(Asset);
				}

				return false;
			});
		IsAssetAutoReimportEnabledHandle = IsAssetAutoReimportEnabled.GetHandle();
		DatasmithContentEditorModule.RegisterIsAssetAutoReimportEnabledHandler(MoveTemp(IsAssetAutoReimportEnabled));
	}

	{
		FOnBrowseExternalSourceUri BrowseExternalSourceUri = FOnBrowseExternalSourceUri::CreateLambda(
			[](FName UriScheme, const FString& DefaultUriString, FString& OutSourceUri, FString& OutFallbackFilepath)
			{
				using namespace UE::DatasmithImporter;
				FSourceUri DefaultUri(DefaultUriString);

				if (TSharedPtr<FExternalSource> ExternalSource = IExternalSourceModule::Get().GetManager().BrowseExternalSource(UriScheme, DefaultUri))
				{
					OutSourceUri = ExternalSource->GetSourceUri().ToString();
					OutFallbackFilepath = ExternalSource->GetFallbackFilepath();
					return true;
				}

				return false;
			});

		BrowseExternalSourceUriHandle = BrowseExternalSourceUri.GetHandle();
		DatasmithContentEditorModule.RegisterBrowseExternalSourceUriHandler(MoveTemp(BrowseExternalSourceUri));
	}

	{
		FOnGetSupportedUriSchemes GetSupporedUriScheme = FOnGetSupportedUriSchemes::CreateLambda(
			[]() -> const TArray<FName>&
			{
				using namespace UE::DatasmithImporter;
				return IExternalSourceModule::Get().GetManager().GetSupportedSchemes();
			});

		GetSupporedUriSchemeHandle = GetSupporedUriScheme.GetHandle();
		DatasmithContentEditorModule.RegisterGetSupportedUriSchemeHandler(MoveTemp(GetSupporedUriScheme));
	}
}

void FDatasmithImporterModule::RemoveDatasmithContentDelegates()
{
	if (SpawnSceneActorsDelegateHandle.IsValid() && FModuleManager::Get().IsModuleLoaded(TEXT("DatasmithContentEditor")))
	{
		IDatasmithContentEditorModule& DatasmithContentEditorModule = FModuleManager::GetModuleChecked< IDatasmithContentEditorModule >(TEXT("DatasmithContentEditor"));
		DatasmithContentEditorModule.UnregisterDatasmithImporter(this);
		DatasmithContentEditorModule.UnregisterSetAssetAutoReimportHandler(SetAssetAutoReimportHandle);
		DatasmithContentEditorModule.UnregisterIsAssetAutoReimportAvailableHandler(IsAssetAutoReimportSupportedHandle);
		DatasmithContentEditorModule.UnregisterIsAssetAutoReimportEnabledHandler(IsAssetAutoReimportEnabledHandle);
		DatasmithContentEditorModule.UnregisterBrowseExternalSourceUriHandler(BrowseExternalSourceUriHandle);
		DatasmithContentEditorModule.UnregisterGetSupportedUriSchemeHandler(GetSupporedUriSchemeHandle);
	}
}

IMPLEMENT_MODULE(FDatasmithImporterModule, DatasmithImporter);

#undef LOCTEXT_NAMESPACE
