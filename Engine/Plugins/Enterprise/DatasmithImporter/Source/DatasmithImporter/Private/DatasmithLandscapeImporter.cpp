// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithLandscapeImporter.h"

#include "DatasmithActorImporter.h"
#include "DatasmithImportContext.h"
#include "IDatasmithSceneElements.h"
#include "ObjectTemplates/DatasmithLandscapeTemplate.h"

#include "Landscape.h"
#include "LandscapeDataAccess.h"
#include "LandscapeEditorModule.h"
#include "LandscapeEditorObject.h"
#include "LandscapeFileFormatInterface.h"
#include "LandscapeImportHelper.h"

#include "Modules/ModuleManager.h"
#include "Utility/DatasmithImporterUtils.h"

#define LOCTEXT_NAMESPACE "DatasmithImportFactory"

AActor* FDatasmithLandscapeImporter::ImportLandscapeActor( const TSharedRef< IDatasmithLandscapeElement >& LandscapeActorElement, FDatasmithImportContext& ImportContext, EDatasmithImportActorPolicy ImportActorPolicy )
{
	const bool bSingleFile = true;
	const bool bFlipYAxis = false;
	FLandscapeImportDescriptor OutImportDescriptor;
	OutImportDescriptor.Scale = LandscapeActorElement->GetScale();
	FText OutMessage;
	ELandscapeImportResult ImportResult = FLandscapeImportHelper::GetHeightmapImportDescriptor(LandscapeActorElement->GetHeightmap(), bSingleFile, bFlipYAxis, OutImportDescriptor, OutMessage);
	if (ImportResult == ELandscapeImportResult::Error)
	{
		return nullptr;
	}
	// 0 could be used but this keeps the previous behavior.
	int32 DescriptorIndex = OutImportDescriptor.FileResolutions.Num() / 2;
	
	ULandscapeEditorObject* DefaultValueObject = ULandscapeEditorObject::StaticClass()->GetDefaultObject<ULandscapeEditorObject>();
	check(DefaultValueObject);

	int32 OutQuadsPerSection = DefaultValueObject->NewLandscape_QuadsPerSection;
	int32 OutSectionsPerComponent = DefaultValueObject->NewLandscape_SectionsPerComponent;
	FIntPoint OutComponentCount = DefaultValueObject->NewLandscape_ComponentCount;

	FLandscapeImportHelper::ChooseBestComponentSizeForImport(OutImportDescriptor.ImportResolutions[DescriptorIndex].Width, OutImportDescriptor.ImportResolutions[DescriptorIndex].Height, OutQuadsPerSection, OutSectionsPerComponent, OutComponentCount);

	TArray<uint16> ImportData;
	ImportResult = FLandscapeImportHelper::GetHeightmapImportData(OutImportDescriptor, DescriptorIndex, ImportData, OutMessage);
	if (ImportResult == ELandscapeImportResult::Error)
	{
		return nullptr;
	}
		
	const int32 QuadsPerComponent = OutSectionsPerComponent * OutQuadsPerSection;
	const int32 SizeX = OutComponentCount.X * QuadsPerComponent + 1;
	const int32 SizeY = OutComponentCount.Y * QuadsPerComponent + 1;

	TArray<uint16> FinalHeightData;
	FLandscapeImportHelper::TransformHeightmapImportData(ImportData, FinalHeightData, OutImportDescriptor.ImportResolutions[DescriptorIndex], FLandscapeImportResolution(SizeX, SizeY), ELandscapeImportTransformType::ExpandCentered);
		
	const FVector Offset = FTransform( LandscapeActorElement->GetRotation(), FVector::ZeroVector,
		LandscapeActorElement->GetScale() ).TransformVector( FVector( -OutComponentCount.X * QuadsPerComponent / 2., -OutComponentCount.Y * QuadsPerComponent / 2., 0. ) );

	FVector OriginalTranslation = LandscapeActorElement->GetTranslation();
	FVector OriginalScale = LandscapeActorElement->GetScale();

	LandscapeActorElement->SetTranslation( LandscapeActorElement->GetTranslation() + Offset );
	LandscapeActorElement->SetScale( LandscapeActorElement->GetScale() );
		
	ALandscape* Landscape = Cast< ALandscape >(FDatasmithActorImporter::ImportActor( ALandscape::StaticClass(), LandscapeActorElement, ImportContext, ImportActorPolicy,
		[ &FinalHeightData, SizeX, SizeY, OutSectionsPerComponent, OutQuadsPerSection, ActorScale = OutImportDescriptor.Scale ]( AActor* NewActor )
	{
		check( Cast< ALandscape >( NewActor ) );

		NewActor->SetActorRelativeScale3D( ActorScale );

		TMap<FGuid, TArray<uint16>> HeightmapDataPerLayers;
		HeightmapDataPerLayers.Add(FGuid(), MoveTemp(FinalHeightData));

		// The is not Material Layer Import Data
		TMap<FGuid, TArray<FLandscapeImportLayerInfo>> MaterialLayerDataPerLayer;
		MaterialLayerDataPerLayer.Add(FGuid(), TArray<FLandscapeImportLayerInfo>());
		
		Cast< ALandscape >( NewActor )->Import( FGuid::NewGuid(), 0, 0, SizeX - 1, SizeY - 1, OutSectionsPerComponent, OutQuadsPerSection,
			HeightmapDataPerLayers, nullptr, MaterialLayerDataPerLayer, ELandscapeImportAlphamapType::Additive, /*InImportLayers = */MakeArrayView<FLandscapeLayer>({}));
	} ) );

	LandscapeActorElement->SetTranslation( OriginalTranslation );
	LandscapeActorElement->SetScale( OriginalScale );

	if ( !Landscape )
	{
		return nullptr;
	}

	UDatasmithLandscapeTemplate* LandscapeTemplate = NewObject< UDatasmithLandscapeTemplate >( Landscape->GetRootComponent() );
	LandscapeTemplate->LandscapeMaterial = FDatasmithImporterUtils::FindAsset< UMaterialInterface >( ImportContext.AssetsContext, LandscapeActorElement->GetMaterial() );

	// automatically calculate a lighting LOD that won't crash lightmass (hopefully)
	// < 2048x2048 -> LOD0
	// >=2048x2048 -> LOD1
	// >= 4096x4096 -> LOD2
	// >= 8192x8192 -> LOD3
	LandscapeTemplate->StaticLightingLOD = FMath::DivideAndRoundUp(FMath::CeilLogTwo((SizeX * SizeY) / (2048 * 2048) + 1), (uint32)2);

	LandscapeTemplate->Apply( Landscape );

	Landscape->ReimportHeightmapFilePath = LandscapeActorElement->GetHeightmap();

	ULandscapeInfo* LandscapeInfo = Landscape->CreateLandscapeInfo();
	LandscapeInfo->UpdateLayerInfoMap(Landscape);

	// Import doesn't fill in the LayerInfo for layers with no data, do that now
	const TArray< FLandscapeImportLayer >& ImportLandscapeLayersList = DefaultValueObject->ImportLandscape_Layers;
	for(int32 i = 0; i < ImportLandscapeLayersList.Num(); i++)
	{
		if(ImportLandscapeLayersList[i].LayerInfo != nullptr)
		{
			Landscape->AddTargetLayer(ImportLandscapeLayersList[i].LayerInfo->GetLayerName(), FLandscapeTargetLayerSettings(ImportLandscapeLayersList[i].LayerInfo, ImportLandscapeLayersList[i].SourceFilePath));

			int32 LayerInfoIndex = LandscapeInfo->GetLayerInfoIndex(ImportLandscapeLayersList[i].LayerName);
			if(ensure(LayerInfoIndex != INDEX_NONE))
			{
				FLandscapeInfoLayerSettings& LayerSettings = LandscapeInfo->Layers[LayerInfoIndex];
				LayerSettings.LayerInfoObj = ImportLandscapeLayersList[i].LayerInfo;
			}
		}
	}

	return Landscape;
}

#undef LOCTEXT_NAMESPACE
