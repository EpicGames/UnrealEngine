// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithLevelSequenceImporter.h"

#include "DatasmithAnimationElements.h"
#include "DatasmithAreaLightActor.h"
#include "DatasmithDefinitions.h"
#include "DatasmithImportContext.h"
#include "DatasmithSceneActor.h"
#include "IDatasmithSceneElements.h"

#include "Algo/Transform.h"
#include "Channels/MovieSceneBoolChannel.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "LevelSequence.h"
#include "MovieScene.h"
#include "MovieSceneSection.h"
#include "MovieSceneSequence.h"
#include "ObjectTools.h"
#include "Sections/MovieScene3DTransformSection.h"
#include "Sections/MovieSceneBoolSection.h"
#include "Sections/MovieSceneSubSection.h"
#include "Tracks/MovieScene3DTransformTrack.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "Tracks/MovieSceneVisibilityTrack.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "DatasmithLevelSequenceImporter"

namespace DatasmithLevelSequenceImporterImpl
{
	FFrameRate GetFrameRateAsFraction(const TSharedRef<IDatasmithLevelSequenceElement>& LevelSequenceElement)
	{
		// Convert the frame rate from float to a fraction A / B
		// where A can be greater than B, and B doesn't need to be minimized
		float FrameRate = LevelSequenceElement->GetFrameRate();
		if (FrameRate <= 0.f)
		{
			FrameRate = 30.f;
		}

		float IntegralPart = FMath::FloorToFloat(FrameRate);
		float FractionalPart = FrameRate - IntegralPart;

		const int32 Precision = 1000000;
		int32 Divisor = FMath::GreatestCommonDivisor(FMath::RoundToFloat(FractionalPart * Precision), Precision);

		int32 Denominator = Precision / Divisor;
		int32 Numerator = IntegralPart * Denominator + FMath::RoundToFloat(FractionalPart * Precision) / Divisor;

		return FFrameRate(Numerator, Denominator);
	}

	AActor* FindActorByName(const FString& Name, const FDatasmithImportContext& ImportContext)
	{
		AActor* ExistingActor = nullptr;

		ADatasmithSceneActor* DatasmithSceneActor = ImportContext.ActorsContext.ImportSceneActor;
		if (DatasmithSceneActor)
		{
			TSoftObjectPtr<AActor>* RelatedActor = DatasmithSceneActor->RelatedActors.Find(FName(*Name));
			if (RelatedActor)
			{
				ExistingActor = RelatedActor->Get();
			}
		}
		return ExistingActor;
	}

	FGuid BindActorToLevelSequence(AActor* Actor, ULevelSequence* LevelSequence, int32 AnimIndex)
	{
		if (!Actor || !LevelSequence || !LevelSequence->MovieScene)
		{
			return FGuid();
		}

		UMovieScene* MovieScene = LevelSequence->MovieScene;

		// Bind the actor to the level sequence
		// But first, check if there's already a possessable at the current index
		FMovieScenePossessable* OldPossessable = nullptr;
		if (AnimIndex < MovieScene->GetPossessableCount())
		{
			OldPossessable = &MovieScene->GetPossessable(AnimIndex);
		}

		FString ActorLabel = Actor->GetActorLabel();
		FGuid ObjectBinding;
		if (OldPossessable)
		{
			// If so, remove any track associated with it and unbind it from the level sequence
			FGuid OldGuid = OldPossessable->GetGuid();
			UMovieScene3DTransformTrack* TransformTrack = MovieScene->FindTrack<UMovieScene3DTransformTrack>(OldGuid);
			if (TransformTrack)
			{
				MovieScene->RemoveTrack(*TransformTrack);
			}
			LevelSequence->UnbindPossessableObjects(OldGuid);

			// Replace the old possessable with the new one in the MovieScene
			FMovieScenePossessable NewPossessable(ActorLabel, Actor->GetClass());
			MovieScene->ReplacePossessable(OldGuid, NewPossessable);
			ObjectBinding = NewPossessable.GetGuid();
		}
		else
		{
			// No previous possessable, so simply add it to the MovieScene
			ObjectBinding = MovieScene->AddPossessable(ActorLabel, Actor->GetClass());
		}

		// Finally, associate the guid with the actor
		LevelSequence->BindPossessableObject(ObjectBinding, *Actor, Actor->GetWorld());

		return ObjectBinding;
	}

	// Creates duplicates of OriginalTrack for every child (recursively) of ParentActor, adding them to the MovieScene
	void PropagateVisibilityTrackRecursive(AActor* ParentActor, int32 AnimIndex, ULevelSequence* LevelSequence, UMovieScene* MovieScene, UMovieSceneVisibilityTrack* OriginalTrack)
	{
		TArray<AActor*> Children;
		ParentActor->GetAttachedActors(Children, false);
		for (AActor* Child : Children)
		{
			FGuid ChildBinding = BindActorToLevelSequence(Child, LevelSequence, AnimIndex);

			// Can only have one track of a type per binding
			if (UMovieSceneVisibilityTrack* ChildVisibilityTrack = MovieScene->FindTrack<UMovieSceneVisibilityTrack>(ChildBinding))
			{
				MovieScene->RemoveTrack(*ChildVisibilityTrack);
			}

			UMovieSceneVisibilityTrack* ClonedParentTrack = DuplicateObject(OriginalTrack, nullptr);
			MovieScene->AddGivenTrack(ClonedParentTrack, ChildBinding);

			PropagateVisibilityTrackRecursive(Child, ++AnimIndex, LevelSequence, MovieScene, OriginalTrack);
		}
	}

	int32 GetTransformValues(const TSharedRef<IDatasmithTransformAnimationElement>& Animation, EDatasmithTransformType TransformType, const FFrameRate& SrcFrameRate, const FFrameRate& DestFrameRate, TArray<FFrameNumber>& FrameNumbers, TArray<double>& XValues, TArray<double>& YValues, TArray<double>& ZValues, FFrameNumber& MinFrameNumber, FFrameNumber& MaxFrameNumber)
	{
		int32 NumFrames = Animation->GetFramesCount(TransformType);
		FrameNumbers.Reset(NumFrames);
		XValues.Reset(NumFrames);
		YValues.Reset(NumFrames);
		ZValues.Reset(NumFrames);

		for (int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
		{
			const FDatasmithTransformFrameInfo& FrameInfo = Animation->GetFrame(TransformType, FrameIndex);
			if (FrameInfo.IsValid())
			{
				// Express the source frame number as time using the source frame rate
				float FrameTime = SrcFrameRate.AsSeconds(FrameInfo.FrameNumber);

				// Then, convert the frame time to destination frame number using the destination frame rate
				FFrameNumber FrameNumber = DestFrameRate.AsFrameNumber(FrameTime);

				// Update the min/max frame number
				if (FrameNumber < MinFrameNumber)
				{
					MinFrameNumber = FrameNumber;
				}
				if (FrameNumber > MaxFrameNumber)
				{
					MaxFrameNumber = FrameNumber;
				}

				FrameNumbers.Add(FrameNumber);
				XValues.Add(FrameInfo.X);
				YValues.Add(FrameInfo.Y);
				ZValues.Add(FrameInfo.Z);
			}
		}
		return FrameNumbers.Num();
	}

	int32 GetVisibilityValues(const TSharedRef<IDatasmithVisibilityAnimationElement>& Animation, const FFrameRate& SrcFrameRate, const FFrameRate& DestFrameRate, TArray<FFrameNumber>& FrameNumbers, TArray<bool>& Values, FFrameNumber& MinFrameNumber, FFrameNumber& MaxFrameNumber)
	{
		int32 NumFrames = Animation->GetFramesCount();
		FrameNumbers.Reset(NumFrames);
		Values.Reset(NumFrames);

		for (int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
		{
			const FDatasmithVisibilityFrameInfo& FrameInfo = Animation->GetFrame(FrameIndex);
			if (FrameInfo.IsValid())
			{
				// Express the source frame number as time using the source frame rate
				float FrameTime = SrcFrameRate.AsSeconds(FrameInfo.FrameNumber);

				// Then, convert the frame time to destination frame number using the destination frame rate
				FFrameNumber FrameNumber = DestFrameRate.AsFrameNumber(FrameTime);

				// Update the min/max frame number
				if (FrameNumber < MinFrameNumber)
				{
					MinFrameNumber = FrameNumber;
				}
				if (FrameNumber > MaxFrameNumber)
				{
					MaxFrameNumber = FrameNumber;
				}

				FrameNumbers.Add(FrameNumber);
				Values.Add(FrameInfo.bVisible);
			}
		}
		return FrameNumbers.Num();
	}

	void SetTransformChannels(TArrayView<FMovieSceneDoubleChannel*>& Channels, EDatasmithTransformType TransformType, ERichCurveInterpMode Interpolation, const TArray<FFrameNumber>& FrameNumbers, TArray<double>& XValues, TArray<double>& YValues, TArray<double>& ZValues)
	{
		int8 ChannelIndex = 0;

		switch (TransformType)
		{
		case EDatasmithTransformType::Translation:
			ChannelIndex = 0;
			break;
		case EDatasmithTransformType::Rotation:
			{
				ChannelIndex = 3;

				//  Rewind the rotations to prevent rotations that look like axis flips (eg. from 180 to -180 degrees)
				int32 NumFrames = FrameNumbers.Num();
				for (int32 FrameIndex = 0; FrameIndex < NumFrames - 1; ++FrameIndex)
				{
					FMath::WindRelativeAnglesDegrees(XValues[FrameIndex], XValues[FrameIndex + 1]);
					FMath::WindRelativeAnglesDegrees(YValues[FrameIndex], YValues[FrameIndex + 1]);
					FMath::WindRelativeAnglesDegrees(ZValues[FrameIndex], ZValues[FrameIndex + 1]);
				}
			}
			break;
		case EDatasmithTransformType::Scale:
			ChannelIndex = 6;
			break;
		default:
			break;
		}

		// The input double values must be converted to MovieSceneDoubleValue with the given interpolation
		auto FloatConverter = [Interpolation](double InValue)
		{
			FMovieSceneDoubleValue NewValue(InValue);
			NewValue.InterpMode = Interpolation;
			return NewValue;
		};

		TArray<FMovieSceneDoubleValue> MovieSceneDoubleValues;

		// X channel
		Algo::Transform(XValues, MovieSceneDoubleValues, FloatConverter);
		Channels[ChannelIndex]->Set(FrameNumbers, MoveTemp(MovieSceneDoubleValues));

		// Y channel
		MovieSceneDoubleValues.Reset(FrameNumbers.Num());
		Algo::Transform(YValues, MovieSceneDoubleValues, FloatConverter);
		Channels[ChannelIndex + 1]->Set(FrameNumbers, MoveTemp(MovieSceneDoubleValues));

		// Z channel
		MovieSceneDoubleValues.Reset(FrameNumbers.Num());
		Algo::Transform(ZValues, MovieSceneDoubleValues, FloatConverter);
		Channels[ChannelIndex + 2]->Set(FrameNumbers, MoveTemp(MovieSceneDoubleValues));
	}

	void SetVisibilityChannels(TArrayView<FMovieSceneBoolChannel*>& Channels, ERichCurveInterpMode Interpolation, const TArray<FFrameNumber>& FrameNumbers, TArray<bool>& Values)
	{
		TMovieSceneChannelData<bool> Data = Channels[0]->GetData();

		Data.Reset();

		for (int32 KeyIndex = 0; KeyIndex < FrameNumbers.Num(); ++KeyIndex)
		{
			Data.AddKey(FrameNumbers[KeyIndex], Values[KeyIndex]);
		}
	}

	void PopulateTransformTrack(const TSharedRef<IDatasmithTransformAnimationElement>& TransformAnimation, int32 AnimIndex, FDatasmithImportContext& ImportContext, ULevelSequence* LevelSequence, FFrameNumber& MinFrameNumber, FFrameNumber& MaxFrameNumber)
	{
		AActor* Actor = FindActorByName(TransformAnimation->GetName(), ImportContext);
		if (!Actor)
		{
			ImportContext.LogError( FText::Format( LOCTEXT("MissingActor", "Could not find actor {0} for Level Sequence {1}"), FText::FromString( TransformAnimation->GetName() ), FText::FromString( LevelSequence->GetName() ) ) );
			return;
		}

		FGuid ObjectBinding = BindActorToLevelSequence(Actor, LevelSequence, AnimIndex);

		// Create track and 3D transform section associated with that object binding
		UMovieScene* MovieScene = LevelSequence->MovieScene;
		UMovieScene3DTransformTrack* TransformTrack = MovieScene->FindTrack<UMovieScene3DTransformTrack>(ObjectBinding);
		if (!TransformTrack)
		{
			TransformTrack = MovieScene->AddTrack<UMovieScene3DTransformTrack>(ObjectBinding);
		}
		else
		{
			TransformTrack->RemoveAllAnimationData();
		}

		if (!TransformTrack)
		{
			return;
		}

		bool bSectionAdded = false;
		UMovieScene3DTransformSection* TransformSection = Cast<UMovieScene3DTransformSection>(TransformTrack->FindOrAddSection(0, bSectionAdded));
		if (!TransformSection)
		{
			return;
		}

		static_assert((uint8)EDatasmithCompletionMode::KeepState == (uint8)EMovieSceneCompletionMode::KeepState, "INVALID_ENUM_VALUE");
		static_assert((uint8)EDatasmithCompletionMode::RestoreState == (uint8)EMovieSceneCompletionMode::RestoreState, "INVALID_ENUM_VALUE");
		static_assert((uint8)EDatasmithCompletionMode::ProjectDefault == (uint8)EMovieSceneCompletionMode::ProjectDefault, "INVALID_ENUM_VALUE");

		if (bSectionAdded)
		{
			TransformSection->SetRange(TRange<FFrameNumber>::All());
			TransformSection->EvalOptions.CompletionMode = (EMovieSceneCompletionMode) TransformAnimation->GetCompletionMode();
		}

		// Populate the 3D transform section with the keyframes for each transform type
		TArray<FFrameNumber> FrameNumbers;
		TArray<double> XValues;
		TArray<double> YValues;
		TArray<double> ZValues;

		const FFrameRate FrameRate = MovieScene->GetDisplayRate();
		const FFrameRate TickResolution = MovieScene->GetTickResolution();

		TArrayView<FMovieSceneDoubleChannel*> Channels = TransformSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();

		static_assert((int)ERichCurveInterpMode::RCIM_Linear == (int)EDatasmithCurveInterpMode::Linear, "INVALID_ENUM_VALUE");
		static_assert((int)ERichCurveInterpMode::RCIM_Constant == (int)EDatasmithCurveInterpMode::Constant, "INVALID_ENUM_VALUE");
		static_assert((int)ERichCurveInterpMode::RCIM_Cubic == (int)EDatasmithCurveInterpMode::Cubic, "INVALID_ENUM_VALUE");

		if (GetTransformValues(TransformAnimation, EDatasmithTransformType::Translation, FrameRate, TickResolution, FrameNumbers, XValues, YValues, ZValues, MinFrameNumber, MaxFrameNumber))
		{
			ERichCurveInterpMode InterpMode = (ERichCurveInterpMode)TransformAnimation->GetCurveInterpMode(EDatasmithTransformType::Translation);
			SetTransformChannels(Channels, EDatasmithTransformType::Translation, InterpMode, FrameNumbers, XValues, YValues, ZValues);
		}

		if (GetTransformValues(TransformAnimation, EDatasmithTransformType::Rotation, FrameRate, TickResolution, FrameNumbers, XValues, YValues, ZValues, MinFrameNumber, MaxFrameNumber))
		{
			ERichCurveInterpMode InterpMode = (ERichCurveInterpMode)TransformAnimation->GetCurveInterpMode(EDatasmithTransformType::Rotation);
			SetTransformChannels(Channels, EDatasmithTransformType::Rotation, InterpMode, FrameNumbers, XValues, YValues, ZValues);
		}

		if (GetTransformValues(TransformAnimation, EDatasmithTransformType::Scale, FrameRate, TickResolution, FrameNumbers, XValues, YValues, ZValues, MinFrameNumber, MaxFrameNumber))
		{
			ERichCurveInterpMode InterpMode = (ERichCurveInterpMode)TransformAnimation->GetCurveInterpMode(EDatasmithTransformType::Scale);
			SetTransformChannels(Channels, EDatasmithTransformType::Scale, InterpMode, FrameNumbers, XValues, YValues, ZValues);
		}

		static_assert((uint32)EMovieSceneTransformChannel::TranslationX == (uint32)EDatasmithTransformChannels::TranslationX, "INVALID_ENUM_VALUE");
		static_assert((uint32)EMovieSceneTransformChannel::TranslationY == (uint32)EDatasmithTransformChannels::TranslationY, "INVALID_ENUM_VALUE");
		static_assert((uint32)EMovieSceneTransformChannel::TranslationZ == (uint32)EDatasmithTransformChannels::TranslationZ, "INVALID_ENUM_VALUE");
		static_assert((uint32)EMovieSceneTransformChannel::RotationX == (uint32)EDatasmithTransformChannels::RotationX, "INVALID_ENUM_VALUE");
		static_assert((uint32)EMovieSceneTransformChannel::RotationY == (uint32)EDatasmithTransformChannels::RotationY, "INVALID_ENUM_VALUE");
		static_assert((uint32)EMovieSceneTransformChannel::RotationZ == (uint32)EDatasmithTransformChannels::RotationZ, "INVALID_ENUM_VALUE");
		static_assert((uint32)EMovieSceneTransformChannel::ScaleX == (uint32)EDatasmithTransformChannels::ScaleX, "INVALID_ENUM_VALUE");
		static_assert((uint32)EMovieSceneTransformChannel::ScaleY == (uint32)EDatasmithTransformChannels::ScaleY, "INVALID_ENUM_VALUE");
		static_assert((uint32)EMovieSceneTransformChannel::ScaleZ == (uint32)EDatasmithTransformChannels::ScaleZ, "INVALID_ENUM_VALUE");

		FKeyDataOptimizationParams OptimParams;
		OptimParams.DisplayRate = MovieScene->GetDisplayRate();
		for (FMovieSceneDoubleChannel* Channel : Channels)
		{
			Channel->Optimize(OptimParams);
		}

		TransformSection->SetMask((EMovieSceneTransformChannel)TransformAnimation->GetEnabledTransformChannels());

		if (USceneComponent* SceneComp = Actor->GetRootComponent())
		{
			SceneComp->SetMobility(EComponentMobility::Movable);
		}

		if (ADatasmithAreaLightActor* DatasmithAreaLightActor = Cast<ADatasmithAreaLightActor>(Actor))
		{
			DatasmithAreaLightActor->Mobility = EComponentMobility::Movable;
		}
	}

	void PopulateVisibilityTrack(const TSharedRef<IDatasmithVisibilityAnimationElement>& VisibilityAnimation, int32 AnimIndex, FDatasmithImportContext& ImportContext, ULevelSequence* LevelSequence, FFrameNumber& MinFrameNumber, FFrameNumber& MaxFrameNumber)
	{
		AActor* Actor = FindActorByName(VisibilityAnimation->GetName(), ImportContext);
		if (!Actor)
		{
			ImportContext.LogError( FText::Format( LOCTEXT("MissingActor", "Could not find actor {0} for Level Sequence {1}"), FText::FromString( VisibilityAnimation->GetName() ), FText::FromString( LevelSequence->GetName() ) ) );
			return;
		}

		FGuid ObjectBinding = BindActorToLevelSequence(Actor, LevelSequence, AnimIndex);

		// Create track and visibility section associated with that object binding
		UMovieScene* MovieScene = LevelSequence->MovieScene;
		UMovieSceneVisibilityTrack* VisibilityTrack = MovieScene->FindTrack<UMovieSceneVisibilityTrack>(ObjectBinding);
		if (!VisibilityTrack)
		{
			VisibilityTrack = MovieScene->AddTrack<UMovieSceneVisibilityTrack>(ObjectBinding);
		}
		else
		{
			VisibilityTrack->RemoveAllAnimationData();
		}

		if (!VisibilityTrack)
		{
			return;
		}

		bool bSectionAdded = false;
		UMovieSceneBoolSection* Section = Cast<UMovieSceneBoolSection>(VisibilityTrack->FindOrAddSection(0, bSectionAdded));
		if (!Section)
		{
			return;
		}

		if (bSectionAdded)
		{
			Section->SetRange(TRange<FFrameNumber>::All());
			Section->EvalOptions.CompletionMode = (EMovieSceneCompletionMode) VisibilityAnimation->GetCompletionMode();
		}

		// Populate the section with the keyframes
		TArray<FFrameNumber> FrameNumbers;
		TArray<bool> Values;

		const FFrameRate FrameRate = MovieScene->GetDisplayRate();
		const FFrameRate TickResolution = MovieScene->GetTickResolution();

		TArrayView<FMovieSceneBoolChannel*> Channels = Section->GetChannelProxy().GetChannels<FMovieSceneBoolChannel>();

		static_assert((int)ERichCurveInterpMode::RCIM_Linear == (int)EDatasmithCurveInterpMode::Linear, "INVALID_ENUM_VALUE");
		static_assert((int)ERichCurveInterpMode::RCIM_Constant == (int)EDatasmithCurveInterpMode::Constant, "INVALID_ENUM_VALUE");
		static_assert((int)ERichCurveInterpMode::RCIM_Cubic == (int)EDatasmithCurveInterpMode::Cubic, "INVALID_ENUM_VALUE");

		if (GetVisibilityValues(VisibilityAnimation, FrameRate, TickResolution, FrameNumbers, Values, MinFrameNumber, MaxFrameNumber))
		{
			ERichCurveInterpMode InterpMode = (ERichCurveInterpMode)VisibilityAnimation->GetCurveInterpMode();
			SetVisibilityChannels(Channels, InterpMode, FrameNumbers, Values);
		}

		FKeyDataOptimizationParams OptimParams;
		OptimParams.DisplayRate = MovieScene->GetDisplayRate();
		for (FMovieSceneBoolChannel* Channel : Channels)
		{
			Channel->Optimize(OptimParams);
		}

		if (VisibilityAnimation->GetPropagateToChildren())
		{
			PropagateVisibilityTrackRecursive(Actor, AnimIndex + 1, LevelSequence, MovieScene, VisibilityTrack);
		}

		if (USceneComponent* SceneComp = Actor->GetRootComponent())
		{
			SceneComp->SetMobility(EComponentMobility::Movable);
		}
	}

	void PopulateSubsequenceTrack(const TSharedRef<IDatasmithSubsequenceAnimationElement>& SubsequenceAnimation, bool bShouldClearReferenceTrack, FDatasmithImportContext& ImportContext, ULevelSequence* LevelSequence, FFrameNumber& MinFrameNumber, FFrameNumber& MaxFrameNumber)
	{
		// Find the TargetMovieSceneSequence that will be added as a subsequence
		TSharedPtr<IDatasmithLevelSequenceElement> PinnedTargetSequence = SubsequenceAnimation->GetSubsequence().Pin();
		if (!PinnedTargetSequence.IsValid())
		{
			ImportContext.LogError( FText::Format( LOCTEXT("InvalidTargetSequence", "Could not parse subsequence animation {0} as its target IDatasmithLevelSequenceElement is invalid!"), FText::FromString( SubsequenceAnimation->GetName() ) ) );
			return;
		}
		auto* FoundTargetSequenceAsset = ImportContext.ImportedLevelSequences.Find(PinnedTargetSequence.ToSharedRef());
		if (!FoundTargetSequenceAsset)
		{
			ImportContext.LogError( FText::Format( LOCTEXT("ULevelSequenceNotParsedYet", "Could not parse subsequence animation {0} as its target ULevelSequence has not been imported!"), FText::FromString( SubsequenceAnimation->GetName() ) ) );
			return;
		}
		ULevelSequence* TargetSequenceAsset = *FoundTargetSequenceAsset;
		if (!TargetSequenceAsset)
		{
			ImportContext.LogError( FText::Format( LOCTEXT("NullptrAsset", "Could not parse subsequence animation {0} as its target ULevelSequence is nullptr!"), FText::FromString( SubsequenceAnimation->GetName() ) ) );
			return;
		}
		UMovieSceneSequence* TargetMovieSceneSequence = CastChecked<UMovieSceneSequence>(TargetSequenceAsset);

		// Create SubTrack
		UMovieScene* MovieScene = LevelSequence->GetMovieScene();
		UMovieSceneSubTrack* SubTrack = MovieScene->FindTrack<UMovieSceneSubTrack>();
		if (!SubTrack)
		{
			SubTrack = MovieScene->AddTrack<UMovieSceneSubTrack>();
		}
		else if (SubTrack && bShouldClearReferenceTrack)
		{
			SubTrack->RemoveAllAnimationData();
		}

		if (!SubTrack)
		{
			return;
		}

		FFrameRate DisplayRate = MovieScene->GetDisplayRate();
		FFrameRate TickResolution = MovieScene->GetTickResolution();
		float SrcLowerBound = DisplayRate.AsSeconds(SubsequenceAnimation->GetStartTime());
		float SrcUpperBound = DisplayRate.AsSeconds(SubsequenceAnimation->GetStartTime() + SubsequenceAnimation->GetDuration());
		FFrameNumber DstLowerBound = TickResolution.AsFrameNumber(SrcLowerBound);
		FFrameNumber DstUpperBound = TickResolution.AsFrameNumber(SrcUpperBound);

		// Internally AddSequenceOnRow will automatically bump overlapping subsequences, so we can just add where it's ideal for us
		UMovieSceneSubSection* NewSection = SubTrack->AddSequenceOnRow(TargetMovieSceneSequence, DstLowerBound, DstUpperBound.Value - DstLowerBound.Value, INDEX_NONE);
		NewSection->Parameters.TimeScale = SubsequenceAnimation->GetTimeScale();
		NewSection->EvalOptions.CompletionMode = (EMovieSceneCompletionMode) SubsequenceAnimation->GetCompletionMode();

		MinFrameNumber = FMath::Min(MinFrameNumber, NewSection->GetRange().GetLowerBoundValue());
		MaxFrameNumber = FMath::Max(MaxFrameNumber, NewSection->GetRange().GetUpperBoundValue());
	}

	void PopulateLevelSequence(const TSharedRef<IDatasmithLevelSequenceElement>& LevelSequenceElement, FDatasmithImportContext& ImportContext, ULevelSequence* LevelSequence)
	{
		if (!LevelSequence || !LevelSequence->MovieScene)
		{
			return;
		}

		// Min/max frame numbers used to determine the playback range
		FFrameNumber MinFrameNumber;
		FFrameNumber MaxFrameNumber;

		// We might be reusing the LevelSequence asset, so we must reset it before we add our first subsequence.
		// We will add further subsequences onto the same reference track though, so we can't clear again
		bool bShouldClearSubsequenceReferenceTrack = true;

		const int32 NumAnimations = LevelSequenceElement->GetAnimationsCount();
		for (int32 AnimIndex = 0; AnimIndex < NumAnimations; ++AnimIndex)
		{
			TSharedPtr<IDatasmithBaseAnimationElement> AnimationElement = LevelSequenceElement->GetAnimation(AnimIndex);
			if (!AnimationElement)
			{
				continue;
			}

			if (AnimationElement->IsSubType(EDatasmithElementAnimationSubType::TransformAnimation))
			{
				TSharedRef<IDatasmithTransformAnimationElement> TransformAnimation = StaticCastSharedRef<IDatasmithTransformAnimationElement>(AnimationElement.ToSharedRef());
				PopulateTransformTrack(TransformAnimation, AnimIndex, ImportContext, LevelSequence, MinFrameNumber, MaxFrameNumber);
			}
			else if (AnimationElement->IsSubType(EDatasmithElementAnimationSubType::VisibilityAnimation))
			{
				TSharedRef<IDatasmithVisibilityAnimationElement> VisibilityAnimation = StaticCastSharedRef<IDatasmithVisibilityAnimationElement>(AnimationElement.ToSharedRef());
				PopulateVisibilityTrack(VisibilityAnimation, AnimIndex, ImportContext, LevelSequence, MinFrameNumber, MaxFrameNumber);
			}
			else if (AnimationElement->IsSubType(EDatasmithElementAnimationSubType::SubsequenceAnimation))
			{
				TSharedRef<IDatasmithSubsequenceAnimationElement> SubsequenceAnimation = StaticCastSharedRef<IDatasmithSubsequenceAnimationElement>(AnimationElement.ToSharedRef());
				PopulateSubsequenceTrack(SubsequenceAnimation, bShouldClearSubsequenceReferenceTrack, ImportContext, LevelSequence, MinFrameNumber, MaxFrameNumber);

				bShouldClearSubsequenceReferenceTrack = false;
			}
		}

		LevelSequence->MovieScene->SetPlaybackRange(TRange<FFrameNumber>::Inclusive(MinFrameNumber, MaxFrameNumber));
		LevelSequence->MovieScene->SetEvaluationType(EMovieSceneEvaluationType::FrameLocked);
	}
}

ULevelSequence* FDatasmithLevelSequenceImporter::ImportLevelSequence(const TSharedRef<IDatasmithLevelSequenceElement>& LevelSequenceElement, FDatasmithImportContext& ImportContext, ULevelSequence* ExistingLevelSequence)
{
	if (LevelSequenceElement->GetAnimationsCount() == 0)
	{
		return nullptr;
	}

	ULevelSequence* LevelSequence = nullptr;
	UPackage* ImportOuter = ImportContext.AssetsContext.LevelSequencesImportPackage.Get();
	FString SequenceName = ObjectTools::SanitizeObjectName(LevelSequenceElement->GetName());

	if (ExistingLevelSequence)
	{
		// Make sure to close the level sequence editor for the existing level sequence
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllEditorsForAsset(ExistingLevelSequence);

		if (ExistingLevelSequence->GetOuter() != ImportOuter)
		{
			LevelSequence = DuplicateObject<ULevelSequence>(ExistingLevelSequence, ImportOuter, *SequenceName);
		}
		else
		{
			LevelSequence = ExistingLevelSequence;
		}

		LevelSequence->SetFlags(ImportContext.ObjectFlags);
	}
	else
	{
		LevelSequence = NewObject<ULevelSequence>(ImportOuter, *SequenceName, ImportContext.ObjectFlags);
		LevelSequence->Initialize();
	}

	// Set the source frame rate as the display rate for the MovieScene
	FFrameRate FrameRate(DatasmithLevelSequenceImporterImpl::GetFrameRateAsFraction(LevelSequenceElement));
	LevelSequence->MovieScene->SetDisplayRate(FrameRate);

	LevelSequence->MarkPackageDirty();

	DatasmithLevelSequenceImporterImpl::PopulateLevelSequence(LevelSequenceElement, ImportContext, LevelSequence);

	return LevelSequence;
}

bool FDatasmithLevelSequenceImporter::CanImportLevelSequence(const TSharedRef<IDatasmithLevelSequenceElement>& LevelSequenceElement, const FDatasmithImportContext& InImportContext)
{
	const int32 NumAnimations = LevelSequenceElement->GetAnimationsCount();
	if (NumAnimations == 0)
	{
		return true;
	}

	for (int32 AnimIndex = 0; AnimIndex < NumAnimations; ++AnimIndex)
	{
		TSharedPtr<IDatasmithBaseAnimationElement> AnimationElement = LevelSequenceElement->GetAnimation(AnimIndex);
		if (!AnimationElement)
		{
			continue;
		}

		if (AnimationElement->IsSubType(EDatasmithElementAnimationSubType::SubsequenceAnimation))
		{
			TSharedRef<IDatasmithSubsequenceAnimationElement> SubsequenceAnimation = StaticCastSharedRef<IDatasmithSubsequenceAnimationElement>(AnimationElement.ToSharedRef());
			TSharedPtr<IDatasmithLevelSequenceElement> PinnedTargetSequence = SubsequenceAnimation->GetSubsequence().Pin();
			if (PinnedTargetSequence.IsValid())
			{
				if (!InImportContext.ImportedLevelSequences.Contains(PinnedTargetSequence.ToSharedRef()))
				{
					return false;
				}
			}
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
