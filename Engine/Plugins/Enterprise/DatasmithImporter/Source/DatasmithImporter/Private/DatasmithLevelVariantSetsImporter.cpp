// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithLevelVariantSetsImporter.h"

#include "DatasmithDefinitions.h"
#include "DatasmithImportContext.h"
#include "DatasmithImportContext.h"
#include "DatasmithVariantElements.h"
#include "LevelVariantSets.h"
#include "PropertyValue.h"
#include "Variant.h"
#include "VariantObjectBinding.h"
#include "VariantSet.h"
#include "VariantManager.h"
#include "VariantManagerModule.h"
#include "DatasmithSceneActor.h"

#include "CoreMinimal.h"
#include "Editor.h"
#include "ObjectTools.h"
#include "Subsystems/AssetEditorSubsystem.h"

#define LOCTEXT_NAMESPACE "DatasmithLevelVariantSetsImporter"

class FLevelVariantSetsImporterHelper
{
private:
	TSharedPtr<FVariantManager> VariantManager;
	FDatasmithImportContext& ImportContext;

	ULevelVariantSets* LevelVariantSets;
	const TSharedRef<IDatasmithLevelVariantSetsElement>& LevelVariantSetsElement;

	TMap<FName, AActor*> ImportedActorsByOriginalName;
	TMap<TSharedRef<IDatasmithBaseMaterialElement>, UMaterialInterface*> ImportedMaterials;

public:
	FLevelVariantSetsImporterHelper(ULevelVariantSets* InLevelVariantSets,
									const TSharedRef<IDatasmithLevelVariantSetsElement>& InLevelVariantSetsElement,
									FDatasmithImportContext& InImportContext)
		: ImportContext(InImportContext)
		, LevelVariantSets(InLevelVariantSets)
		, LevelVariantSetsElement(InLevelVariantSetsElement)
	{
		IVariantManagerModule& VariantManagerModule = FModuleManager::LoadModuleChecked<IVariantManagerModule>("VariantManager");
		VariantManager = VariantManagerModule.CreateVariantManager(LevelVariantSets);

		// Grouping them by their original name (Tag[0]) is more reliable, as IDatasmithActorElement names and labels
		// get sanitized, and the actors themselves will be renamed for uniqueness before spawning. Our tags will remain
		// untouched.
		// In case the original names weren't unique in the first place, it would be impossible to tell which actor
		// was targetted by the variant anyway, so it's not a problem if we have to just randomly pick one of them (hash collision)
		ImportedActorsByOriginalName.Reset();
		if (ADatasmithSceneActor* SceneActor = ImportContext.ActorsContext.ImportSceneActor)
		{
			for (const TPair<FName, TSoftObjectPtr<AActor>>& Pair : SceneActor->RelatedActors)
			{
				AActor* Actor = Pair.Value.LoadSynchronous();
				if (!Actor || Actor->Tags.Num() == 0)
				{
					continue;
				}

				ImportedActorsByOriginalName.Add(Actor->Tags[0], Actor);
			}
		}

		ImportedMaterials = ObjectPtrDecay(InImportContext.ImportedMaterials);
	}

	void Import()
	{
		for (int32 VarSetCount = 0; VarSetCount < LevelVariantSetsElement->GetVariantSetsCount(); ++VarSetCount)
		{
			TSharedPtr<IDatasmithVariantSetElement> VarSetElement = LevelVariantSetsElement->GetVariantSet(VarSetCount);
			ImportVariantSet(VarSetElement.ToSharedRef());
		}
	}

	void ImportPropertyCapture(const TSharedRef<IDatasmithBasePropertyCaptureElement>& BasePropCaptureElement, UVariantObjectBinding* ParentBinding)
	{
		static_assert((uint8)EDatasmithPropertyCategory::Undefined == (uint8)EPropertyValueCategory::Undefined, "INVALID_ENUM_VALUE");
		static_assert((uint8)EDatasmithPropertyCategory::Generic == (uint8)EPropertyValueCategory::Generic, "INVALID_ENUM_VALUE");
		static_assert((uint8)EDatasmithPropertyCategory::RelativeLocation == (uint8)EPropertyValueCategory::RelativeLocation, "INVALID_ENUM_VALUE");
		static_assert((uint8)EDatasmithPropertyCategory::RelativeRotation == (uint8)EPropertyValueCategory::RelativeRotation, "INVALID_ENUM_VALUE");
		static_assert((uint8)EDatasmithPropertyCategory::RelativeScale3D == (uint8)EPropertyValueCategory::RelativeScale3D, "INVALID_ENUM_VALUE");
		static_assert((uint8)EDatasmithPropertyCategory::Visibility == (uint8)EPropertyValueCategory::Visibility, "INVALID_ENUM_VALUE");
		static_assert((uint8)EDatasmithPropertyCategory::Material == (uint8)EPropertyValueCategory::Material, "INVALID_ENUM_VALUE");
		static_assert((uint8)EDatasmithPropertyCategory::Color == (uint8)EPropertyValueCategory::Color, "INVALID_ENUM_VALUE");
		static_assert((uint8)EDatasmithPropertyCategory::Option == (uint8)EPropertyValueCategory::Option, "INVALID_ENUM_VALUE");

		EDatasmithPropertyCategory Category = BasePropCaptureElement->GetCategory();

		if (BasePropCaptureElement->IsSubType(EDatasmithElementVariantSubType::PropertyCapture))
		{
			TSharedPtr<IDatasmithPropertyCaptureElement> PropCaptureElement = StaticCastSharedRef<IDatasmithPropertyCaptureElement>(BasePropCaptureElement);
			const TArray<uint8>& RecordedData = PropCaptureElement->GetRecordedData();

			UPropertyValue* PropVal = nullptr;

			if (EnumHasAnyFlags(Category, EDatasmithPropertyCategory::Visibility))
			{
				TArray<UPropertyValue*> PropertyValues = VariantManager->CreateVisibilityPropertyCaptures({ParentBinding});
				if (PropertyValues.Num() == 1)
				{
					PropVal = PropertyValues[0];
				}
			}
			else if (EnumHasAnyFlags(Category, EDatasmithPropertyCategory::RelativeLocation))
			{
				TArray<UPropertyValue*> PropertyValues = VariantManager->CreateLocationPropertyCaptures({ParentBinding});
				if (PropertyValues.Num() == 1)
				{
					PropVal = PropertyValues[0];
				}
			}
			else if (EnumHasAnyFlags(Category, EDatasmithPropertyCategory::RelativeRotation))
			{
				TArray<UPropertyValue*> PropertyValues = VariantManager->CreateRotationPropertyCaptures({ParentBinding});
				if (PropertyValues.Num() == 1)
				{
					PropVal = PropertyValues[0];
				}
			}
			else if (EnumHasAnyFlags(Category, EDatasmithPropertyCategory::RelativeScale3D))
			{
				TArray<UPropertyValue*> PropertyValues = VariantManager->CreateScale3DPropertyCaptures({ParentBinding});
				if (PropertyValues.Num() == 1)
				{
					PropVal = PropertyValues[0];
				}
			}

			if (PropVal)
			{
				PropVal->SetRecordedData(RecordedData.GetData(), RecordedData.Num());
			}
		}
		else if (BasePropCaptureElement->IsSubType(EDatasmithElementVariantSubType::ObjectPropertyCapture))
		{
			TSharedPtr<IDatasmithObjectPropertyCaptureElement> PropCaptureElement = StaticCastSharedRef<IDatasmithObjectPropertyCaptureElement>(BasePropCaptureElement);
			TSharedPtr<IDatasmithElement> TargetElement = PropCaptureElement->GetRecordedObject().Pin();

			const UMaterialInterface* TargetMaterial = nullptr;
			if (TargetElement.IsValid() && TargetElement->IsA(EDatasmithElementType::BaseMaterial))
			{
				TSharedPtr<IDatasmithBaseMaterialElement> TargetMaterialElement = StaticCastSharedPtr<IDatasmithBaseMaterialElement>(TargetElement);
				if (TargetMaterialElement.IsValid())
				{
					const UMaterialInterface* const* FoundMaterial = ImportedMaterials.Find(TargetMaterialElement.ToSharedRef());
					if (FoundMaterial == nullptr)
					{
						ImportContext.LogWarning(FText::Format(LOCTEXT("DidNotFindMaterial", "Did not find material '{0}' when creating variant assets"), FText::FromString(TargetElement->GetName())));
						return;
					}
					TargetMaterial = *FoundMaterial;
				}
			}

			UPropertyValue* PropVal = nullptr;

			if (EnumHasAnyFlags(Category, EDatasmithPropertyCategory::Material))
			{
				TArray<UPropertyValue*> PropertyValues = VariantManager->CreateMaterialPropertyCaptures({ParentBinding});
				if (PropertyValues.Num() == 1)
				{
					PropVal = PropertyValues[0];
				}
			}

			if (PropVal)
			{
				PropVal->SetRecordedData((uint8*)&TargetMaterial, sizeof(UMaterialInterface*));
			}
		}
	}

	void ImportBinding(const TSharedRef<IDatasmithActorBindingElement>& BindingElement, UVariant* ParentVar)
	{
		FName OriginalName = NAME_None;
		TSharedPtr<IDatasmithActorElement> BoundActorElement = BindingElement->GetActor();
		if (BoundActorElement.IsValid())
		{
			if (BoundActorElement->GetTagsCount() > 0)
			{
				OriginalName = BoundActorElement->GetTag(0);
			}
		}

		if (OriginalName.IsNone())
		{
			ImportContext.LogWarning(FText::Format(LOCTEXT("DidNotFindActor", "Did not find actor '{0}' when creating variant '{1}'"), FText::FromString(BindingElement->GetName()), ParentVar->GetDisplayText()));
			return;
		}

		if (AActor** BoundActor = ImportedActorsByOriginalName.Find(OriginalName))
		{
			TArray<UVariantObjectBinding*> Bindings = VariantManager->CreateObjectBindings({*BoundActor}, {ParentVar});
			if (Bindings.Num() != 1 || Bindings[0] == nullptr)
			{
				return;
			}

			for (int32 PropIndex = 0; PropIndex < BindingElement->GetPropertyCapturesCount(); ++PropIndex)
			{
				const TSharedPtr<IDatasmithBasePropertyCaptureElement>& PropCaptureElement = BindingElement->GetPropertyCapture(PropIndex);

				ImportPropertyCapture(PropCaptureElement.ToSharedRef(), Bindings[0]);
			}
		}
	}

	void ImportVariant(const TSharedRef<IDatasmithVariantElement>& VarElement, UVariantSet* ParentVarSet)
	{
		UVariant* Var = VariantManager->CreateVariant(ParentVarSet);
		Var->SetDisplayText(FText::FromString(VarElement->GetName()));

		for (int32 BindingCount = 0; BindingCount < VarElement->GetActorBindingsCount(); ++BindingCount)
		{
			const TSharedPtr<IDatasmithActorBindingElement>& BindingElement = VarElement->GetActorBinding(BindingCount);

			ImportBinding(BindingElement.ToSharedRef(), Var);
		}
	}

	void ImportVariantSet(const TSharedRef<IDatasmithVariantSetElement>& VarSetElement)
	{
		UVariantSet* VarSet = VariantManager->CreateVariantSet(LevelVariantSets);
		VarSet->SetDisplayText(FText::FromString(VarSetElement->GetName()));

		for (int32 VarCount = 0; VarCount < VarSetElement->GetVariantsCount(); ++VarCount)
		{
			const TSharedPtr<IDatasmithVariantElement>& VarElement = VarSetElement->GetVariant(VarCount);

			ImportVariant(VarElement.ToSharedRef(), VarSet);
		}
	}
};

namespace FDatasmithLevelVariantSetsImporterImpl
{
	ULevelVariantSets* CreateLevelVariantSets(const FString& AssetName, UPackage* Package, EObjectFlags Flags, ULevelVariantSets* ExistingLevelVariantSets)
	{
		ULevelVariantSets* LevelVariantSets = nullptr;

		if (ExistingLevelVariantSets)
		{
			// Make sure to close the editor for the existing level variant sets
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllEditorsForAsset(ExistingLevelVariantSets);

			if (ExistingLevelVariantSets->GetOuter() != Package)
			{
				LevelVariantSets = DuplicateObject<ULevelVariantSets>(ExistingLevelVariantSets, Package, *AssetName);
			}
			else
			{
				LevelVariantSets = ExistingLevelVariantSets;
			}

			LevelVariantSets->SetFlags(Flags);
		}
		else
		{
			LevelVariantSets = NewObject<ULevelVariantSets>(Package, *AssetName, Flags);
		}

		return LevelVariantSets;
	}
};

ULevelVariantSets* FDatasmithLevelVariantSetsImporter::ImportLevelVariantSets(const TSharedRef<IDatasmithLevelVariantSetsElement>& LevelVariantSetsElement, FDatasmithImportContext& ImportContext, ULevelVariantSets* ExistingLevelVariantSets)
{
	if (LevelVariantSetsElement->GetVariantSetsCount() == 0)
	{
		return nullptr;
	}

	FString LevelVariantSetsName = ObjectTools::SanitizeObjectName(LevelVariantSetsElement->GetName());
	UPackage* ImportOuter = ImportContext.AssetsContext.LevelVariantSetsImportPackage.Get();
	ULevelVariantSets* LevelVariantSets = FDatasmithLevelVariantSetsImporterImpl::CreateLevelVariantSets(LevelVariantSetsName, ImportOuter, ImportContext.ObjectFlags, ExistingLevelVariantSets);
	if (!LevelVariantSets)
	{
		return nullptr;
	}

	FLevelVariantSetsImporterHelper Helper(LevelVariantSets, LevelVariantSetsElement, ImportContext);
	Helper.Import();

	LevelVariantSets->MarkPackageDirty();

	return LevelVariantSets;
}

#undef LOCTEXT_NAMESPACE
