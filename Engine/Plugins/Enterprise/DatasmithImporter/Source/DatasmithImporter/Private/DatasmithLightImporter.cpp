// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithLightImporter.h"

#include "DatasmithActorImporter.h"
#include "DatasmithAreaLightActor.h"
#include "DatasmithImportContext.h"
#include "DatasmithImportOptions.h"
#include "DatasmithMaterialExpressions.h"
#include "DatasmithSceneFactory.h"
#include "Engine/Blueprint.h"
#include "Utility/DatasmithImporterUtils.h"

#include "ObjectTemplates/DatasmithAreaLightActorTemplate.h"
#include "ObjectTemplates/DatasmithLightComponentTemplate.h"
#include "ObjectTemplates/DatasmithPointLightComponentTemplate.h"
#include "ObjectTemplates/DatasmithSceneComponentTemplate.h"
#include "ObjectTemplates/DatasmithSkyLightComponentTemplate.h"

#include "ActorFactories/ActorFactory.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ComponentReregisterContext.h"
#include "Editor.h"
#include "EngineUtils.h"
#include "Internationalization/Internationalization.h"
#include "Logging/LogMacros.h"
#include "ObjectTools.h"
#include "RawMesh.h"

#include "Components/ChildActorComponent.h"
#include "Components/DirectionalLightComponent.h"
#include "Components/LightComponent.h"
#include "Components/LightmassPortalComponent.h"
#include "Components/PointLightComponent.h"
#include "Components/SkyLightComponent.h"
#include "Components/SpotLightComponent.h"
#include "Components/StaticMeshComponent.h"

#include "Editor/GroupActor.h"

#include "Engine/DirectionalLight.h"
#include "Engine/Light.h"
#include "Engine/PointLight.h"
#include "Engine/SkyLight.h"
#include "Engine/SpotLight.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/TextureCube.h"
#include "Engine/TextureLightProfile.h"

#include "Lightmass/LightmassPortal.h"

#include "Materials/Material.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"

#define LOCTEXT_NAMESPACE "DatasmithLightImporter"

namespace
{
	UClass* GetActorClassForLightActorElement( const TSharedRef< IDatasmithLightActorElement >& LightElement )
	{
		UClass* LightActorClass = APointLight::StaticClass();

		if ( LightElement->IsA(EDatasmithElementType::AreaLight) )
		{
			LightActorClass = AActor::StaticClass();
		}
		else if (LightElement->IsA(EDatasmithElementType::LightmassPortal))
		{
			LightActorClass = ALightmassPortal::StaticClass();
		}
		else if (LightElement->IsA(EDatasmithElementType::DirectionalLight))
		{
			LightActorClass = ADirectionalLight::StaticClass();
		}
		else if ( LightElement->IsA( EDatasmithElementType::SpotLight ) )
		{
			LightActorClass = ASpotLight::StaticClass();
		}
		else if ( LightElement->IsA( EDatasmithElementType::PointLight ) )
		{
			LightActorClass = APointLight::StaticClass();
		}

		return LightActorClass;
	}

	EDatasmithAreaLightActorType GetLightActorTypeForLightType( const EDatasmithAreaLightType LightType )
	{
		EDatasmithAreaLightActorType LightActorType = EDatasmithAreaLightActorType::Point;

		switch ( LightType )
		{
		case EDatasmithAreaLightType::Spot:
			LightActorType = EDatasmithAreaLightActorType::Spot;
			break;

		case EDatasmithAreaLightType::Point:
			LightActorType = EDatasmithAreaLightActorType::Point;
			break;

		case EDatasmithAreaLightType::IES_DEPRECATED:
			LightActorType = EDatasmithAreaLightActorType::Point;
			break;

		case EDatasmithAreaLightType::Rect:
			LightActorType = EDatasmithAreaLightActorType::Rect;
			break;
		}

		return LightActorType;
	}

	UTextureLightProfile* GetIesTexture(const TSharedRef< IDatasmithLightActorElement >& LightElement, FDatasmithAssetsImportContext& AssetsContext)
	{
		UTexture* Texture = FDatasmithImporterUtils::FindAsset<UTexture>(AssetsContext, LightElement->GetIesTexturePathName());
		return Cast<UTextureLightProfile>(Texture);
	}
}

AActor* FDatasmithLightImporter::ImportLightActor( const TSharedRef< IDatasmithLightActorElement >& LightElement, FDatasmithImportContext& ImportContext )
{
	if ( ImportContext.Options->LightImportPolicy == EDatasmithImportActorPolicy::Ignore )
	{
		return nullptr;
	}

	AActor* ImportedLightActor = nullptr;
	UClass* LightActorClass = GetActorClassForLightActorElement( LightElement );

	if ( LightElement->IsA( EDatasmithElementType::AreaLight ) )
	{
		TSharedRef< IDatasmithAreaLightElement > AreaLightElement = StaticCastSharedRef< IDatasmithAreaLightElement >( LightElement );

		ImportedLightActor = ImportAreaLightActor( AreaLightElement, ImportContext );

		// In the case of a re-import, the returned value will be null if area light was deleted. Abort import
		if (ImportedLightActor == nullptr)
		{
			return nullptr;
		}
	}
	else
	{
		FQuat LightRotation = LightElement->GetRotation();

		if ( LightElement->GetUseIes() ) // For IES lights that are not area lights, the IES rotation should be baked into the light transform
		{
			FQuat IesLightRotation = LightElement->GetRotation() * LightElement->GetIesRotation();
			LightElement->SetRotation( IesLightRotation );
		}

		AActor* Actor = FDatasmithActorImporter::ImportActor( LightActorClass, LightElement, ImportContext, ImportContext.Options->LightImportPolicy );
		if ( !Actor )
		{
			return nullptr;
		}

		if ( LightElement->IsA( EDatasmithElementType::LightmassPortal ) )
		{
			ALightmassPortal* LightmassPortal = Cast< ALightmassPortal >( Actor );
			ImportedLightActor = LightmassPortal;
		}
		else if ( LightElement->IsA( EDatasmithElementType::DirectionalLight ) )
		{
			ADirectionalLight* DirectionalLight = Cast< ADirectionalLight >( Actor );

			if ( !DirectionalLight )
			{
				return nullptr;
			}

			SetupLightComponent( DirectionalLight->GetLightComponent(), LightElement, ImportContext.AssetsContext );

			ImportedLightActor = DirectionalLight;
		}
		else if ( LightElement->IsA( EDatasmithElementType::SpotLight ) )
		{
			ASpotLight* SpotLight = Cast< ASpotLight >(Actor);

			if ( !SpotLight )
			{
				return nullptr;
			}

			SetupSpotLightComponent( Cast< USpotLightComponent >( SpotLight->GetLightComponent() ), StaticCastSharedRef< IDatasmithSpotLightElement >( LightElement ), ImportContext.AssetsContext );

			ImportedLightActor = SpotLight;
		}
		else if ( LightElement->IsA( EDatasmithElementType::PointLight ) )
		{
			APointLight* PointLight = Cast< APointLight >(Actor);

			if ( !PointLight )
			{
				return nullptr;
			}

			SetupPointLightComponent( Cast< UPointLightComponent >( PointLight->GetLightComponent() ), StaticCastSharedRef< IDatasmithPointLightElement >( LightElement ), ImportContext.AssetsContext );

			ImportedLightActor = PointLight;
		}
	}

	if ( ImportedLightActor && ImportedLightActor->GetRootComponent() && !ImportedLightActor->GetRootComponent()->IsRegistered() )
	{
		ImportedLightActor->GetRootComponent()->RegisterComponent();
	}

	return ImportedLightActor;
}

USceneComponent* FDatasmithLightImporter::ImportLightComponent( const TSharedRef< IDatasmithLightActorElement >& LightElement, FDatasmithImportContext& ImportContext, UObject* Outer, FDatasmithActorUniqueLabelProvider& UniqueNameProvider )
{
	USceneComponent* LightComponent = nullptr;

	if ( LightElement->IsA(EDatasmithElementType::AreaLight) )
	{
		TSharedRef< IDatasmithAreaLightElement > AreaLightElement = StaticCastSharedRef< IDatasmithAreaLightElement >( LightElement );
		LightComponent = ImportAreaLightComponent( AreaLightElement, ImportContext, Outer, UniqueNameProvider );
	}
	else if ( LightElement->IsA( EDatasmithElementType::LightmassPortal ) )
	{
		TSharedRef< IDatasmithLightmassPortalElement > LightmassPortalElement = StaticCastSharedRef< IDatasmithLightmassPortalElement >( LightElement );
		LightComponent = ImportLightmassPortalComponent( LightmassPortalElement, ImportContext, Outer, UniqueNameProvider );
	}
	else if ( LightElement->IsA( EDatasmithElementType::DirectionalLight ) )
	{
		USceneComponent* SceneComponent = FDatasmithActorImporter::ImportSceneComponent( UDirectionalLightComponent::StaticClass(), LightElement, ImportContext, Outer, UniqueNameProvider );

		UDirectionalLightComponent* DirectionalLightComponent = Cast< UDirectionalLightComponent >( SceneComponent );

		if ( !DirectionalLightComponent )
		{
			return nullptr;
		}

		SetupLightComponent( DirectionalLightComponent, LightElement, ImportContext.AssetsContext );

		LightComponent = DirectionalLightComponent;
	}
	else if ( LightElement->IsA( EDatasmithElementType::SpotLight ) || LightElement->IsA( EDatasmithElementType::PointLight ) )
	{
		FQuat LightRotation = LightElement->GetRotation();

		if ( LightElement->GetUseIes() ) // For IES lights that are not area lights, the IES rotation should be baked into the light transform
		{
			FQuat IesLightRotation = LightElement->GetRotation() * LightElement->GetIesRotation();
			LightElement->SetRotation( IesLightRotation );
		}

		if ( LightElement->IsA( EDatasmithElementType::SpotLight ) )
		{
			USceneComponent* SceneComponent = FDatasmithActorImporter::ImportSceneComponent( USpotLightComponent::StaticClass(), LightElement, ImportContext, Outer, UniqueNameProvider );

			USpotLightComponent* SpotLightComponent = Cast< USpotLightComponent >( SceneComponent );

			if ( !SpotLightComponent )
			{
				return nullptr;
			}

			TSharedRef< IDatasmithSpotLightElement > SpotLightElement = StaticCastSharedRef< IDatasmithSpotLightElement >( LightElement );
			SetupSpotLightComponent( SpotLightComponent, SpotLightElement, ImportContext.AssetsContext );

			LightComponent = SpotLightComponent;
		}
		else if ( LightElement->IsA( EDatasmithElementType::PointLight ) )
		{
			USceneComponent* SceneComponent = FDatasmithActorImporter::ImportSceneComponent( UPointLightComponent::StaticClass(), LightElement, ImportContext, Outer, UniqueNameProvider );

			UPointLightComponent* PointLightComponent = Cast< UPointLightComponent >( SceneComponent );

			if ( !PointLightComponent )
			{
				return nullptr;
			}

			SetupPointLightComponent( PointLightComponent, StaticCastSharedRef< IDatasmithPointLightElement >( LightElement ), ImportContext.AssetsContext );

			LightComponent = PointLightComponent;
		}

		LightElement->SetRotation( LightRotation );
	}

	if ( LightComponent )
	{
		LightComponent->RegisterComponent();
	}

	return LightComponent;
}

void FDatasmithLightImporter::SetTextureLightProfile( const TSharedRef< IDatasmithLightActorElement >& LightElement, UDatasmithLightComponentTemplate* LightComponentTemplate, FDatasmithAssetsImportContext& AssetsContext )
{
	if ( !LightComponentTemplate )
	{
		return;
	}

	if ( LightElement->GetUseIes() )
	{
		LightComponentTemplate->bUseIESBrightness = LightElement->GetUseIesBrightness();
		LightComponentTemplate->IESBrightnessScale = LightElement->GetIesBrightnessScale();
		LightComponentTemplate->IESTexture = GetIesTexture(LightElement, AssetsContext);
	}
}

void FDatasmithLightImporter::SetupLightComponent( ULightComponent* LightComponent, const TSharedPtr< IDatasmithLightActorElement >& LightElement, FDatasmithAssetsImportContext& AssetsContext )
{
	if ( !LightComponent )
	{
		return;
	}

	UDatasmithLightComponentTemplate* LightComponentTemplate = NewObject< UDatasmithLightComponentTemplate >( LightComponent );

	LightComponentTemplate->bVisible = LightElement->IsEnabled();
	LightComponentTemplate->Intensity = LightElement->GetIntensity();
	LightComponentTemplate->CastShadows = true;
	LightComponentTemplate->LightColor = LightElement->GetColor().ToFColor( true );
	LightComponentTemplate->bUseTemperature = LightElement->GetUseTemperature();
	LightComponentTemplate->Temperature = LightElement->GetTemperature();

	if ( LightElement->GetLightFunctionMaterial().IsValid() )
	{
		FString BaseName = LightElement->GetLightFunctionMaterial()->GetName();
		FString MaterialName = FPaths::Combine( AssetsContext.MaterialsFinalPackage->GetPathName(), BaseName + TEXT(".") + BaseName );
		UMaterialInterface* Material = Cast< UMaterialInterface >( FSoftObjectPath( *MaterialName ).TryLoad() );

		if ( Material )
		{
			LightComponentTemplate->LightFunctionMaterial = Material;
		}
	}

	if ( LightComponent->IsA< UPointLightComponent >() )
	{
		SetTextureLightProfile( LightElement.ToSharedRef(), LightComponentTemplate, AssetsContext );
	}

	LightComponentTemplate->Apply( LightComponent );
	LightComponent->UpdateColorAndBrightness();
}

void FDatasmithLightImporter::SetupPointLightComponent( UPointLightComponent* PointLightComponent, const TSharedRef< IDatasmithPointLightElement >& PointLightElement, FDatasmithAssetsImportContext& AssetsContext )
{
	if ( !PointLightComponent )
	{
		return;
	}

	SetupLightComponent( PointLightComponent, PointLightElement, AssetsContext );

	UDatasmithPointLightComponentTemplate* PointLightComponentTemplate = NewObject< UDatasmithPointLightComponentTemplate >( PointLightComponent );

	switch ( PointLightElement->GetIntensityUnits() )
	{
	case EDatasmithLightUnits::Candelas:
		PointLightComponentTemplate->IntensityUnits = ELightUnits::Candelas;
		break;
	case EDatasmithLightUnits::Lumens:
		PointLightComponentTemplate->IntensityUnits = ELightUnits::Lumens;
		break;
	case EDatasmithLightUnits::EV:
		PointLightComponentTemplate->IntensityUnits = ELightUnits::EV;
		break;
	default:
		PointLightComponentTemplate->IntensityUnits = ELightUnits::Unitless;
		break;
	}

	if ( PointLightElement->GetSourceRadius() > 0.f )
	{
		PointLightComponentTemplate->SourceRadius = PointLightElement->GetSourceRadius();
	}

	if ( PointLightElement->GetSourceLength() > 0.f )
	{
		PointLightComponentTemplate->SourceLength = PointLightElement->GetSourceLength();
	}

	if ( PointLightElement->GetAttenuationRadius() > 0.f )
	{
		PointLightComponentTemplate->AttenuationRadius = PointLightElement->GetAttenuationRadius();
	}

	PointLightComponentTemplate->Apply( PointLightComponent );
	PointLightComponent->UpdateColorAndBrightness();
}

void FDatasmithLightImporter::SetupSpotLightComponent( USpotLightComponent* SpotLightComponent, const TSharedRef< IDatasmithSpotLightElement >& SpotLightElement, FDatasmithAssetsImportContext& AssetsContext )
{
	SetupPointLightComponent( SpotLightComponent, SpotLightElement, AssetsContext );

	SpotLightComponent->InnerConeAngle = SpotLightElement->GetInnerConeAngle();
	SpotLightComponent->OuterConeAngle = SpotLightElement->GetOuterConeAngle();
}

AActor* FDatasmithLightImporter::ImportAreaLightActor( const TSharedRef< IDatasmithAreaLightElement >& AreaLightElement, FDatasmithImportContext& ImportContext )
{
	AActor* AreaLightActor = CreateAreaLightActor( AreaLightElement, ImportContext );

	return AreaLightActor;
}

USceneComponent* FDatasmithLightImporter::ImportAreaLightComponent( const TSharedRef< IDatasmithAreaLightElement >& AreaLightElement, FDatasmithImportContext& ImportContext, UObject* Outer, FDatasmithActorUniqueLabelProvider& UniqueNameProvider )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithLightImporter::ImportAreaLightComponent);

	UE_LOGF(LogDatasmithImport, Warning, "For %ls, instead of a ChildActorComponent, a DatasmithAreaLightActor will be created instead.", AreaLightElement->GetLabel());

	AActor* AreaLightActor = nullptr;

	// This feature has stopped working in 4.27 :-(
	// For now, just create an actor
	// We will see whether it makes sense to address in future releases
	if (AActor* RootActor = Cast<AActor>(Outer))
	{
		ImportContext.Hierarchy.Push(RootActor->GetRootComponent());

		AreaLightActor = CreateAreaLightActor(AreaLightElement, ImportContext);

		ImportContext.Hierarchy.Pop();
	}

	return AreaLightActor ? AreaLightActor->GetRootComponent() : nullptr;
}

ULightmassPortalComponent* FDatasmithLightImporter::ImportLightmassPortalComponent( const TSharedRef< IDatasmithLightmassPortalElement >& LightElement, FDatasmithImportContext& ImportContext, UObject* Outer, FDatasmithActorUniqueLabelProvider& UniqueNameProvider )
{
	USceneComponent* SceneComponent = FDatasmithActorImporter::ImportSceneComponent( ULightmassPortalComponent::StaticClass(), LightElement, ImportContext, Outer, UniqueNameProvider );

	SceneComponent->RegisterComponent();

	return Cast< ULightmassPortalComponent >( SceneComponent );
}

AActor* FDatasmithLightImporter::CreateAreaLightActor( const TSharedRef< IDatasmithAreaLightElement >& AreaLightElement, FDatasmithImportContext& ImportContext )
{
	FSoftObjectPath LightShapeBlueprintRef = FSoftObjectPath( TEXT("/DatasmithContent/Datasmith/DatasmithArealight.DatasmithArealight") );
	UBlueprint* LightShapeBlueprint = Cast< UBlueprint >( LightShapeBlueprintRef.TryLoad() );

	if (!ensure(LightShapeBlueprint))
	{
		return nullptr;
	}

	static TWeakObjectPtr<UActorFactory> ActorBPFactory;
	if (!ActorBPFactory.IsValid())
	{
		FAssetData AssetData(LightShapeBlueprint);
		FText UnusedErrorMessage;

		const TArray<UActorFactory*>& ActorFactories = GEditor->ActorFactories;
		for (int32 FactoryIdx = 0; FactoryIdx < ActorFactories.Num(); FactoryIdx++)
		{
			UActorFactory* ActorFactory = ActorFactories[FactoryIdx];

			// Check if the actor can be created using this factory, making sure to check for an asset to be assigned from the selector
			if (ActorFactory->CanCreateActorFrom(AssetData, UnusedErrorMessage))
			{
				ActorBPFactory = ActorFactory;
				break;
			}
		}

		if (!ActorBPFactory.IsValid())
		{
			return nullptr;
		}
	}

	ULevel* CurrentLevel = ImportContext.ActorsContext.ImportWorld->GetCurrentLevel();
	ADatasmithAreaLightActor* LightShapeActor = Cast< ADatasmithAreaLightActor >(ActorBPFactory->CreateActor(LightShapeBlueprint, CurrentLevel, FTransform::Identity));

	if (!ensure(LightShapeActor))
	{
		return nullptr;
	}

	LightShapeActor->SetActorLabel(AreaLightElement->GetLabel());
	FDatasmithActorImporter::SetupActorProperties(LightShapeActor, AreaLightElement, ImportContext);
	
	USceneComponent* Parent = ImportContext.Hierarchy.Num() > 0 ? ImportContext.Hierarchy.Top() : nullptr;
	FDatasmithActorImporter::SetupSceneComponent(LightShapeActor->GetRootComponent(), AreaLightElement, Parent);

	SetupAreaLightActor(AreaLightElement, ImportContext, LightShapeActor);

	return LightShapeActor;
}

void FDatasmithLightImporter::SetupAreaLightActor( const TSharedRef< IDatasmithAreaLightElement >& AreaLightElement, FDatasmithImportContext& ImportContext, ADatasmithAreaLightActor* LightShapeActor )
{
	if ( !LightShapeActor )
	{
		return;
	}

	UDatasmithAreaLightActorTemplate* LightActorTemplate = NewObject< UDatasmithAreaLightActorTemplate >( LightShapeActor );

	LightShapeActor->UnregisterAllComponents( true );

	LightActorTemplate->LightType = GetLightActorTypeForLightType( AreaLightElement->GetLightType() );
	LightActorTemplate->LightShape = (EDatasmithAreaLightActorShape)AreaLightElement->GetLightShape();
	LightActorTemplate->Dimensions = FVector2D( AreaLightElement->GetLength(), AreaLightElement->GetWidth() );
	LightActorTemplate->Color = AreaLightElement->GetColor();
	LightActorTemplate->Intensity = AreaLightElement->GetIntensity();
	LightActorTemplate->IntensityUnits = (ELightUnits)AreaLightElement->GetIntensityUnits();

	if ( AreaLightElement->GetUseTemperature() )
	{
		LightActorTemplate->Temperature = AreaLightElement->GetTemperature();
	}

	if ( AreaLightElement->GetUseIes() )
	{
		LightActorTemplate->IESTexture = GetIesTexture( AreaLightElement, ImportContext.AssetsContext );
		LightActorTemplate->bUseIESBrightness = AreaLightElement->GetUseIesBrightness();
		LightActorTemplate->IESBrightnessScale = AreaLightElement->GetIesBrightnessScale();
		LightActorTemplate->Rotation = AreaLightElement->GetIesRotation().Rotator();
	}

	if ( AreaLightElement->GetSourceRadius() > 0.f )
	{
		LightActorTemplate->SourceRadius = AreaLightElement->GetSourceRadius();
	}

	if ( AreaLightElement->GetSourceLength() > 0.f )
	{
		LightActorTemplate->SourceLength = AreaLightElement->GetSourceLength();
	}

	if ( AreaLightElement->GetAttenuationRadius() > 0.f )
	{
		LightActorTemplate->AttenuationRadius = AreaLightElement->GetAttenuationRadius();
	}

	LightActorTemplate->Apply( LightShapeActor );

	LightShapeActor->RegisterAllComponents();

	LightShapeActor->RerunConstructionScripts();
}

AActor* FDatasmithLightImporter::CreateSkyLight(const TSharedPtr< IDatasmithShaderElement >& ShaderElement, FDatasmithImportContext& ImportContext, bool bUseHDRMat)
{
	if ( ImportContext.Options->LightImportPolicy == EDatasmithImportActorPolicy::Ignore )
	{
		return nullptr;
	}

	TSharedRef< IDatasmithDirectionalLightElement > SkyLightElement = FDatasmithSceneFactory::CreateDirectionalLight( TEXT("SkyLight") );

	if ( bUseHDRMat )
	{
		FQuat ActorRotation = FQuat::MakeFromEuler( FVector(0.f, 0.f, ShaderElement->GetEmitTextureSampler().Rotation * 360.f) );
		SkyLightElement->SetRotation( ActorRotation );
	}

	AActor* Actor = FDatasmithActorImporter::ImportActor( ASkyLight::StaticClass(), SkyLightElement, ImportContext, ImportContext.Options->LightImportPolicy );

	ASkyLight* SkyLight = Cast< ASkyLight >( Actor );

	if (SkyLight == nullptr)
	{
		ImportContext.LogError( FText::Format( LOCTEXT( "CreateSkyLight", "Failed to create the sky light {0}" ), FText::FromString( ShaderElement->GetLabel() ) ) );

		return nullptr;
	}

	UDatasmithSceneComponentTemplate* SceneComponentTemplate = NewObject< UDatasmithSceneComponentTemplate >( SkyLight );
	SceneComponentTemplate->Mobility = EComponentMobility::Static;

	UDatasmithSkyLightComponentTemplate* SkyLightComponentTemplate = NewObject< UDatasmithSkyLightComponentTemplate >( SkyLight );

	if (bUseHDRMat)
	{
		FString EmitTexturePath = ShaderElement->GetEmitTexture();

		if ( !EmitTexturePath.IsEmpty() && FPaths::IsRelative( EmitTexturePath ) )
		{
			EmitTexturePath = FPaths::Combine( ImportContext.AssetsContext.TexturesFinalPackage->GetPathName(), ShaderElement->GetEmitTexture() );
		}

		UTextureCube* CubeTexture = Cast< UTextureCube >( FSoftObjectPath( EmitTexturePath ).TryLoad() );

		if (CubeTexture)
		{
			SkyLightComponentTemplate->SourceType = ESkyLightSourceType::SLS_SpecifiedCubemap;
			SkyLightComponentTemplate->CubemapResolution = 512;
			SkyLightComponentTemplate->Cubemap = CubeTexture;
		}
	}
	else
	{
		SkyLightComponentTemplate->Cubemap = nullptr;
	}

	SceneComponentTemplate->Apply( SkyLight->GetLightComponent() );
	SkyLightComponentTemplate->Apply( SkyLight->GetLightComponent() );

	SkyLight->GetLightComponent()->RegisterComponent();

	SkyLight->MarkComponentsRenderStateDirty();
	SkyLight->MarkPackageDirty();

	return Actor;
}

AActor* FDatasmithLightImporter::CreateHDRISkyLight(const TSharedPtr< IDatasmithShaderElement >& ShaderElement, FDatasmithImportContext& ImportContext)
{
	return CreateSkyLight(ShaderElement, ImportContext, true);
}

AActor* FDatasmithLightImporter::CreatePhysicalSky(FDatasmithImportContext& ImportContext)
{
	TSharedPtr< IDatasmithShaderElement > ShaderElement = FDatasmithSceneFactory::CreateShader(TEXT("voiddummymat"));
	return CreateSkyLight(ShaderElement, ImportContext, false);
}

#undef LOCTEXT_NAMESPACE
