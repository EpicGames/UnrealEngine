// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithMaterialExpressions.h"

#include "DatasmithAssetUserData.h"
#include "DatasmithImportContext.h"
#include "DatasmithMaterialElements.h"
#include "DatasmithMaterialsUtils.h"
#include "DatasmithScene.h"
#include "DatasmithTypes.h"
#include "IDatasmithSceneElements.h"
#include "ObjectTemplates/DatasmithMaterialInstanceTemplate.h"

#include "Utility/DatasmithImporterUtils.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "ComponentReregisterContext.h"
#include "MaterialEditingLibrary.h"
#include "MaterialEditorUtilities.h"
#include "ObjectTools.h"

#include "EditorFramework/AssetImportData.h"

#include "Engine/Texture2D.h"
#include "Engine/TextureCube.h"
#include "Engine/TextureLightProfile.h"

#include "Factories/MaterialFactoryNew.h"
#include "Factories/MaterialInstanceConstantFactoryNew.h"
#include "Factories/TextureFactory.h"

#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpressionAdd.h"
#include "Materials/MaterialExpressionConstant.h"
#include "Materials/MaterialExpressionConstant2Vector.h"
#include "Materials/MaterialExpressionConstant3Vector.h"
#include "Materials/MaterialExpressionConstant4Vector.h"
#include "Materials/MaterialExpressionCustom.h"
#include "Materials/MaterialExpressionDesaturation.h"
#include "Materials/MaterialExpressionDivide.h"
#include "Materials/MaterialExpressionFresnel.h"
#include "Materials/MaterialExpressionFunctionOutput.h"
#include "Materials/MaterialExpressionLinearInterpolate.h"
#include "Materials/MaterialExpressionMakeMaterialAttributes.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialExpressionMultiply.h"
#include "Materials/MaterialExpressionNormalize.h"
#include "Materials/MaterialExpressionOneMinus.h"
#include "Materials/MaterialExpressionPower.h"
#include "Materials/MaterialExpressionRotateAboutAxis.h"
#include "Materials/MaterialExpressionRotator.h"
#include "Materials/MaterialExpressionScalarParameter.h"
#include "Materials/MaterialExpressionStaticBoolParameter.h"
#include "Materials/MaterialExpressionSubtract.h"
#include "Materials/MaterialExpressionTextureBase.h"
#include "Materials/MaterialExpressionTextureCoordinate.h"
#include "Materials/MaterialExpressionTextureObject.h"
#include "Materials/MaterialExpressionTextureSampleParameter2D.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Materials/MaterialExpressionVectorParameter.h"
#include "Materials/MaterialExpressionVertexNormalWS.h"
#include "Materials/MaterialExpressionWorldPosition.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialExpressionUtils.h"

#include "Math/UnrealMathUtility.h"
#include "Misc/FileHelper.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "Templates/EnableIf.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/UnrealTypeTraits.h"
#include "Materials/MaterialExpressionCustomOutput.h"

#define MAXIMUM_IOR 16
#define METAL_IOR 8
#define MIXEDMETAL_IOR 4

namespace
{
	UMaterialExpression* CreateMaterialExpression( UObject* MaterialOrFunction, UClass* MaterialExpressionClass )
	{
		if ( UMaterial* Material = Cast< UMaterial >( MaterialOrFunction ) )
		{
			return UMaterialEditingLibrary::CreateMaterialExpression( Material, MaterialExpressionClass );
		}
		else if ( UMaterialFunction* MaterialFunction = Cast< UMaterialFunction >( MaterialOrFunction ) )
		{
			return UMaterialEditingLibrary::CreateMaterialExpressionInFunction( MaterialFunction, MaterialExpressionClass );
		}

		return nullptr;
	}

	template< typename MaterialExpressionType >
	MaterialExpressionType* CreateMaterialExpression( UObject* MaterialOrFunction )
	{
		return Cast< MaterialExpressionType >( CreateMaterialExpression( MaterialOrFunction, MaterialExpressionType::StaticClass() ) );
	}

	UClass* FindClass(const TCHAR* ClassName)
	{
		check(ClassName);

		if (UClass* Result = FindFirstObject<UClass>(ClassName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("Datasmith FindClass")))
		{
			return Result;
		}

		if (UObjectRedirector* RenamedClassRedirector = FindFirstObject<UObjectRedirector>(ClassName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("Datasmith FindClass")))
		{
			return CastChecked<UClass>(RenamedClassRedirector->DestinationObject);
		}

		return nullptr;
	}

	int32 GetNumberOfExpressionInMaterialOrFunction(UObject* MaterialOrFunction)
	{
		if (UMaterial* Material = Cast< UMaterial >(MaterialOrFunction))
		{
			return Material->GetExpressions().Num();
		}
		else if (UMaterialFunction* MaterialFunction = Cast< UMaterialFunction >(MaterialOrFunction))
		{
			return MaterialFunction->GetExpressions().Num();
		}

		return 0;
	}

	UClass* GetGenericExpressionClass(IDatasmithMaterialExpressionGeneric& GenericExpression)
	{
		const TCHAR* MaterialExpressionCharPtr = TEXT("MaterialExpression");
		const int32 MaterialExpressionLength = FCString::Strlen( MaterialExpressionCharPtr );
		const int32 ExpressionNameLength = FCString::Strlen( GenericExpression.GetExpressionName() );

		FString ClassName;
		ClassName.Reserve( MaterialExpressionLength + ExpressionNameLength );
		ClassName.AppendChars( TEXT("MaterialExpression"), MaterialExpressionLength );
		ClassName.AppendChars(  GenericExpression.GetExpressionName(), ExpressionNameLength );
		return FindClass( *ClassName );
	}

	template<EDatasmithMaterialExpressionType ExpressionType>
	FString GetDefaultParameterName();

	template<>
	FString GetDefaultParameterName<EDatasmithMaterialExpressionType::ConstantBool>()
	{
		return TEXT("Bool");
	}

	template<>
	FString GetDefaultParameterName<EDatasmithMaterialExpressionType::Texture>()
	{
		return TEXT("Texture");
	}

	template<>
	FString GetDefaultParameterName<EDatasmithMaterialExpressionType::ConstantColor>()
	{
		return TEXT("Color");
	}
	template<>
	FString GetDefaultParameterName<EDatasmithMaterialExpressionType::ConstantScalar>()
	{
		return TEXT("Scalar");
	}

	template<>
	FString GetDefaultParameterName<EDatasmithMaterialExpressionType::Generic>()
	{
		return TEXT("Generic");
	}

	template<EDatasmithMaterialExpressionType ExpressionType>
	FName GenerateParamName( IDatasmithMaterialExpression& DatasmithExpression, int32 Index )
	{
		FString ParameterNameString;

		if ( FCString::Strlen( DatasmithExpression.GetName() ) > 0 )
		{
			ParameterNameString = DatasmithExpression.GetName();
		}
		else
		{
			ParameterNameString = GetDefaultParameterName<ExpressionType>();
		}

		ParameterNameString += TEXT(" (") + FString::FromInt( Index ) + TEXT(")");

		return FName( *ParameterNameString );
	}

	FString GenerateUniqueMaterialName( const TCHAR* MaterialLabel, UPackage* Package, FDatasmithUniqueNameProvider& UniqueNameProvider )
	{
		FString Label = ObjectTools::SanitizeObjectName( MaterialLabel );
		// Generate unique name from label if valid, otherwise return element's name as it is unique
		int32 MaxCharCount = FDatasmithImporterUtils::GetAssetNameMaxCharCount(Package);
		return UniqueNameProvider.GenerateUniqueName( Label, MaxCharCount );
	}

	template<class IDatasmithMaterialExpression>
	bool ShouldExpressionBeAParameter(IDatasmithMaterialExpression& Expression);

	template<>
	bool ShouldExpressionBeAParameter(IDatasmithMaterialExpressionColor& ExpressionColor)
	{
		return FCString::Strlen( ExpressionColor.GetName() ) != 0;
	}

	template<>
	bool ShouldExpressionBeAParameter(IDatasmithMaterialExpressionScalar& ExpressionScalar)
	{
		return FCString::Strlen( ExpressionScalar.GetName() ) != 0;
	}

	template<>
	bool ShouldExpressionBeAParameter(IDatasmithMaterialExpressionGeneric& ExpressionGeneric)
	{
		if ( UClass* ExpressionClass = GetGenericExpressionClass( ExpressionGeneric ) )
		{
			if ( UMaterialExpression* DefaultExpression = ExpressionClass->GetDefaultObject<UMaterialExpression>() )
			{
				return DefaultExpression->HasAParameterName();
			}
		}

		ensure( false );
		return false;
	}
}

UMaterialExpressionMakeMaterialAttributes* FDatasmithMaterialExpressions::FindOrAddAttributesFromMatFunc(UMaterialFunction* Func)
{
	for (UMaterialExpression* Expression : Func->GetExpressions())
	{
		if (Expression->IsA< UMaterialExpressionMakeMaterialAttributes >())
		{
			return StaticCast< UMaterialExpressionMakeMaterialAttributes* >(Expression);
		}
	}

	UMaterialExpressionFunctionOutput* Output = FindOrAddOutputFromMatFunc( Func );
	UMaterialExpressionMakeMaterialAttributes* Attrib = CreateMaterialExpression<UMaterialExpressionMakeMaterialAttributes>(Func);
	Attrib->ConnectExpression(Output->GetInput(0), 0);

	return Attrib;
}

UMaterialExpressionFunctionOutput* FDatasmithMaterialExpressions::FindOrAddOutputFromMatFunc(UMaterialFunction* Func)
{
	for (UMaterialExpression* Expression : Func->GetExpressions())
	{
		if (Expression->IsA< UMaterialExpressionFunctionOutput >() )
		{
			return StaticCast< UMaterialExpressionFunctionOutput* >(Expression);
		}
	}

	return CreateMaterialExpression< UMaterialExpressionFunctionOutput >( Func );
}

void FDatasmithMaterialExpressions::GetSamplersRecursive(UMaterialExpression* Expression, TArray<UMaterialExpressionTextureSample*>& TextureSamplers)
{
	if (Expression == nullptr)
	{
		return;
	}

	if (Expression->IsA< UMaterialExpressionTextureSample >())
	{
		TextureSamplers.AddUnique((UMaterialExpressionTextureSample*)Expression);
	}

	for (FExpressionInputIterator It{ Expression }; It; ++It)
	{
		if (UMaterialExpression* InputExpression = It->Expression)
		{
			GetSamplersRecursive(InputExpression, TextureSamplers);
		}
	}
}

void FDatasmithMaterialExpressions::GetTextureSamplersMaterial(UMaterialInterface* MaterialInterface, TArray<UMaterialExpressionTextureSample*>& TextureSamplers)
{
	UMaterial* Material = MaterialInterface->GetMaterial();
	Material->EditorX = 0;
	Material->EditorY = 0;

	TArray<FExpressionInput*> ExpressionInputs;
	// all the connections to the material
	for (int32 MaterialPropertyIndex = 0; MaterialPropertyIndex < MP_MAX; ++MaterialPropertyIndex)
	{
		FExpressionInput* FirstLevelExpression = Material->GetExpressionInputForProperty(EMaterialProperty(MaterialPropertyIndex));
		ExpressionInputs.Add(FirstLevelExpression);
	}

	UMaterialEditorOnlyData* MaterialEditorOnly = Material->GetEditorOnlyData();
	if (MaterialEditorOnly->MaterialAttributes.Expression)
	{
		ExpressionInputs.Add(&MaterialEditorOnly->MaterialAttributes);
	}

	GetSamplers(Material->GetExpressions(), ExpressionInputs, TextureSamplers);
}

void FDatasmithMaterialExpressions::GetTextureSamplersFunc(UMaterialFunction* MaterialFunc, TArray<UMaterialExpressionTextureSample*>& TextureSamplers)
{
	UMaterialExpressionMakeMaterialAttributes* Attrib = FindOrAddAttributesFromMatFunc(MaterialFunc);
	TArray<UMaterialExpression*> Expressions;
	Attrib->GetAllInputExpressions(Expressions);
	Attrib->MaterialExpressionEditorX = 0;
	Attrib->MaterialExpressionEditorY = 0;

	TArray<FExpressionInput*> ExpressionInputs;
	ExpressionInputs.Add(&Attrib->BaseColor);
	ExpressionInputs.Add(&Attrib->Metallic);
	ExpressionInputs.Add(&Attrib->Specular);
	ExpressionInputs.Add(&Attrib->Roughness);
	ExpressionInputs.Add(&Attrib->EmissiveColor);
	ExpressionInputs.Add(&Attrib->Opacity);
	ExpressionInputs.Add(&Attrib->OpacityMask);
	ExpressionInputs.Add(&Attrib->Normal);
	ExpressionInputs.Add(&Attrib->WorldPositionOffset);
	ExpressionInputs.Add(&Attrib->SubsurfaceColor);
	ExpressionInputs.Add(&Attrib->ClearCoat);
	ExpressionInputs.Add(&Attrib->ClearCoatRoughness);
	ExpressionInputs.Add(&Attrib->AmbientOcclusion);
	ExpressionInputs.Add(&Attrib->Refraction);
	ExpressionInputs.Add(&Attrib->PixelDepthOffset);

	GetSamplers(Expressions, ExpressionInputs, TextureSamplers);
}

void FDatasmithMaterialExpressions::GetTextureSamplers(UObject* UnrealMatOrFunc, TArray<UMaterialExpressionTextureSample*>& TextureSamplers)
{
	TextureSamplers.Empty();
	UMaterial* UnrealMaterial = nullptr;

	if (UnrealMatOrFunc->IsA< UMaterial >())
	{
		UnrealMaterial = (UMaterial*)UnrealMatOrFunc;
	}

	UMaterialFunction* Func = nullptr;
	if (UnrealMatOrFunc->IsA< UMaterialFunction >())
	{
		Func = (UMaterialFunction*)UnrealMatOrFunc;
	}

	if (UnrealMaterial != nullptr)
	{
		GetTextureSamplersMaterial(UnrealMaterial, TextureSamplers);
	}

	if (Func != nullptr)
	{
		GetTextureSamplersFunc(Func, TextureSamplers);
	}
}

bool FDatasmithMaterialExpressions::MatOrFuncDelExpression(UObject* Object, UMaterialExpression* ToBeConnected)
{
	if (Object->IsA< UMaterial >())
	{
		UMaterial* Mat = (UMaterial*)Object;
		Mat->GetExpressionCollection().RemoveExpression(ToBeConnected);
		return true;
	}
	else if (Object->IsA< UMaterialFunction >())
	{
		UMaterialFunction* Func = (UMaterialFunction*)Object;
		Func->GetExpressionCollection().RemoveExpression(ToBeConnected);
		return true;
	}
	return false;
}

UMaterialExpression* FDatasmithMaterialExpressions::Constant(UObject* UnrealMaterial, double Value)
{
	UMaterialExpressionConstant* ConstantExp = CreateMaterialExpression<UMaterialExpressionConstant>(UnrealMaterial);
	ConstantExp->R = Value;

	return ConstantExp;
}

UMaterialExpression* FDatasmithMaterialExpressions::Multiply(UObject* UnrealMaterial, UMaterialExpression* ToBeConnectedA, double ValueA, UMaterialExpression* ToBeConnectedB, double ValueB)
{
	UMaterialExpressionMultiply* MultiplyExp = CreateMaterialExpression<UMaterialExpressionMultiply>(UnrealMaterial);

	if (ToBeConnectedA)
	{
		ToBeConnectedA->ConnectExpression(MultiplyExp->GetInput(0), 0);
	}
	else
	{
		MultiplyExp->ConstA = ValueA;
	}

	if (ToBeConnectedB)
	{
		ToBeConnectedB->ConnectExpression(MultiplyExp->GetInput(1), 0);
	}
	else
	{
		MultiplyExp->ConstB = ValueB;
	}

	return MultiplyExp;
}

UMaterialExpression* FDatasmithMaterialExpressions::Divide(UObject* UnrealMaterial, UMaterialExpression* ToBeConnectedA, double ValueA, UMaterialExpression* ToBeConnectedB, double ValueB)
{
	UMaterialExpressionDivide* DivideExp = CreateMaterialExpression<UMaterialExpressionDivide>(UnrealMaterial);

	if (ToBeConnectedA)
	{
		ToBeConnectedA->ConnectExpression(DivideExp->GetInput(0), 0);
	}
	else
	{
		DivideExp->ConstA = ValueA;
	}

	if (ToBeConnectedB)
	{
		ToBeConnectedB->ConnectExpression(DivideExp->GetInput(1), 0);
	}
	else
	{
		DivideExp->ConstB = ValueB;
	}

	return DivideExp;
}

UMaterialExpression* FDatasmithMaterialExpressions::Add(UObject* UnrealMaterial, UMaterialExpression* ToBeConnectedA, double ValueA, UMaterialExpression* ToBeConnectedB, double ValueB)
{
	UMaterialExpressionAdd* AddExp = CreateMaterialExpression<UMaterialExpressionAdd>(UnrealMaterial);

	if (ToBeConnectedA)
	{
		ToBeConnectedA->ConnectExpression(AddExp->GetInput(0), 0);
	}
	else
	{
		AddExp->ConstA = ValueA;
	}

	if (ToBeConnectedB)
	{
		ToBeConnectedB->ConnectExpression(AddExp->GetInput(1), 0);
	}
	else
	{
		AddExp->ConstB = ValueB;
	}

	return AddExp;
}

UMaterialExpression* FDatasmithMaterialExpressions::Subtract(UObject* UnrealMaterial, UMaterialExpression* ToBeConnectedA, double ValueA, UMaterialExpression* ToBeConnectedB, double ValueB)
{
	UMaterialExpressionSubtract* SupExp = CreateMaterialExpression<UMaterialExpressionSubtract>(UnrealMaterial);

	if (ToBeConnectedA)
	{
		ToBeConnectedA->ConnectExpression(SupExp->GetInput(0), 0);
	}
	else
	{
		SupExp->ConstA = ValueA;
	}

	if (ToBeConnectedB)
	{
		ToBeConnectedB->ConnectExpression(SupExp->GetInput(1), 0);
	}
	else
	{
		SupExp->ConstB = ValueB;
	}

	return SupExp;
}

UMaterialExpression* FDatasmithMaterialExpressions::Lerp(UObject* UnrealMaterial, UMaterialExpression* ToBeConnectedA, double ValueA, bool bAllowCleanUpA, UMaterialExpression* ToBeConnectedB, double ValueB, bool bAllowCleanUpB,
														UMaterialExpression* Alpha, double ValueAlpha)
{
	if (Alpha == nullptr && ValueAlpha == 0.0)
	{
		if (ToBeConnectedB && bAllowCleanUpB)
		{
			MatOrFuncDelExpression(UnrealMaterial, ToBeConnectedB);
			ToBeConnectedB->RemoveFromRoot();
		}
		if (ToBeConnectedA != nullptr)
		{
			return ToBeConnectedA;
		}
		else
		{
			return Constant(UnrealMaterial, ValueA);
		}
	}

	if (Alpha == nullptr && ValueAlpha == 1.0)
	{
		if (ToBeConnectedA && bAllowCleanUpA)
		{
			MatOrFuncDelExpression(UnrealMaterial, ToBeConnectedA);
		}
		if (ToBeConnectedB != nullptr)
		{
			return ToBeConnectedB;
		}
		else
		{
			return Constant(UnrealMaterial, ValueB);
		}
	}

	if (ToBeConnectedA == ToBeConnectedB && ToBeConnectedA != nullptr)
	{
		if (ToBeConnectedB && bAllowCleanUpB)
		{
			MatOrFuncDelExpression(UnrealMaterial, ToBeConnectedB);
		}
		return ToBeConnectedA;
	}

	if (ToBeConnectedA == nullptr && ToBeConnectedB == nullptr && ValueA == ValueB)
	{
		return Constant(UnrealMaterial, ValueA);
	}

	UMaterialExpressionLinearInterpolate* LerpExpression = CreateMaterialExpression<UMaterialExpressionLinearInterpolate>(UnrealMaterial);

	bool bInvertOrder = false;
	if (ToBeConnectedA && ToBeConnectedB)
	{
		if (ToBeConnectedA->GetOutputValueType(0) == MCT_Float3)
		{
			bInvertOrder = true;
		}
	}

	if (bInvertOrder == false)
	{
		if (ToBeConnectedA)
		{
			ToBeConnectedA->ConnectExpression(LerpExpression->GetInput(0), 0);
		}
		else
		{
			LerpExpression->ConstA = ValueA;
		}

		if (ToBeConnectedB)
		{
			ToBeConnectedB->ConnectExpression(LerpExpression->GetInput(1), 0);
		}
		else
		{
			LerpExpression->ConstB = ValueB;
		}
	}
	else
	{
		checkSlow(ToBeConnectedA && ToBeConnectedB);
		ToBeConnectedB->ConnectExpression(LerpExpression->GetInput(1), 0);
		ToBeConnectedA->ConnectExpression(LerpExpression->GetInput(0), 0);
	}

	if (Alpha)
	{
		Alpha->ConnectExpression(LerpExpression->GetInput(2), 0);
	}
	else
	{
		LerpExpression->ConstAlpha = ValueAlpha;
	}

	return LerpExpression;
}

UMaterialExpression* FDatasmithMaterialExpressions::HsvExpressionCustom(UObject* UnrealMaterial, UMaterialExpression* ToBeConnectedA, UMaterialExpression* ToBeConnectedB)
{
	UMaterialExpressionCustom* CustomExp = CreateMaterialExpression<UMaterialExpressionCustom>(UnrealMaterial);

	CustomExp->Code = TEXT("\
		//RGB TO HSV\n\
		float4 K = float4(0.0, -1.0 / 3.0, 2.0 / 3.0, -1.0);\n\
		float4 p = rgbinput.g < rgbinput.b ? float4(rgbinput.bg, K.wz) : float4(rgbinput.gb, K.xy);\n\
		float4 q = rgbinput.r < p.x ? float4(p.xyw, rgbinput.r) : float4(rgbinput.r, p.yzx);\n\
		float d = q.x - min(q.w, q.y);\n\
		float e = 1.0e-10;\n\
		float3 hsv = float3(abs(q.z + (q.w - q.y) / (6.0 * d + e)), d / (q.x + e), q.x);\n\
		hsvmod.rgb=(hsvmod.rgb-0.5)*2.0;\n\
		hsv += hsvmod.rgb;\n\
		//HSV TO RGB\n\
		K = float4(1.0, 2.0 / 3.0, 1.0 / 3.0, 3.0);\n\
		float3 p2 = abs(frac(hsv.xxx + K.xyz) * 6.0 - K.www);\n\
		p2 = (hsv.z * Lerp(K.xxx, saturate(p2 - K.xxx), hsv.y));\n\
		return float3(p2.rgb);");
	// return float4(p2.rgb, rgbinput.a); ");

	CustomExp->OutputType = CMOT_Float3;
	CustomExp->Inputs.Empty(); // required: class initializes with one input by default
	int32 IndexA = CustomExp->Inputs.Add({TEXT("rgbinput")});
	int32 IndexB = CustomExp->Inputs.Add({TEXT("hsvmod")});

	CustomExp->GetInput(IndexA)->Expression = ToBeConnectedA;
	CustomExp->GetInput(IndexB)->Expression = ToBeConnectedB;

	return CustomExp;
}

UMaterialExpression* FDatasmithMaterialExpressions::HsvExpression(UObject* UnrealMaterial, UMaterialExpression* ToBeConnectedA, const TArray<double>& Hsv)
{
	FString FunctionString;
	if (Hsv[0] != 0)
	{
		FunctionString = TEXT("/Engine/Functions/Engine_MaterialFunctions02/HueShift.HueShift");
		UMaterialFunction* Huefunc = LoadObject<UMaterialFunction>(nullptr, *FunctionString, nullptr, LOAD_None, nullptr);

		UMaterialExpressionMaterialFunctionCall* ExpHue = CreateMaterialExpression<UMaterialExpressionMaterialFunctionCall>(UnrealMaterial);
		ExpHue->MaterialFunction = Huefunc;
		ExpHue->UpdateFromFunctionResource();

		UMaterialExpression* ConstantHue = Constant(UnrealMaterial, Hsv[0]);
		ConstantHue->Desc = TEXT("Hue Shift");
		ConstantHue->ConnectExpression(ExpHue->GetInput(0), 0);
		ToBeConnectedA->ConnectExpression(ExpHue->GetInput(1), 0);
		ToBeConnectedA = ExpHue;
	}

	if (Hsv[1] != 0.0)
	{
		UMaterialExpression* Desaturation = Desaturate(UnrealMaterial, ToBeConnectedA);
		UMaterialExpression* ConstantSat = Constant(UnrealMaterial, -Hsv[1]);
		ConstantSat->Desc = TEXT("Saturation");
		ConstantSat->ConnectExpression(Desaturation->GetInput(1), 0);
		ToBeConnectedA = Desaturation;
	}

	if (Hsv[2] != 0.0)
	{
		ToBeConnectedA = Add(UnrealMaterial, ToBeConnectedA, 0.0, nullptr, Hsv[2]);
	}

	return ToBeConnectedA;
}

UMaterialExpression* FDatasmithMaterialExpressions::Fresnel(UObject* UnrealMaterial, UMaterialExpression* ToBeConnectedA, double ValueA, UMaterialExpression* ToBeConnectedB, double ValueB)
{
	UMaterialExpressionFresnel* FresnelExp = CreateMaterialExpression<UMaterialExpressionFresnel>(UnrealMaterial);

	if (ToBeConnectedA)
	{
		ToBeConnectedA->ConnectExpression(FresnelExp->GetInput(0), 0);
	}
	else
	{
		FresnelExp->Exponent = ValueA;
	}

	if (ToBeConnectedB)
	{
		ToBeConnectedB->ConnectExpression(FresnelExp->GetInput(1), 0);
	}
	else
	{
		FresnelExp->BaseReflectFraction = ValueB;
	}

	return FresnelExp;
}

UMaterialExpression* FDatasmithMaterialExpressions::Power(UObject* UnrealMaterial, UMaterialExpression* ToBeConnectedA, UMaterialExpression* ToBeConnectedB, double ValueB)
{
	UMaterialExpressionPower* PowerExp = CreateMaterialExpression<UMaterialExpressionPower>(UnrealMaterial);

	ToBeConnectedA->ConnectExpression(PowerExp->GetInput(0), 0);

	if (ToBeConnectedB)
	{
		ToBeConnectedB->ConnectExpression(PowerExp->GetInput(1), 0);
	}
	else
	{
		PowerExp->ConstExponent = ValueB;
	}

	return PowerExp;
}

UMaterialExpression* FDatasmithMaterialExpressions::Desaturate(UObject* UnrealMaterial, UMaterialExpression* ToBeConnected)
{
	if (ToBeConnected == nullptr)
	{
		return nullptr;
	}

	if (UnrealMaterial == nullptr)
	{
		return ToBeConnected;
	}

	UClass* MaterialExpressionDesaturationClass = FindObject<UClass>(nullptr, TEXT("/Script/Engine.MaterialExpressionDesaturation"));

	UMaterialExpression* DesaturateExp = CreateMaterialExpression(UnrealMaterial, MaterialExpressionDesaturationClass);

	UMaterialExpressionDesaturation* DesaturateExpression = (UMaterialExpressionDesaturation*)(DesaturateExp);

	ToBeConnected->ConnectExpression(DesaturateExpression->GetInput(0), 0);

	return DesaturateExpression;
}

UMaterialExpression* FDatasmithMaterialExpressions::OneMinus(UObject* UnrealMaterial, UMaterialExpression* ToBeConnected)
{
	UMaterialExpressionOneMinus* OneMinusExpression = CreateMaterialExpression<UMaterialExpressionOneMinus>(UnrealMaterial);

	ToBeConnected->ConnectExpression(OneMinusExpression->GetInput(0), 0);

	return Multiply(UnrealMaterial, OneMinusExpression, 0.0, OneMinusExpression, 0.0);
}

UMaterialExpression* FDatasmithMaterialExpressions::CalcIORComplex(double IORn, double IORk, UObject* UnrealMaterial, UMaterialExpression* ToBeConnected90, bool bAllowCleanUpExpr90, UMaterialExpression* ToBeConnected0, bool bAllowCleanUpExpr0)
{
	if (IORn >= MAXIMUM_IOR)
	{
		return ToBeConnected90;
	}

	IORn = FMath::Max(IORn, 1.0);
	if (ToBeConnected0 == nullptr)
	{
		// at 0 degrees we use the minumum and the actual input at 90 degrees
		ToBeConnected0 = Multiply(UnrealMaterial, ToBeConnected90, 0, nullptr, 0.1);
	}

	UMaterialExpression* FresnelOrig = Fresnel(UnrealMaterial, nullptr, 1.0, nullptr, 0.0);
	UMaterialExpression* Fresnel = OneMinus(UnrealMaterial, FresnelOrig);
	UMaterialExpression* Fresnel2 = Multiply(UnrealMaterial, Fresnel, 0.0, Fresnel, 0.0);

	UMaterialExpression* ConstantIORn = Constant(UnrealMaterial, IORn);
	ConstantIORn->Desc = TEXT("IOR Value");
	UMaterialExpression* ConstantIORn2 = Multiply(UnrealMaterial, ConstantIORn, 0.0, ConstantIORn, 0.0);

	UMaterialExpression* AddN2K2 = ConstantIORn2;
	if (IORk != 0)
	{
		UMaterialExpression* constantIORk = Constant(UnrealMaterial, IORk);
		UMaterialExpression* constantIORk2 = Multiply(UnrealMaterial, constantIORk, 0.0, constantIORk, 0.0);
		AddN2K2 = Add(UnrealMaterial, ConstantIORn2, 0.0, constantIORk2, 0.0);
	}

	UMaterialExpression* mul_2_n_c_pre = Multiply(UnrealMaterial, ConstantIORn, 0.0, nullptr, 2.0);
	UMaterialExpression* Mul2NC = Multiply(UnrealMaterial, mul_2_n_c_pre, 0.0, Fresnel, 2.0);

	UMaterialExpression* rs_num_pre = Subtract(UnrealMaterial, AddN2K2, 0.0, Mul2NC, 0.0);
	UMaterialExpression* RsNum = Add(UnrealMaterial, rs_num_pre, 0.0, Fresnel2, 0.0);
	UMaterialExpression* rs_den_pre = Add(UnrealMaterial, AddN2K2, 0.0, Mul2NC, 0.0);
	UMaterialExpression* RsDen = Add(UnrealMaterial, rs_den_pre, 0.0, Fresnel2, 0.0);
	UMaterialExpression* Rs = Divide(UnrealMaterial, RsNum, 0.0, RsDen, 0.0);

	UMaterialExpression* add_n2_k2_mul_c2 = Multiply(UnrealMaterial, AddN2K2, 0.0, Fresnel2, 0.0);

	UMaterialExpression* rp_num_pre = Subtract(UnrealMaterial, add_n2_k2_mul_c2, 0.0, Mul2NC, 0.0);
	UMaterialExpression* RpNum = Add(UnrealMaterial, rp_num_pre, 0.0, nullptr, 1.0);
	UMaterialExpression* rp_den_pre = Add(UnrealMaterial, add_n2_k2_mul_c2, 0.0, Mul2NC, 0.0);
	UMaterialExpression* RpDen = Add(UnrealMaterial, rp_den_pre, 0.0, nullptr, 1.0);
	UMaterialExpression* Rp = Divide(UnrealMaterial, RpNum, 0.0, RpDen, 0.0);

	UMaterialExpression* ResPre = Add(UnrealMaterial, Rp, 0.0, Rs, 0.0);
	UMaterialExpression* Res = Multiply(UnrealMaterial, ResPre, 0, nullptr, 0.5);
	Res = Power(UnrealMaterial, Res, nullptr, 0.5);

	return Lerp(UnrealMaterial, ToBeConnected0, 0.0, bAllowCleanUpExpr0, ToBeConnected90, 0.0, bAllowCleanUpExpr90, Res, 0.0);
}

UMaterialExpression* FDatasmithMaterialExpressions::CalcIORSimple(double IORn, double IORk, UObject* UnrealMaterial, UMaterialExpression* ToBeConnected90, bool bAllowCleanUpExpr90, UMaterialExpression* ToBeConnected0, bool bAllowCleanUpExpr0)
{
	// over MAXIMUM_IOR there is no IOR
	// THIS IS ONLY FOR NON-METALS
	if (IORn >= MAXIMUM_IOR)
	{
		return ToBeConnected90;
	}

	IORn = FMath::Max(IORn, 1.0);
	if (ToBeConnected0 == nullptr)
	{
		// at 0 degrees we use the minumum and the actual input at 90 degrees
		ToBeConnected0 = Multiply(UnrealMaterial, ToBeConnected90, 0, nullptr, 0.1);
	}

	double Exponent = MAXIMUM_IOR / IORn;
	Exponent = FMath::Sqrt(Exponent);

	float IORk2 = IORk * IORk;
	float IORn2 = IORn * IORn;
	float C = 1.0f; // frontal
	float C2 = C * C;
	float AddN2K2 = IORn2 + IORk2;
	float Mul2NC = 2 * IORn * C;

	float RsNum = AddN2K2 - Mul2NC + C2;
	float RsDen = AddN2K2 + Mul2NC + C2;
	float Rs = RsNum / RsDen;

	float RpNum = AddN2K2 * C2 - Mul2NC + 1;
	float RpDen = AddN2K2 * C2 + Mul2NC + 1;
	float Rp = RpNum / RpDen;

	float Base = 0.5f * (Rs + Rp);
	Base = (Base + FMath::Sqrt(Base)) / 2.0f;
	Exponent = (Exponent + FMath::Sqrt(Exponent)) / 2.0f;

	Base = FMath::Sqrt(Base);
	Exponent = FMath::Sqrt(Exponent);

	UMaterialExpression* FresnelMatExp = Fresnel(UnrealMaterial, nullptr, Exponent, nullptr, Base);

	return Lerp(UnrealMaterial, ToBeConnected0, 0.0, bAllowCleanUpExpr0, ToBeConnected90, 0.0, bAllowCleanUpExpr90, FresnelMatExp, 0.0);
}

UMaterialExpression* FDatasmithMaterialExpressions::CalcIOR(double IORn, double IORk, UObject* UnrealMaterial, UMaterialExpression* ToBeConnected90, bool bAllowCleanUpExpr90, UMaterialExpression* ToBeConnected0, bool bAllowCleanUpExpr0)
{
	if (IDatasmithShaderElement::bDisableReflectionFresnel && ToBeConnected0 == nullptr) // if no reflection curves and no forced ior as Texmap.
	{
		return ToBeConnected90;
	}

	if (IORn < 1 || IORn > MAXIMUM_IOR)
	{
		return ToBeConnected90;
	}

	if (IDatasmithShaderElement::bUseRealisticFresnel)
	{
		return CalcIORComplex(IORn, IORk, UnrealMaterial, ToBeConnected90, bAllowCleanUpExpr90, ToBeConnected0, bAllowCleanUpExpr0);
	}
	else
	{
		return CalcIORSimple(IORn, IORk, UnrealMaterial, ToBeConnected90, bAllowCleanUpExpr90, ToBeConnected0, bAllowCleanUpExpr0);
	}
}

UMaterialExpression* FDatasmithMaterialExpressions::RefractionIOR(double IOR, UObject* UnrealMaterial)
{
	double Exponent = 5.0;
	double Base = 0.04;
	UMaterialExpression* FresnelMatExp = Fresnel(UnrealMaterial, nullptr, Exponent, nullptr, Base);

	return Lerp(UnrealMaterial, nullptr, 1.0, true, nullptr, FMath::Sqrt(IOR), true, FresnelMatExp, 0.0);
}

EMaterialProperty FDatasmithMaterialExpressions::DatasmithTextureSlotToMaterialProperty(EDatasmithTextureSlot InSlot)
{
	switch (InSlot)
	{
	case EDatasmithTextureSlot::DIFFUSE:				return MP_BaseColor;
	case EDatasmithTextureSlot::METALLIC:				return MP_Metallic;
	case EDatasmithTextureSlot::SPECULAR:				return MP_Specular;
	case EDatasmithTextureSlot::ROUGHNESS:				return MP_Roughness;
	case EDatasmithTextureSlot::EMISSIVECOLOR:			return MP_EmissiveColor;
	case EDatasmithTextureSlot::OPACITY:				return MP_Opacity;
	case EDatasmithTextureSlot::OPACITYMASK:			return MP_OpacityMask;
	case EDatasmithTextureSlot::NORMAL:					return MP_Normal;
	case EDatasmithTextureSlot::WORLDPOSITIONOFFSET:	return MP_WorldPositionOffset;
	case EDatasmithTextureSlot::SUBSURFACECOLOR:		return MP_SubsurfaceColor;
	case EDatasmithTextureSlot::COATSPECULAR:			return MP_CustomData0;
	case EDatasmithTextureSlot::COATROUGHNESS:			return MP_CustomData1;
	case EDatasmithTextureSlot::REFRACTION:				return MP_Refraction;
	case EDatasmithTextureSlot::PIXELDEPTHOFFSET:		return MP_PixelDepthOffset;
	case EDatasmithTextureSlot::SHADINGMODEL:			return MP_ShadingModel;
	case EDatasmithTextureSlot::MATERIALATTRIBUTES:		return MP_MaterialAttributes;
	case EDatasmithTextureSlot::AMBIANTOCCLUSION:		return MP_AmbientOcclusion;
	case EDatasmithTextureSlot::NOSLOT:
		return MP_MAX;
	default:
		ensure(false);
		return MP_MAX;
	}
}

FExpressionInput* FDatasmithMaterialExpressions::GetMaterialOrFunctionSlot( UObject* UnrealMatOrFunc, EDatasmithTextureSlot InSlot )
{
	FExpressionInput* ExpressionInput = nullptr;

	if ( UMaterial* UnrealMaterial = Cast< UMaterial >( UnrealMatOrFunc ) )
	{
		ExpressionInput = UnrealMaterial->GetExpressionInputForProperty( DatasmithTextureSlotToMaterialProperty( InSlot ) );
	}
	else if ( UMaterialFunction* Func = Cast< UMaterialFunction >( UnrealMatOrFunc ) )
	{
		if ( InSlot == EDatasmithTextureSlot::MATERIALATTRIBUTES )
		{
			//If the material function is outputting to MaterialAttributes then we don't need to MakeMatterialAttributes, we can use the function output directly.
			return FindOrAddOutputFromMatFunc(Func)->GetInput(0);
		}

		UMaterialExpressionMakeMaterialAttributes* Attrib = FindOrAddAttributesFromMatFunc( Func );
		switch ( InSlot )
		{
		case EDatasmithTextureSlot::DIFFUSE:
			ExpressionInput = &Attrib->BaseColor;
			break;
		case EDatasmithTextureSlot::METALLIC:
			ExpressionInput = &Attrib->Metallic;
			break;
		case EDatasmithTextureSlot::SPECULAR:
			ExpressionInput = &Attrib->Specular;
			break;
		case EDatasmithTextureSlot::ROUGHNESS:
			ExpressionInput = &Attrib->Roughness;
			break;
		case EDatasmithTextureSlot::EMISSIVECOLOR:
			ExpressionInput = &Attrib->EmissiveColor;
			break;
		case EDatasmithTextureSlot::OPACITY:
			ExpressionInput = &Attrib->Opacity;
			break;
		case EDatasmithTextureSlot::OPACITYMASK:
			ExpressionInput = &Attrib->OpacityMask;
			break;
		case EDatasmithTextureSlot::NORMAL:
			ExpressionInput = &Attrib->Normal;
			break;
		case EDatasmithTextureSlot::WORLDPOSITIONOFFSET:
			ExpressionInput = &Attrib->WorldPositionOffset;
			break;
		case EDatasmithTextureSlot::SUBSURFACECOLOR:
			ExpressionInput = &Attrib->SubsurfaceColor;
			break;
		case EDatasmithTextureSlot::COATSPECULAR:
			ExpressionInput = &Attrib->ClearCoat;
			break;
		case EDatasmithTextureSlot::COATROUGHNESS:
			ExpressionInput = &Attrib->ClearCoatRoughness;
			break;
		case EDatasmithTextureSlot::REFRACTION:
			ExpressionInput = &Attrib->Refraction;
			break;
		case EDatasmithTextureSlot::PIXELDEPTHOFFSET:
			ExpressionInput = &Attrib->PixelDepthOffset;
			break;
		case EDatasmithTextureSlot::SHADINGMODEL:
			ExpressionInput = &Attrib->ShadingModel;
			break;
		case EDatasmithTextureSlot::AMBIANTOCCLUSION:
			ExpressionInput = &Attrib->AmbientOcclusion;
			break;
		default:
			break;
		}
	}

	return ExpressionInput;
}

void FDatasmithMaterialExpressions::ConnectToSlot(UMaterialExpression* ToBeConnected, UObject* UnrealMatOrFunc, EDatasmithTextureSlot Slot, int32 InputChannel)
{
	if (ToBeConnected == nullptr)
	{
		return;
	}

	UMaterial* UnrealMaterial = Cast< UMaterial >( UnrealMatOrFunc );

	UMaterialFunction* Func = Cast< UMaterialFunction >( UnrealMatOrFunc );

	if ((Slot == EDatasmithTextureSlot::METALLIC || Slot == EDatasmithTextureSlot::SPECULAR || Slot == EDatasmithTextureSlot::ROUGHNESS)
	  && !ToBeConnected->IsA< UMaterialExpressionConstant >()
	  && InputChannel == 0)
	{
		if (UnrealMaterial != nullptr)
		{
			ToBeConnected = Desaturate(UnrealMaterial, ToBeConnected);
		}

		if (Func != nullptr)
		{
			ToBeConnected = Desaturate(Func, ToBeConnected);
		}
	}

	FExpressionInput* ExpressionInput = GetMaterialOrFunctionSlot( UnrealMatOrFunc, Slot );
	if ( UnrealMaterial )
	{
		switch (Slot)
		{
		case EDatasmithTextureSlot::OPACITY:
			UnrealMaterial->BlendMode = EBlendMode::BLEND_Translucent;
			break;
		case EDatasmithTextureSlot::OPACITYMASK:
			UnrealMaterial->BlendMode = EBlendMode::BLEND_Masked;
			break;
		}
	}

	if (ExpressionInput != nullptr)
	{
		if (Slot == EDatasmithTextureSlot::OPACITYMASK)
		{
			ExpressionInput->OutputIndex = ToBeConnected->GetOutputs().Num() - 1;
		}

		if (InputChannel > 0 && InputChannel < ToBeConnected->GetOutputs().Num())
		{
			ExpressionInput->OutputIndex = InputChannel;
			int32 Mask[4] = { 0, 0, 0, 0 };
			Mask[InputChannel - 1] = 1;
			ExpressionInput->SetMask(1, Mask[0], Mask[1], Mask[2], Mask[3]);
		}

		ExpressionInput->Expression = ToBeConnected;
	}
}

UMaterialExpressionTextureObject* FDatasmithMaterialExpressions::AddTextureObjectExpression(UTexture* Texture, UObject* UnrealMaterial)
{
	if (Texture != nullptr && UnrealMaterial != nullptr)
	{
		UMaterialExpressionTextureObject* TextureOExpression = CreateMaterialExpression<UMaterialExpressionTextureObject>(UnrealMaterial);
		TextureOExpression->Texture = Texture;
		return TextureOExpression;
	}

	return nullptr;
}

UMaterialExpression* FDatasmithMaterialExpressions::AddTextureExpression(UTexture* Texture, FDatasmithTextureSampler UV, double IOR, double IORk, UObject* UnrealMaterial, EDatasmithTextureSlot Slot, UMaterialExpressionTextureSample** OutTextureSample)
{
	UMaterialExpression* ToBeConnected = nullptr;
	if (Texture != nullptr && UnrealMaterial != nullptr)
	{
		UMaterialExpressionTextureSample* TextureExpression = CreateMaterialExpression<UMaterialExpressionTextureSample>(UnrealMaterial);
		TextureExpression->Texture = Texture;

		if (OutTextureSample != nullptr)
		{
			*OutTextureSample = TextureExpression;
		}

		if (FMath::Abs(UV.ScaleX) < SMALL_NUMBER)
		{
			UV.ScaleX = SMALL_NUMBER;
		}

		if (FMath::Abs(UV.ScaleY) < SMALL_NUMBER)
		{
			UV.ScaleY = SMALL_NUMBER;
		}

		if (UV.CoordinateIndex > 0)
		{
			TextureExpression->ConstCoordinate = UV.CoordinateIndex;
		}

		if (UV.OffsetX != 0.0 || UV.OffsetY != 0.0 || UV.ScaleX != 1.0 || UV.ScaleY != 1.0 || UV.Rotation != 0.0 || UV.MirrorX != 0 || UV.MirrorY != 0)
		{
			UMaterialExpressionTextureCoordinate* TexcoordExpression = CreateMaterialExpression<UMaterialExpressionTextureCoordinate>(UnrealMaterial);
			ToBeConnected = TexcoordExpression;
			TexcoordExpression->CoordinateIndex = UV.CoordinateIndex;

			// Special handling of UV transforms if texture is marked as cropped
			if (UV.bCroppedTexture)
			{
				ToBeConnected = AddCroppedUVMappingExpression(UnrealMaterial, UV, TexcoordExpression);
			}
			else
			{
				TexcoordExpression->UTiling = UV.ScaleX;
				TexcoordExpression->VTiling = UV.ScaleY;

				if (UV.OffsetX != 0 || UV.OffsetY != 0)
				{
					UMaterialExpressionConstant2Vector* Constant2d = CreateMaterialExpression<UMaterialExpressionConstant2Vector>(UnrealMaterial);
					Constant2d->R = UV.OffsetX;
					Constant2d->G = UV.OffsetY;

					ToBeConnected = Add(UnrealMaterial, ToBeConnected, 0.0, Constant2d, 0.0);
				}

				if (UV.MirrorX != 0 || UV.MirrorY != 0)
				{
					ToBeConnected = AddUVMirrorExpression(UnrealMaterial, ToBeConnected, UV.MirrorX, UV.MirrorY);
				}

				if (UV.Rotation != 0)
				{
					UMaterialExpressionConstant* Constant1d = CreateMaterialExpression<UMaterialExpressionConstant>(UnrealMaterial);
					Constant1d->R = UV.Rotation * 2.f * PI;

					UClass* MaterialExpressionRotatorClass = FindObject<UClass>(nullptr, TEXT("/Script/Engine.MaterialExpressionRotator"));

					// This will create a rotator, but the pointer you get back is to Base class.
					UMaterialExpression* RotatorExp = CreateMaterialExpression(UnrealMaterial, MaterialExpressionRotatorClass);

					// I think you can do this, but it's possible Cast requires the class to be exported, not sure.
					UMaterialExpressionRotator* Rotator = (UMaterialExpressionRotator*)(RotatorExp);

					Rotator->Speed = 1.0f;
					Rotator->GetInput(0)->Expression = ToBeConnected;
					Rotator->GetInput(1)->Expression = Constant1d;

					ToBeConnected = Rotator;
				}
			}

			TextureExpression->GetInput(0)->Expression = ToBeConnected;
		}

		ToBeConnected = (UMaterialExpression*)TextureExpression;
		if (UV.Multiplier != 1.0f)
		{
			ToBeConnected = Multiply(UnrealMaterial, ToBeConnected, 0, nullptr, UV.Multiplier);
		}

		if (IOR > 1.0)
		{
			ToBeConnected = CalcIOR(IOR, IORk, UnrealMaterial, ToBeConnected);
		}

		if (UV.bInvert)
		{
			ToBeConnected = OneMinus(UnrealMaterial, ToBeConnected);
		}

		bool bIsLinearColor = !Texture->SRGB;

		// Keeping the check for Bytes/Pixel because not sure how it fits into the check for IsLinearColor
		if (bIsLinearColor && Texture->Source.GetBytesPerPixel() > 4 && Slot != EDatasmithTextureSlot::NORMAL)
		{
			ToBeConnected = Power(UnrealMaterial, ToBeConnected, nullptr, 2.2);
		}

		ConnectToSlot(ToBeConnected, UnrealMaterial, Slot, UV.OutputChannel);

		TextureExpression->SamplerType = MaterialExpressionUtils::GetSamplerTypeForTexture( Texture );
	}

	return ToBeConnected;
}

UMaterialExpression* FDatasmithMaterialExpressions::AddUVMirrorExpression(UObject* UnrealMaterial, UMaterialExpression* Expression, int XTile, int YTile)
{
	FString FunctionString = TEXT("/DatasmithContent/Materials/MirrorUVTiling.MirrorUVTiling");
	UMaterialFunction* UVMirrorFunc = LoadObject<UMaterialFunction>(nullptr, *FunctionString, nullptr, LOAD_None, nullptr);

	UMaterialExpressionMaterialFunctionCall* ExpUVMirror = CreateMaterialExpression<UMaterialExpressionMaterialFunctionCall>(UnrealMaterial);
	ExpUVMirror->MaterialFunction = UVMirrorFunc;
	ExpUVMirror->UpdateFromFunctionResource();

	if (XTile != 0 || YTile != 0)
	{
		UMaterialExpressionConstant2Vector* Constant2d = CreateMaterialExpression<UMaterialExpressionConstant2Vector>(UnrealMaterial);
		Constant2d->R = XTile;
		Constant2d->G = YTile;
		Constant2d->ConnectExpression(ExpUVMirror->GetInput(0), 0);
	}
	Expression->ConnectExpression(ExpUVMirror->GetInput(1), 0);

	return ExpUVMirror;
}

UMaterialExpression* FDatasmithMaterialExpressions::AddConstExpression(double Val, double IOR, double IORk, UObject* UnrealMaterial, EDatasmithTextureSlot Slot)
{
	UMaterialExpression* ToBeConnected = nullptr;
	if (Val != 0.0 && UnrealMaterial != nullptr)
	{
		UMaterialExpressionConstant* ValExpression = CreateMaterialExpression<UMaterialExpressionConstant>(UnrealMaterial);
		ToBeConnected = ValExpression;
		ValExpression->R = Val;

		if (IOR > 1.0)
		{
			ToBeConnected = CalcIOR(IOR, IORk, UnrealMaterial, ToBeConnected);
		}

		ConnectToSlot(ToBeConnected, UnrealMaterial, Slot);
	}
	return ToBeConnected;
}

UMaterialExpression* FDatasmithMaterialExpressions::AddRGBExpression(FLinearColor Col, double IOR, double IORk, UObject* UnrealMaterial, EDatasmithTextureSlot Slot, bool bForceEvenIfBlack)
{
	UMaterialExpression* ToBeConnected = nullptr;
	if ((Col.R > 0.0001 || Col.G > 0.0001 || Col.B > 0.0001 || bForceEvenIfBlack) && UnrealMaterial != nullptr)
	{
		UMaterialExpressionConstant3Vector* ColorExpression = CreateMaterialExpression<UMaterialExpressionConstant3Vector>(UnrealMaterial);
		ToBeConnected = ColorExpression;

		ColorExpression->Constant.R = float(Col.R);
		ColorExpression->Constant.G = float(Col.G);
		ColorExpression->Constant.B = float(Col.B);

		if (IOR > 1.0)
		{
			ToBeConnected = CalcIOR(IOR, IORk, UnrealMaterial, ToBeConnected);
		}

		ConnectToSlot(ToBeConnected, UnrealMaterial, Slot);
	}
	return ToBeConnected;
}

UMaterialExpression* FDatasmithMaterialExpressions::AddRGBAExpression(FLinearColor Col, double IOR, double IORk, UObject* UnrealMaterial, EDatasmithTextureSlot Slot, bool bForceEvenIfBlack)
{
	UMaterialExpression* ToBeConnected = nullptr;
	if ((Col.R > 0.0001 || Col.G > 0.0001 || Col.B > 0.0001 || bForceEvenIfBlack) && UnrealMaterial != nullptr)
	{
		UMaterialExpressionConstant4Vector* ColorExpression = CreateMaterialExpression<UMaterialExpressionConstant4Vector>(UnrealMaterial);
		ToBeConnected = ColorExpression;

		ColorExpression->Constant.R = float(Col.R);
		ColorExpression->Constant.G = float(Col.G);
		ColorExpression->Constant.B = float(Col.B);
		ColorExpression->Constant.A = float(Col.A);

		if (IOR > 1.0)
		{
			ToBeConnected = CalcIOR(IOR, IORk, UnrealMaterial, ToBeConnected);
		}

		ConnectToSlot(ToBeConnected, UnrealMaterial, Slot);
	}
	return ToBeConnected;
}

UMaterialExpression* FDatasmithMaterialExpressions::GetCorrectExpressionFromComp(const TSharedPtr<IDatasmithCompositeTexture>& Comp, UObject* UnrealMaterial, const FDatasmithAssetsImportContext& AssetsContext, int32 Index,
																				bool bMask, bool bRgbNoAlpha)
{
	UTexture* TextureN = nullptr;
	UMaterialExpression* ExprN = nullptr;

	if (bMask == false)
	{
		if (Index < Comp->GetParamSurfacesCount())
		{
			TextureN = FDatasmithImporterUtils::FindAsset< UTexture >( AssetsContext, Comp->GetParamTexture(Index) );

			if (Comp->GetUseTexture(Index) && TextureN != nullptr)
			{
				ExprN = AddTextureExpression(TextureN, Comp->GetParamTextureSampler(Index), 0, 0, UnrealMaterial, EDatasmithTextureSlot::NOSLOT);
			}
			else
			{
				if (Comp->GetUseComposite(Index))
				{
					ExprN = AddCompExpression(Comp->GetParamSubComposite(Index), UnrealMaterial, EDatasmithTextureSlot::NOSLOT, AssetsContext);
				}
				else if (bRgbNoAlpha)
				{
					ExprN = AddRGBExpression(Comp->GetParamColor(Index), 0, 0, UnrealMaterial, EDatasmithTextureSlot::NOSLOT,true);
				}
				else
				{
					ExprN = AddRGBAExpression(Comp->GetParamColor(Index), 0, 0, UnrealMaterial, EDatasmithTextureSlot::NOSLOT,true);
				}
			}
		}
	}
	else if (Index < Comp->GetParamMaskSurfacesCount())
	{
		TextureN = FDatasmithImporterUtils::FindAsset< UTexture >( AssetsContext, Comp->GetParamMask(Index) );

		if (TextureN != nullptr && !Comp->GetMaskUseComposite(Index))
		{
			ExprN = AddTextureExpression(TextureN, Comp->GetParamMaskTextureSampler(Index), 0, 0, UnrealMaterial, EDatasmithTextureSlot::NOSLOT);
		}
		else if (Comp->GetMaskUseComposite(Index) && Comp->GetParamMaskSubComposite(Index)->IsValid())
		{
			ExprN = AddCompExpression(Comp->GetParamMaskSubComposite(Index), UnrealMaterial, EDatasmithTextureSlot::NOSLOT, AssetsContext);
		}
		else
		{
			ExprN = AddRGBExpression(Comp->GetParamMaskColor(Index), 0, 0, UnrealMaterial, EDatasmithTextureSlot::NOSLOT, true);
		}
	}

	return ExprN;
}

UMaterialExpression* FDatasmithMaterialExpressions::AddRegularExpression(const TSharedPtr<IDatasmithCompositeTexture>& Comp, UObject* UnrealMaterial, EDatasmithTextureSlot Slot, const FDatasmithAssetsImportContext& AssetsContext)
{
	UMaterialExpression* Expr0 = GetCorrectExpressionFromComp(Comp, UnrealMaterial, AssetsContext, 0, false, true);
	ConnectToSlot(Expr0, UnrealMaterial, Slot);
	return Expr0;
}

UMaterialExpression* FDatasmithMaterialExpressions::AddHueTintExpression(UObject* UnrealMaterial, UMaterialExpression* Expression, float Amount, FLinearColor Color)
{
	if (Amount == 0)
	{
		return Expression;
	}

	FString FunctionString = TEXT("/DatasmithContent/Materials/RGBtoHSV.RGBtoHSV");
	UMaterialFunction* RgbToHSVfunc = LoadObject<UMaterialFunction>(nullptr, *FunctionString, nullptr, LOAD_None, nullptr);
	if (RgbToHSVfunc == nullptr)
	{
		return Expression;
	}

	FunctionString = TEXT("/DatasmithContent/Materials/HSVtoRGB.HSVtoRGB");
	UMaterialFunction* HSVtoRGBfunc = LoadObject<UMaterialFunction>(nullptr, *FunctionString, nullptr, LOAD_None, nullptr);
	if (HSVtoRGBfunc == nullptr)
	{
		return Expression;
	}

	UMaterialExpressionMaterialFunctionCall* ExpRGBtoHSV = CreateMaterialExpression<UMaterialExpressionMaterialFunctionCall>(UnrealMaterial);
	ExpRGBtoHSV->MaterialFunction = RgbToHSVfunc;
	ExpRGBtoHSV->UpdateFromFunctionResource();
	Expression->ConnectExpression(ExpRGBtoHSV->GetInput(0), 0);

	FunctionString = TEXT("/Engine/Functions/Engine_MaterialFunctions02/SplitComponents.SplitComponents");
	UMaterialFunction* SplitFunc = LoadObject<UMaterialFunction>(nullptr, *FunctionString, nullptr, LOAD_None, nullptr);
	UMaterialExpressionMaterialFunctionCall* ExpSplit = CreateMaterialExpression<UMaterialExpressionMaterialFunctionCall>(UnrealMaterial);
	ExpSplit->MaterialFunction = SplitFunc;
	ExpSplit->UpdateFromFunctionResource();
	ExpRGBtoHSV->ConnectExpression(ExpSplit->GetInput(0), 0);

	//Using the same algorithm to get hue
	FVector4 K(0.0f, -1.0f / 3.0f, 2.0f / 3.0f, -1.0f);
	FVector4 P;
	if (Color.G < Color.B)
	{
		P = FVector4(Color.B, Color.G, K.W, K.Z);
	}
	else
	{
		P = FVector4(Color.G, Color.B, K.X, K.Y);
	}
	FVector4 Q;
	if (Color.R < P.X)
	{
		Q = FVector4(P.X, P.Y, P.W, Color.R);
	}
	else
	{
		Q = FVector4(Color.R, P.Y, P.Z, P.X);
	}

	float D = Q.X - FMath::Min(Q.W, Q.Y);
	float DesiredHue = FMath::Abs(Q.Z + (Q.W - Q.Y) / (6.0 * D + 0.00000001f));

	UMaterialExpression* DesiredHueExpression = Constant(UnrealMaterial,(DesiredHue));
	// Don't want the Lerp to clean up ExpSplit if it's not used in the Lerp since it's actually used in the hue tint expression below
	UMaterialExpression* NewHue = Lerp(UnrealMaterial, ExpSplit, 0, false, DesiredHueExpression, 0, true, NULL, Amount);

	FunctionString = TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility/MakeFloat3.MakeFloat3");
	UMaterialFunction* Make3Func = LoadObject<UMaterialFunction>(nullptr, *FunctionString, nullptr, LOAD_None, nullptr);
	UMaterialExpressionMaterialFunctionCall* ExpMake3 = CreateMaterialExpression<UMaterialExpressionMaterialFunctionCall>(UnrealMaterial);
	ExpMake3->MaterialFunction = Make3Func;
	ExpMake3->UpdateFromFunctionResource();

	NewHue->ConnectExpression(ExpMake3->GetInput(0), 0);
	ExpSplit->ConnectExpression(ExpMake3->GetInput(1), 2);
	ExpSplit->ConnectExpression(ExpMake3->GetInput(2), 3);

	UMaterialExpressionMaterialFunctionCall* ExpHSVtoRGB = CreateMaterialExpression<UMaterialExpressionMaterialFunctionCall>(UnrealMaterial);
	ExpHSVtoRGB->MaterialFunction = HSVtoRGBfunc;
	ExpHSVtoRGB->UpdateFromFunctionResource();
	ExpMake3->ConnectExpression(ExpHSVtoRGB->GetInput(0), 0);

	return ExpHSVtoRGB;
}

UMaterialExpression* FDatasmithMaterialExpressions::AddCroppedUVMappingExpression(UObject* UnrealMaterial, const FDatasmithTextureSampler& UV, UMaterialExpressionTextureCoordinate* Expression)
{
	// Try to instantiate the material function that enabled cropped UV mapping
	// The idea behind cropped UV mapping is to allow seamless tiling of procedural textures
	// without having to crop them when they contain a fraction of the repeating pattern
	FString FunctionString = TEXT("/DatasmithContent/Materials/CroppedUVMapping.CroppedUVMapping");
	UMaterialFunction* Mappingfunc = LoadObject<UMaterialFunction>(nullptr, *FunctionString, nullptr, LOAD_None, nullptr);
	if (Mappingfunc == nullptr)
	{
		return Expression;
	}

	// Set UV scaling on Texture Coordinates expression to 1
	Expression->UTiling = 1.0f;
	Expression->VTiling = 1.0f;

	UMaterialExpression* OutputExpression = Expression;
	if (UV.OffsetX != 0 || UV.OffsetY != 0)
	{
		UMaterialExpressionConstant2Vector* Constant2d = CreateMaterialExpression<UMaterialExpressionConstant2Vector>(UnrealMaterial);
		Constant2d->R = UV.OffsetX/UV.ScaleX;
		Constant2d->G = UV.OffsetY/UV.ScaleY;

		OutputExpression = Add(UnrealMaterial, OutputExpression, 0.0, Constant2d, 0.0);
	}

	UMaterialExpressionMaterialFunctionCall* ExpMapping = CreateMaterialExpression<UMaterialExpressionMaterialFunctionCall>(UnrealMaterial);
	ExpMapping->MaterialFunction = Mappingfunc;
	ExpMapping->UpdateFromFunctionResource();

	// The UV tiling values are transferred to the mapping function instead
	UMaterialExpressionConstant2Vector* Constant2d = CreateMaterialExpression<UMaterialExpressionConstant2Vector>(UnrealMaterial);
	Constant2d->R = UV.ScaleX;
	Constant2d->G = UV.ScaleY;
	Constant2d->ConnectExpression(ExpMapping->GetInput(0), 0);

	OutputExpression->ConnectExpression(ExpMapping->GetInput(1), 0);

	return ExpMapping;
}

UMaterialExpression* FDatasmithMaterialExpressions::AddCCorrectExpression(const TSharedPtr<IDatasmithCompositeTexture>& Comp, bool bUseGamma, UObject* UnrealMaterial, EDatasmithTextureSlot Slot,
																		 const FDatasmithAssetsImportContext& AssetsContext)
{
	UMaterialExpression* Expr0 = GetCorrectExpressionFromComp(Comp, UnrealMaterial, AssetsContext, 0, false, true);
	if (Expr0 == nullptr)
	{
		return nullptr;
	}

	UMaterialExpression* HsvExp = nullptr;
	UMaterialExpression* ToBeConnected = nullptr;

	if (Comp->GetParamVal1(0).Key != 0 || Comp->GetParamVal1(1).Key != 0 || Comp->GetParamVal1(2).Key != 0)
	{
		TArray<double> Hsv;

		for (int32 i = 0; i < Comp->GetParamVal1Count(); ++i)
		{
			Hsv.Add(Comp->GetParamVal1(i).Key);
		}

		ToBeConnected = HsvExpression(UnrealMaterial, Expr0, Hsv);
	}
	else
	{
		ToBeConnected = Expr0;
	}

	if (Comp->GetParamVal1Count() > 4)
	{
		ToBeConnected = AddHueTintExpression(UnrealMaterial, ToBeConnected, Comp->GetParamVal1(4).Key, Comp->GetParamColor(1));
	}

	if (bUseGamma)
	{
		double Gamma = Comp->GetParamVal1(3).Key;
		if (Gamma != 1.0)
		{
			if (FMath::Abs(Gamma) < 0.01)
			{
				Gamma = 0.01;
			}
			ToBeConnected = Power(UnrealMaterial, ToBeConnected, nullptr, (1.0 / Gamma));
		}
	}
	else
	{
		// contrast
		if (Comp->GetParamVal1(3).Key != 0.0)
		{
			UMaterialExpression* SubExpression = Add(UnrealMaterial, ToBeConnected, 0.0, nullptr, -0.5);
			ToBeConnected = Multiply(UnrealMaterial, SubExpression, 0.0, nullptr, FMath::Max(0.0, Comp->GetParamVal1(3).Key + 1.0));
			ToBeConnected = Add(UnrealMaterial, ToBeConnected, 0.0, nullptr, 0.5);
		}
	}

	ConnectToSlot(ToBeConnected, UnrealMaterial, Slot);
	return ToBeConnected;
}

UMaterialExpression* FDatasmithMaterialExpressions::AddMultiplyExpression(const TSharedPtr<IDatasmithCompositeTexture>& Comp, UObject* UnrealMaterial, EDatasmithTextureSlot Slot, const FDatasmithAssetsImportContext& AssetsContext)
{
	UMaterialExpression* Expr0 = GetCorrectExpressionFromComp(Comp, UnrealMaterial, AssetsContext, 0, false, true);
	UMaterialExpression* Expr1 = GetCorrectExpressionFromComp(Comp, UnrealMaterial, AssetsContext, 1, false, true);
	UMaterialExpression* ExprMask = GetCorrectExpressionFromComp(Comp, UnrealMaterial, AssetsContext, 0, true, true);

	double Mul0 = 0.0;
	double Mul1 = 0.0;
	if (Comp->GetParamVal1Count() > 0)
	{
		Mul0 = Comp->GetParamVal1(0).Key;
	}

	if (Comp->GetParamVal2Count() > 0)
	{
		Mul1 = Comp->GetParamVal2(0).Key;
	}

	// if Exp is nullptr value is used if not value is ignored
	UMaterialExpression* ToBeConnected = Multiply(UnrealMaterial, Expr0, Mul0, Expr1, Mul1);

	if (ExprMask != nullptr)
	{
		ToBeConnected = Lerp(UnrealMaterial, Expr0, 0.0, true, ToBeConnected, 0.0, true, ExprMask, 0);
	}

	ConnectToSlot(ToBeConnected, UnrealMaterial, Slot);
	return ToBeConnected;
}

UMaterialExpression* FDatasmithMaterialExpressions::AddMixExpression(const TSharedPtr<IDatasmithCompositeTexture>& Comp, UObject* UnrealMaterial, EDatasmithTextureSlot Slot, const FDatasmithAssetsImportContext& AssetsContext)
{
	UMaterialExpression* Expr0 = GetCorrectExpressionFromComp(Comp, UnrealMaterial, AssetsContext, 0, false, true);
	UMaterialExpression* Expr1 = GetCorrectExpressionFromComp(Comp, UnrealMaterial, AssetsContext, 1, false, true);
	UMaterialExpression* ExprMask = GetCorrectExpressionFromComp(Comp, UnrealMaterial, AssetsContext, 0, true, true);

	if (ExprMask == nullptr)
	{
		if (Comp->GetParamVal1Count() > 0)
		{
			ExprMask = AddConstExpression(Comp->GetParamVal1(0).Key, 0, 0, UnrealMaterial, EDatasmithTextureSlot::NOSLOT);
		}
		else
		{
			ExprMask = AddConstExpression(0.72, 0, 0, UnrealMaterial, EDatasmithTextureSlot::NOSLOT);
		}
	}

	UMaterialExpression* LerpExpression = Lerp(UnrealMaterial, Expr0, 0.0, true, Expr1, 0.0, true, ExprMask, 0);
	ConnectToSlot(LerpExpression, UnrealMaterial, Slot);
	return LerpExpression;
}

UMaterialExpression* FDatasmithMaterialExpressions::AddCompositeExpression(const TSharedPtr<IDatasmithCompositeTexture>& Comp, UObject* UnrealMaterial, EDatasmithTextureSlot Slot, const FDatasmithAssetsImportContext& AssetsContext)
{
	UMaterialExpression* BaseExpr = GetCorrectExpressionFromComp(Comp, UnrealMaterial, AssetsContext, 0, false, true);
	UMaterialExpression* Expr1 = nullptr;
	UMaterialExpression* ExprMask = nullptr;
	UMaterialExpression* ExprAux = nullptr;
	FString BlendString;

	int32 MaskIndex = 0;
	for (int32 SurfaceIndex = 1; SurfaceIndex < Comp->GetParamSurfacesCount(); SurfaceIndex++)
	{
		Expr1 = GetCorrectExpressionFromComp(Comp, UnrealMaterial, AssetsContext, SurfaceIndex, false, true);
		ExprMask = nullptr;

		if (Comp->GetParamVal1(SurfaceIndex).Key < 0)
		{
			ExprMask = GetCorrectExpressionFromComp(Comp, UnrealMaterial, AssetsContext, MaskIndex, true, true);
		}

		if (ExprMask == nullptr)
		{
			ExprMask = AddConstExpression(Comp->GetParamVal1(SurfaceIndex).Key, 0.0, 0.0, UnrealMaterial, EDatasmithTextureSlot::NOSLOT);
		}
		else
		{
			MaskIndex++;
		}

		if ( Comp->GetParamVal2Count() <= SurfaceIndex )
		{
			// We don't have a blend mode so just use Expr1 directly
			ExprAux = Expr1;
		}
		else
		{
			EDatasmithCompositeCompMode CompositeCompMode = (EDatasmithCompositeCompMode)(int32)Comp->GetParamVal2(SurfaceIndex).Key;
			if (CompositeCompMode <= EDatasmithCompositeCompMode::Mult || CompositeCompMode >= EDatasmithCompositeCompMode::Hue) // Hue, Saturation, Value and Color or not supported through this path
			{
				switch (CompositeCompMode)
				{
				case EDatasmithCompositeCompMode::Average:
					ExprAux = Lerp(UnrealMaterial, BaseExpr, 0.0, true, Expr1, 0.0, true, nullptr, 0.5);
					break;
				case EDatasmithCompositeCompMode::Add:
					ExprAux = Add(UnrealMaterial, BaseExpr, 0.0, Expr1, 0.0);
					break;
				case EDatasmithCompositeCompMode::Sub:
					ExprAux = Subtract(UnrealMaterial, BaseExpr, 0.0, Expr1, 0.0);
					break;
				case EDatasmithCompositeCompMode::Mult:
					ExprAux = Multiply(UnrealMaterial, BaseExpr, 0.0, Expr1, 0.0);
					break;
				default:
					ExprAux = Expr1;
					break;
				}
			}
			else
			{
				switch (CompositeCompMode)
				{
				case EDatasmithCompositeCompMode::Burn:
					BlendString = TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends/Blend_ColorBurn.Blend_ColorBurn");
					break;
				case EDatasmithCompositeCompMode::Dodge:
					BlendString = TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends/Blend_ColorDodge.Blend_ColorDodge");
					break;
				case EDatasmithCompositeCompMode::Darken:
					BlendString = TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends/Blend_Darken.Blend_Darken");
					break;
				case EDatasmithCompositeCompMode::Difference:
					BlendString = TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends/Blend_Difference.Blend_Difference");
					break;
				case EDatasmithCompositeCompMode::Exclusion:
					BlendString = TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends/Blend_Exclusion.Blend_Exclusion");
					break;
				case EDatasmithCompositeCompMode::HardLight:
					BlendString = TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends/Blend_HardLight.Blend_HardLight");
					break;
				case EDatasmithCompositeCompMode::Lighten:
					BlendString = TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends/Blend_Lighten.Blend_Lighten");
					break;
				case EDatasmithCompositeCompMode::Screen:
					BlendString = TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends/Blend_Screen.Blend_Screen");
					break;
				case EDatasmithCompositeCompMode::LinearBurn:
					BlendString = TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends/Blend_LinearBurn.Blend_LinearBurn");
					break;
				case EDatasmithCompositeCompMode::LinearLight:
					BlendString = TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends/Blend_LinearLight.Blend_LinearLight");
					break;
				case EDatasmithCompositeCompMode::LinearDodge:
					BlendString = TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends/Blend_LinearDodge.Blend_LinearDodge");
					break;
				case EDatasmithCompositeCompMode::Overlay:
					BlendString = TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends/Blend_Overlay.Blend_Overlay");
					break;
				case EDatasmithCompositeCompMode::PinLight:
					BlendString = TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends/Blend_PinLight.Blend_PinLight");
					break;
				case EDatasmithCompositeCompMode::SoftLight:
					BlendString = TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends/Blend_SoftLight.Blend_SoftLight");
					break;
				}
				UMaterialFunction* Blendfunc = LoadObject<UMaterialFunction>(nullptr, *BlendString, nullptr, LOAD_None, nullptr);
				UMaterialExpressionMaterialFunctionCall* ExpBlend = CreateMaterialExpression<UMaterialExpressionMaterialFunctionCall>(UnrealMaterial);
				ExpBlend->MaterialFunction = Blendfunc;
				ExpBlend->UpdateFromFunctionResource();

				if ( BaseExpr )
				{
					BaseExpr->ConnectExpression(ExpBlend->GetInput(0), 0);
				}

				if ( Expr1 )
				{
					Expr1->ConnectExpression(ExpBlend->GetInput(1), 0);
				}

				ExprAux = ExpBlend;
			}
		}

		if (ExprMask != nullptr)
		{
			ExprAux = Lerp(UnrealMaterial, BaseExpr, 0.0, true, ExprAux, 0.0, true, ExprMask, 0);
		}
		else if (Comp->GetParamVal1(SurfaceIndex).Key < 1.0)
		{
			ExprAux = Lerp(UnrealMaterial, BaseExpr, 0.0, true, ExprAux, 0.0, true, nullptr, Comp->GetParamVal1(SurfaceIndex).Key);
		}

		BaseExpr = ExprAux;
	}

	ConnectToSlot(BaseExpr, UnrealMaterial, Slot);
	return BaseExpr;
}

UMaterialExpression* FDatasmithMaterialExpressions::AddIorExpression(const TSharedPtr<IDatasmithCompositeTexture>& Comp, UObject* UnrealMaterial, EDatasmithTextureSlot Slot, const FDatasmithAssetsImportContext& AssetsContext)
{
	UMaterialExpression* Expr0 = GetCorrectExpressionFromComp(Comp, UnrealMaterial, AssetsContext, 0, false, true);
	UMaterialExpression* Expr1 = GetCorrectExpressionFromComp(Comp, UnrealMaterial, AssetsContext, 1, false, true);

	UMaterialExpression* IorExp = CalcIOR(Comp->GetParamVal1(0).Key, Comp->GetParamVal2(0).Key, UnrealMaterial, Expr1, true, Expr0, true);

	ConnectToSlot(IorExp, UnrealMaterial, Slot);
	return IorExp;
}

UMaterialExpression* FDatasmithMaterialExpressions::AddFresnelExpression(const TSharedPtr<IDatasmithCompositeTexture>& Comp, UObject* UnrealMaterial, EDatasmithTextureSlot Slot, const FDatasmithAssetsImportContext& AssetsContext)
{
	UMaterialExpression* Expr0 = GetCorrectExpressionFromComp(Comp, UnrealMaterial, AssetsContext, 0, false, true);
	UMaterialExpression* Expr1 = GetCorrectExpressionFromComp(Comp, UnrealMaterial, AssetsContext, 1, false, true);

	float Exp = 2.0f;

	if (Comp->GetParamVal1Count() > 0)
	{
		Exp = Comp->GetParamVal1(0).Key;
	}

	float Base = 0;
	if (Comp->GetParamVal2Count() > 0)
	{
		Base = Comp->GetParamVal2(0).Key;
	}

	UMaterialExpression* FresnelExpression = Fresnel(UnrealMaterial, nullptr, Exp, nullptr, Base);
	UMaterialExpression* LerpExpression = Lerp(UnrealMaterial, Expr0, 0.0, true, Expr1, 0.0, true, FresnelExpression, 0);

	ConnectToSlot(LerpExpression, UnrealMaterial, Slot);
	return LerpExpression;
}

UMaterialExpression* FDatasmithMaterialExpressions::AddCompExpression(const TSharedPtr<IDatasmithCompositeTexture>& Comp, UObject* UnrealMaterial, EDatasmithTextureSlot Slot, const FDatasmithAssetsImportContext& AssetsContext)
{
	if (Comp->IsValid() == false)
	{
		return nullptr;
	}

	switch (Comp->GetMode())
	{
	case EDatasmithCompMode::Regular:
		return AddRegularExpression(Comp, UnrealMaterial, Slot, AssetsContext);
	case EDatasmithCompMode::Mix:
		return AddMixExpression(Comp, UnrealMaterial, Slot, AssetsContext);
	case EDatasmithCompMode::Fresnel:
		return AddFresnelExpression(Comp, UnrealMaterial, Slot, AssetsContext);
	case EDatasmithCompMode::Ior:
		return AddIorExpression(Comp, UnrealMaterial, Slot, AssetsContext);
	case EDatasmithCompMode::ColorCorrectGamma:
		return AddCCorrectExpression(Comp, true, UnrealMaterial, Slot, AssetsContext);
	case EDatasmithCompMode::ColorCorrectContrast:
		return AddCCorrectExpression(Comp, false, UnrealMaterial, Slot, AssetsContext);
	case EDatasmithCompMode::Multiply:
		return AddMultiplyExpression(Comp, UnrealMaterial, Slot, AssetsContext);
	case EDatasmithCompMode::Composite:
		return AddCompositeExpression(Comp, UnrealMaterial, Slot, AssetsContext);
	}

	return nullptr;
}

void FDatasmithMaterialExpressions::ForEachParamsNameInMaterial(const TSharedPtr<IDatasmithUEPbrMaterialElement>& MaterialElement, const TFunctionRef<void (FName Expression, const EDatasmithMaterialExpressionType& ExpressionType, int32 Index)>& CallbackForEach)
{
	if (MaterialElement)
	{
		for ( int32 ExpressionIndex = 0; ExpressionIndex < MaterialElement->GetExpressionsCount(); ++ExpressionIndex )
		{
			IDatasmithMaterialExpression* MaterialExpression = MaterialElement->GetExpression( ExpressionIndex );
			if ( !MaterialExpression )
			{
				check( false );
			}

			// Should be keep in sync with the Create...Expression functions
			if ( MaterialExpression->IsSubType( EDatasmithMaterialExpressionType::Texture ) )
			{
				FName ParamName = GenerateParamName<EDatasmithMaterialExpressionType::Texture>( *MaterialExpression, ExpressionIndex + 1 );
				CallbackForEach( MoveTemp( ParamName ), EDatasmithMaterialExpressionType::Texture, ExpressionIndex );
			}
			else if ( MaterialExpression->IsSubType( EDatasmithMaterialExpressionType::ConstantBool ) )
			{
				FName ParamName = GenerateParamName<EDatasmithMaterialExpressionType::ConstantBool>( *MaterialExpression, ExpressionIndex + 1 );
				CallbackForEach( MoveTemp( ParamName ), EDatasmithMaterialExpressionType::ConstantBool, ExpressionIndex );
			}
			else if ( MaterialExpression->IsSubType( EDatasmithMaterialExpressionType::ConstantColor ) )
			{
				if ( ShouldExpressionBeAParameter( static_cast<IDatasmithMaterialExpressionColor&>( *MaterialExpression ) ) )
				{
					FName ParamName = GenerateParamName<EDatasmithMaterialExpressionType::ConstantColor>( *MaterialExpression, ExpressionIndex + 1 );
					CallbackForEach( ParamName, EDatasmithMaterialExpressionType::ConstantColor, ExpressionIndex );
				}
			}
			else if ( MaterialExpression->IsSubType( EDatasmithMaterialExpressionType::ConstantScalar ) )
			{
				if ( ShouldExpressionBeAParameter( static_cast<IDatasmithMaterialExpressionScalar&>( *MaterialExpression ) ) )
				{
					FName ParamName = GenerateParamName<EDatasmithMaterialExpressionType::ConstantScalar>( *MaterialExpression, ExpressionIndex + 1 );
					CallbackForEach( ParamName, EDatasmithMaterialExpressionType::ConstantScalar, ExpressionIndex );
				}
			}
			else if ( MaterialExpression->IsSubType( EDatasmithMaterialExpressionType::Generic ) )
			{
				if ( ShouldExpressionBeAParameter( static_cast<IDatasmithMaterialExpressionGeneric&>( *MaterialExpression ) ) )
				{
					FName ParamName = GenerateParamName<EDatasmithMaterialExpressionType::Generic>( *MaterialExpression, ExpressionIndex + 1 );
					CallbackForEach( ParamName, EDatasmithMaterialExpressionType::Generic, ExpressionIndex );
				}
			}
			else if ( MaterialExpression->IsSubType( EDatasmithMaterialExpressionType::FunctionCall ) )
			{
				// Do noting
			}
			else if ( MaterialExpression->IsSubType( EDatasmithMaterialExpressionType::TextureCoordinate ) )
			{
				// Do noting
			}
			else if ( MaterialExpression->IsSubType( EDatasmithMaterialExpressionType::FlattenNormal ) )
			{
				// Do noting
			}
			else
			{
				ensure( false );
			}
		}
	}
}

// glass node includes custom reflection
void FDatasmithMaterialExpressions::AddGlassNode(UMaterialExpression* RefleExpression, UMaterialExpression* OpacityExpression, double IOR, double IORRefra, UObject* UnrealMaterial)
{
	UMaterial* ActualMaterial = nullptr;
	UMaterialFunction* FuncMaterial = nullptr;
	UMaterialExpressionMakeMaterialAttributes* Attrib = nullptr;

	if (UnrealMaterial->IsA< UMaterial >())
	{
		ActualMaterial = (UMaterial*)UnrealMaterial;
	}

	if (UnrealMaterial->IsA< UMaterialFunction >())
	{
		FuncMaterial = (UMaterialFunction*)UnrealMaterial;
		if (FuncMaterial)
		{
			Attrib = FindOrAddAttributesFromMatFunc(FuncMaterial);
		}
	}

	UMaterialExpression* Multiply05 = Multiply(UnrealMaterial, nullptr, 0.5, OpacityExpression, 0);
	UMaterialExpression* Multiply09 = Multiply(UnrealMaterial, nullptr, 0.9, OpacityExpression, 0);
	UMaterialExpression* MixedByIOR = CalcIOR(IOR, 0, UnrealMaterial, Multiply05, true, Multiply09, false); // prevent Multiply09 from being cleaned up by the Lerp inside CalcIOR

	UMaterialExpression* Power5 = Power(UnrealMaterial, Multiply09, nullptr, 5.0);
	UMaterialExpression* Multiply06 = Multiply(UnrealMaterial, nullptr, 0.6, Power5, 0);
	UMaterialExpression* Desaturated = Desaturate(UnrealMaterial, MixedByIOR);

	// usually fully white so specular gets invalidated
	if (ActualMaterial)
	{
		UMaterialEditorOnlyData* ActualMaterialEditorOnly = ActualMaterial->GetEditorOnlyData();
		ActualMaterialEditorOnly->BaseColor.Expression = Multiply06;
		ActualMaterialEditorOnly->Specular.Expression = RefleExpression;
		ActualMaterialEditorOnly->Metallic.Expression = RefleExpression;
		ActualMaterialEditorOnly->Opacity.Expression = OneMinus(UnrealMaterial, Desaturated);
		ActualMaterialEditorOnly->Refraction.Expression = RefractionIOR(IORRefra, UnrealMaterial);
		ActualMaterial->BlendMode = EBlendMode::BLEND_Translucent;
		ActualMaterial->TranslucencyLightingMode = ETranslucencyLightingMode::TLM_Surface;
		ActualMaterial->TwoSided = true;
	}

	if (Attrib)
	{
		Attrib->BaseColor.Expression = Multiply06;
		Attrib->Specular.Expression = RefleExpression;
		Attrib->Metallic.Expression = RefleExpression;
		Attrib->Opacity.Expression = OneMinus(UnrealMaterial, Desaturated);
		Attrib->Refraction.Expression = RefractionIOR(IORRefra, UnrealMaterial);
		/*Attrib->BlendMode = EBlendMode::BLEND_Translucent;
		Attrib->TranslucencyLightingMode = ETranslucencyLightingMode::TLM_Surface;
		Attrib->TwoSided = true;*/
	}
}

void FDatasmithMaterialExpressions::CreateDatasmithMaterialCoat(const FDatasmithAssetsImportContext& AssetsContext, const TSharedPtr< IDatasmithShaderElement >& ShaderElement,
															   UObject* UnrealMaterial)
{
	UTexture* SpecularTexture = FDatasmithImporterUtils::FindAsset< UTexture >( AssetsContext, ShaderElement->GetReflectanceTexture() );
	UTexture* RoughnessTexture = FDatasmithImporterUtils::FindAsset< UTexture >( AssetsContext, ShaderElement->GetRoughnessTexture() );
	UTexture* WeightTexture = FDatasmithImporterUtils::FindAsset< UTexture >( AssetsContext, ShaderElement->GetWeightTexture() );

	// roughness
	if (AddCompExpression(ShaderElement->GetRoughnessComp(), UnrealMaterial, EDatasmithTextureSlot::COATROUGHNESS, AssetsContext) == nullptr)
	{
		if (RoughnessTexture != nullptr)
		{
			AddTextureExpression(RoughnessTexture, ShaderElement->GetRoughTextureSampler(), 0, 0, UnrealMaterial, EDatasmithTextureSlot::COATROUGHNESS);
		}
		else
		{
			AddConstExpression(ShaderElement->GetRoughness(), 0, 0, UnrealMaterial, EDatasmithTextureSlot::COATROUGHNESS);
		}
	}

	// specular
	UMaterialExpression* RefleExp = AddCompExpression(ShaderElement->GetRefleComp(), UnrealMaterial, EDatasmithTextureSlot::COATSPECULAR, AssetsContext);
	if (RefleExp == nullptr)
	{
		if (SpecularTexture != nullptr)
		{
			RefleExp = AddTextureExpression(SpecularTexture, ShaderElement->GetRefleTextureSampler(), ShaderElement->GetIOR(), ShaderElement->GetIORk(), UnrealMaterial, EDatasmithTextureSlot::COATSPECULAR);
		}
		else
		{
			RefleExp = AddRGBAExpression(ShaderElement->GetReflectanceColor(), ShaderElement->GetIOR(), ShaderElement->GetIORk(), UnrealMaterial, EDatasmithTextureSlot::COATSPECULAR);
		}
	}

	UMaterialExpression* WeightExp = AddCompExpression(ShaderElement->GetWeightComp(), UnrealMaterial, EDatasmithTextureSlot::NOSLOT, AssetsContext);
	if (WeightExp == nullptr)
	{
		if (WeightTexture != nullptr)
		{
			WeightExp = AddTextureExpression(WeightTexture, ShaderElement->GetWeightTextureSampler(), 0, 0, UnrealMaterial, EDatasmithTextureSlot::NOSLOT);
		}
		else if (!FMath::IsNearlyEqual( ShaderElement->GetWeightValue(), 1.0 ))
		{
			WeightExp = Constant(UnrealMaterial, ShaderElement->GetWeightValue());
		}
	}

	if (WeightExp)
	{
		UMaterialExpression* ToConnect = Multiply(UnrealMaterial, RefleExp, 0.0, WeightExp, 0.0);
		ConnectToSlot(ToConnect, UnrealMaterial, EDatasmithTextureSlot::COATSPECULAR);
	}
}

void FDatasmithMaterialExpressions::ModulateNormalAmount(UObject* UnrealMatOrFunc, double Amount)
{
	UMaterial* UnrealMaterial = Cast< UMaterial >( UnrealMatOrFunc );

	UMaterialFunction* Func = Cast< UMaterialFunction >( UnrealMatOrFunc );

	UMaterialExpressionMakeMaterialAttributes* Attrib = nullptr;
	if (Func)
	{
		Attrib = FindOrAddAttributesFromMatFunc(Func);
	}

	UMaterialEditorOnlyData* UnrealMaterialEditorOnly = nullptr;
	UMaterialExpression* ToBeConnected = nullptr;
	if (UnrealMaterial)
	{
		UnrealMaterialEditorOnly = UnrealMaterial->GetEditorOnlyData();
		ToBeConnected = UnrealMaterialEditorOnly->Normal.Expression;
	}

	if (Attrib)
	{
		ToBeConnected = Attrib->Normal.Expression;
	}

	if (ToBeConnected == nullptr)
	{
		return;
	}

	UMaterialExpression* InvAmountExpr = Constant(UnrealMatOrFunc, 1.0 - Amount);

	UMaterialFunction* Flatten = LoadObject<UMaterialFunction>(nullptr, TEXT("/Engine/Functions/Engine_MaterialFunctions01/Texturing/FlattenNormal.FlattenNormal"), nullptr, LOAD_None, nullptr);

	UMaterialExpressionMaterialFunctionCall* ExpFuncCall = CreateMaterialExpression<UMaterialExpressionMaterialFunctionCall>(UnrealMatOrFunc);
	ExpFuncCall->MaterialFunction = Flatten;
	ExpFuncCall->UpdateFromFunctionResource();

	if (UnrealMaterialEditorOnly)
	{
		UnrealMaterialEditorOnly->Normal.Expression = ExpFuncCall;
	}

	if (Attrib)
	{
		Attrib->Normal.Expression = ExpFuncCall;
	}

	ToBeConnected->ConnectExpression(ExpFuncCall->GetInput(0), 0);
	InvAmountExpr->ConnectExpression(ExpFuncCall->GetInput(1), 0);
}

void FDatasmithMaterialExpressions::CreateParallaxOffset(UObject* UnrealMatOrFunc, UMaterialExpressionTextureObject* TextureObject, FDatasmithTextureSampler UV, float Amount)
{
	UMaterialFunction* Func = Cast< UMaterialFunction >( UnrealMatOrFunc );

	UMaterialExpressionMakeMaterialAttributes* Attrib = nullptr;
	if (Func)
	{
		Attrib = FindOrAddAttributesFromMatFunc(Func);
	}

	UMaterialFunction* ParallaxFunc =
		LoadObject<UMaterialFunction>(nullptr, TEXT("/Engine/Functions/Engine_MaterialFunctions01/Texturing/ParallaxOcclusionMapping.ParallaxOcclusionMapping"), nullptr, LOAD_None, nullptr);

	UMaterialExpression* MinSamples = Constant(UnrealMatOrFunc, 8);
	UMaterialExpression* MaxSamples = Constant(UnrealMatOrFunc, 32);
	UMaterialExpression* Height = Constant(UnrealMatOrFunc, 0.05 * Amount);
	UMaterialExpression* Weight = AddRGBAExpression(FLinearColor(1.0, 0.0, 0.0, 0.0), 0, 0, UnrealMatOrFunc, EDatasmithTextureSlot::NOSLOT);

	UMaterialExpressionMaterialFunctionCall* ExpFuncCall = CreateMaterialExpression<UMaterialExpressionMaterialFunctionCall>(UnrealMatOrFunc);
	ExpFuncCall->MaterialFunction = ParallaxFunc;
	ExpFuncCall->UpdateFromFunctionResource();

	TextureObject->ConnectExpression(ExpFuncCall->GetInput(0), 0);
	Height->ConnectExpression(ExpFuncCall->GetInput(1), 0);
	MinSamples->ConnectExpression(ExpFuncCall->GetInput(2), 0);
	MaxSamples->ConnectExpression(ExpFuncCall->GetInput(3), 0);
	Weight->ConnectExpression(ExpFuncCall->GetInput(5), 0);

	if (UV.OffsetX != 0 || UV.OffsetY != 0 || UV.ScaleX != 1 || UV.ScaleY != 1 || UV.Rotation != 0)
	{
		UMaterialExpressionTextureCoordinate* TexcoordExpression = CreateMaterialExpression<UMaterialExpressionTextureCoordinate>(UnrealMatOrFunc);
		UMaterialExpression* ToBeConnected = TexcoordExpression;
		TexcoordExpression->CoordinateIndex = UV.CoordinateIndex;

		if (FMath::Abs(UV.ScaleX) < SMALL_NUMBER)
		{
			UV.ScaleX = SMALL_NUMBER;
		}

		if (FMath::Abs(UV.ScaleY) < SMALL_NUMBER)
		{
			UV.ScaleY = SMALL_NUMBER;
		}

		TexcoordExpression->UTiling = UV.ScaleX;
		TexcoordExpression->VTiling = UV.ScaleY;

		if (UV.OffsetX != 0 || UV.OffsetY != 0)
		{
			UMaterialExpressionConstant2Vector* Constant2d = CreateMaterialExpression<UMaterialExpressionConstant2Vector>(UnrealMatOrFunc);
			Constant2d->R = UV.OffsetX;
			Constant2d->G = UV.OffsetY;

			ToBeConnected = Add(UnrealMatOrFunc, ToBeConnected, 0.0, Constant2d, 0.0);
		}

		if (UV.Rotation != 0)
		{
			UMaterialExpressionConstant* Constant1d = CreateMaterialExpression<UMaterialExpressionConstant>(UnrealMatOrFunc);
			Constant1d->R = UV.Rotation * 2.0 * PI;

			UClass* MaterialExpressionRotatorClass = FindObject<UClass>(nullptr, TEXT("/Script/Engine.MaterialExpressionRotator"));

			// This will create a rotator, but the pointer you get back is to Base class.
			UMaterialExpression* RotatorExp = CreateMaterialExpression(UnrealMatOrFunc, MaterialExpressionRotatorClass);

			// I think you can do this, but it's possible Cast requires the class to be exported, not sure.
			UMaterialExpressionRotator* Rotator = (UMaterialExpressionRotator*)(RotatorExp);

			Rotator->Speed = 1.0;
			Rotator->GetInput(0)->Expression = ToBeConnected;
			Rotator->GetInput(1)->Expression = Constant1d;

			ToBeConnected = Rotator;
		}

		ToBeConnected->ConnectExpression(ExpFuncCall->GetInput(4), 0);
	}

	TArray<UMaterialExpressionTextureSample*> TextureSamplers;
	GetTextureSamplers(UnrealMatOrFunc, TextureSamplers);

	UMaterialExpressionTextureCoordinate* NeutralTexcoordExpression = NULL;
	for (int32 i = 0; i < TextureSamplers.Num(); i++)
	{
		FExpressionInput* TextureInput = TextureSamplers[i]->GetInput(0);
		UMaterialExpression* TextureInputExp = TextureInput->Expression;
		if (TextureInputExp == nullptr)
		{
			if (NeutralTexcoordExpression == NULL)
			{
				NeutralTexcoordExpression = CreateMaterialExpression<UMaterialExpressionTextureCoordinate>(UnrealMatOrFunc);
			}

			UMaterialExpression* ToBeConnected = Add(UnrealMatOrFunc, NeutralTexcoordExpression, 0.0, NeutralTexcoordExpression, 0.0);
			ExpFuncCall->ConnectExpression(ToBeConnected->GetInput(1), 1);
			TextureInput->Expression = ToBeConnected;
		}
		else
		{
			UMaterialExpression* ToBeConnected = Add(UnrealMatOrFunc, TextureInputExp, 0.0, TextureInputExp, 0.0);
			ExpFuncCall->ConnectExpression(ToBeConnected->GetInput(1), 1);
			TextureInput->Expression = ToBeConnected;
		}
	}
}

void FDatasmithMaterialExpressions::CreateDatasmithMaterialHelper(UPackage* Package, const TSharedPtr< IDatasmithShaderElement >& ShaderElement, const FDatasmithAssetsImportContext& AssetsContext,
																UObject* UnrealMaterial)
{
	Package->SetDirtyFlag(true);

	// make Texture sampler
	UTexture* DiffuseTexture = FDatasmithImporterUtils::FindAsset< UTexture >( AssetsContext, ShaderElement->GetDiffuseTexture() );
	UTexture* SpecularTexture = FDatasmithImporterUtils::FindAsset< UTexture >( AssetsContext, ShaderElement->GetReflectanceTexture() );
	UTexture* RoughnessTexture = FDatasmithImporterUtils::FindAsset< UTexture >( AssetsContext, ShaderElement->GetRoughnessTexture() );
	UTexture* NormalTexture = FDatasmithImporterUtils::FindAsset< UTexture >( AssetsContext, ShaderElement->GetNormalTexture() );
	UTexture* AdditionalBumpTexture = FDatasmithImporterUtils::FindAsset< UTexture >( AssetsContext, ShaderElement->GetBumpTexture() );
	UTexture* EmissiveTexture = FDatasmithImporterUtils::FindAsset< UTexture >( AssetsContext, ShaderElement->GetEmitTexture() );
	UTexture* RefraTexture = FDatasmithImporterUtils::FindAsset< UTexture >( AssetsContext, ShaderElement->GetTransparencyTexture() );
	UTexture* MaskTexture = FDatasmithImporterUtils::FindAsset< UTexture >( AssetsContext, ShaderElement->GetMaskTexture() );
	UTexture* MetalTexture = FDatasmithImporterUtils::FindAsset< UTexture >( AssetsContext, ShaderElement->GetMetalTexture() );

	// roughness
	if (AddCompExpression(ShaderElement->GetRoughnessComp(), UnrealMaterial, EDatasmithTextureSlot::ROUGHNESS, AssetsContext) == nullptr)
	{
		if (RoughnessTexture != nullptr)
		{
			AddTextureExpression(RoughnessTexture, ShaderElement->GetRoughTextureSampler(), 0, 0, UnrealMaterial, EDatasmithTextureSlot::ROUGHNESS);
		}
		else
		{
			AddConstExpression(ShaderElement->GetRoughness(), 0, 0, UnrealMaterial, EDatasmithTextureSlot::ROUGHNESS);
		}
	}

	// mask/clip
	if (AddCompExpression(ShaderElement->GetMaskComp(), UnrealMaterial, EDatasmithTextureSlot::OPACITYMASK, AssetsContext) == nullptr)
	{
		if (MaskTexture != nullptr)
		{
			AddTextureExpression(MaskTexture, ShaderElement->GetMaskTextureSampler(), 0, 0, UnrealMaterial, EDatasmithTextureSlot::OPACITYMASK);
		}
	}

	// emittance
	UMaterialExpression* EmissiveExpression = AddCompExpression(ShaderElement->GetEmitComp(), UnrealMaterial, EDatasmithTextureSlot::NOSLOT, AssetsContext);
	if (EmissiveExpression == nullptr)
	{
		if (EmissiveTexture != nullptr)
		{
			EmissiveExpression = AddTextureExpression(EmissiveTexture, ShaderElement->GetEmitTextureSampler(), 0, 0, UnrealMaterial, EDatasmithTextureSlot::NOSLOT);
		}
		else
		{
			if (ShaderElement->GetEmitColor().IsAlmostBlack() == false)
			{
				EmissiveExpression = AddRGBExpression(ShaderElement->GetEmitColor(), 0, 0, UnrealMaterial, EDatasmithTextureSlot::NOSLOT);
			}
			else if (ShaderElement->GetEmitTemperature() > 0)
			{
				EmissiveExpression = AddRGBExpression(DatasmithMaterialsUtils::TemperatureToColor(ShaderElement->GetEmitTemperature()), 0, 0, UnrealMaterial, EDatasmithTextureSlot::NOSLOT);
			}
		}
	}

	UMaterial* ActualMaterial = Cast< UMaterial >( UnrealMaterial );

	if (EmissiveExpression != nullptr)
	{
		if (ShaderElement->GetLightOnly() && ActualMaterial)
		{
			ActualMaterial->SetShadingModel(EMaterialShadingModel::MSM_Unlit);
		}
		if (ShaderElement->GetEmitPower() != 1)
		{
			EmissiveExpression = Multiply(UnrealMaterial, EmissiveExpression, 0, nullptr, ShaderElement->GetEmitPower());
		}

		ConnectToSlot(EmissiveExpression, UnrealMaterial, EDatasmithTextureSlot::EMISSIVECOLOR);
	}

	if (ActualMaterial)
	{
		ActualMaterial->TwoSided = ShaderElement->GetTwoSided();
		ActualMaterial->bUseEmissiveForDynamicAreaLighting = ShaderElement->GetUseEmissiveForDynamicAreaLighting();
		if (ShaderElement->GetShaderUsage() == EDatasmithShaderUsage::LightFunction)
		{
			ActualMaterial->MaterialDomain = EMaterialDomain::MD_LightFunction;
		}
	}

	EDatasmithMaterialMode MaterialMode = EDatasmithMaterialMode::Regular;
	if (ShaderElement->GetIORRefra() <= 4.5 &&
		(RefraTexture != nullptr || ShaderElement->GetTransparencyColor().IsAlmostBlack() == false || ShaderElement->GetTransComp()->IsValid()) &&
		(SpecularTexture != nullptr || ShaderElement->GetReflectanceColor().IsAlmostBlack() == false || ShaderElement->GetRefleComp()->IsValid()))
	{
		MaterialMode = EDatasmithMaterialMode::Glass;
	}
	else
	{
		//Metal mode is set when the user tried to create a metal material without using metalness
		if (ShaderElement->GetMetal() <= 0 && ShaderElement->GetMetalComp()->IsValid() == false && MetalTexture != nullptr)
		{
			if (((RefraTexture == nullptr && ShaderElement->GetTransparencyColor().IsAlmostBlack() && ShaderElement->GetTransComp()->IsValid() == false) &&
				(SpecularTexture != nullptr || ShaderElement->GetReflectanceColor().IsAlmostBlack() == false || ShaderElement->GetRefleComp()->IsValid()) &&
				ShaderElement->GetIOR() > METAL_IOR &&
				(DiffuseTexture == nullptr && ShaderElement->GetDiffuseColor().IsAlmostBlack() && ShaderElement->GetDiffuseComp()->IsValid() == false)))
			{
				MaterialMode = EDatasmithMaterialMode::Metal;
			}
			else if ((RefraTexture == nullptr && ShaderElement->GetTransparencyColor().IsAlmostBlack() && ShaderElement->GetTransComp()->IsValid() == false) &&
				(SpecularTexture != nullptr || ShaderElement->GetReflectanceColor().IsAlmostBlack() == false || ShaderElement->GetRefleComp()->IsValid()) &&
				ShaderElement->GetIOR() > MIXEDMETAL_IOR)
			{
				MaterialMode = EDatasmithMaterialMode::MixedMetal;
			}
		}
	}

	switch (MaterialMode)
	{
		case EDatasmithMaterialMode::Regular:
		{
			// diffuse
			if (AddCompExpression(ShaderElement->GetDiffuseComp(), UnrealMaterial, EDatasmithTextureSlot::DIFFUSE, AssetsContext) == nullptr)
			{
				if (DiffuseTexture != nullptr)
				{
					AddTextureExpression(DiffuseTexture, ShaderElement->GetDiffTextureSampler(), 0, 0, UnrealMaterial, EDatasmithTextureSlot::DIFFUSE);
				}
				else
				{
					AddRGBExpression(ShaderElement->GetDiffuseColor(), 0, 0, UnrealMaterial, EDatasmithTextureSlot::DIFFUSE);
				}
			}

			// specular
			if (AddCompExpression(ShaderElement->GetRefleComp(), UnrealMaterial, EDatasmithTextureSlot::SPECULAR, AssetsContext) == nullptr)
			{
				if (SpecularTexture != nullptr)
				{
					AddTextureExpression(SpecularTexture, ShaderElement->GetRefleTextureSampler(), ShaderElement->GetIOR(), ShaderElement->GetIORk(), UnrealMaterial, EDatasmithTextureSlot::SPECULAR);
				}
				else
				{
					if (AddRGBExpression(ShaderElement->GetReflectanceColor(), ShaderElement->GetIOR(), ShaderElement->GetIORk(), UnrealMaterial, EDatasmithTextureSlot::SPECULAR) == nullptr)
					{
						ConnectToSlot(Constant(UnrealMaterial, 0), UnrealMaterial, EDatasmithTextureSlot::SPECULAR);
					}
				}
			}

			// refract
			if (AddCompExpression(ShaderElement->GetTransComp(), UnrealMaterial, EDatasmithTextureSlot::OPACITY, AssetsContext) == nullptr)
			{
				if (RefraTexture != nullptr)
				{
					AddTextureExpression(RefraTexture, ShaderElement->GetTransTextureSampler(), ShaderElement->GetIORRefra(), 0, UnrealMaterial, EDatasmithTextureSlot::OPACITY);
				}
				else
				{
					AddRGBExpression(ShaderElement->GetTransparencyColor(), ShaderElement->GetIORRefra(), 0, UnrealMaterial, EDatasmithTextureSlot::OPACITY);
				}
			}

			// metal
			if (AddCompExpression(ShaderElement->GetMetalComp(), UnrealMaterial, EDatasmithTextureSlot::METALLIC, AssetsContext) == nullptr)
			{
				if (MetalTexture != nullptr)
				{
					AddTextureExpression(MetalTexture, ShaderElement->GetMetalTextureSampler(), 0, 0, UnrealMaterial, EDatasmithTextureSlot::METALLIC);
				}
				else
				{
					AddConstExpression(ShaderElement->GetMetal(), 0, 0, UnrealMaterial, EDatasmithTextureSlot::METALLIC);
				}
			}
			break;
		}
		case EDatasmithMaterialMode::Glass:
		{
			UMaterialExpression* RefleExpression = nullptr;
			UMaterialExpression* OpacityExpression = nullptr;

			// specular
			RefleExpression = AddCompExpression(ShaderElement->GetRefleComp(), UnrealMaterial, EDatasmithTextureSlot::NOSLOT, AssetsContext);
			if (RefleExpression == nullptr)
			{
				if (SpecularTexture != nullptr)
				{
					RefleExpression = AddTextureExpression(SpecularTexture, ShaderElement->GetRefleTextureSampler(), 0, 0, UnrealMaterial, EDatasmithTextureSlot::NOSLOT);
				}
				else
				{
					RefleExpression = AddRGBExpression(ShaderElement->GetReflectanceColor(), 0, 0, UnrealMaterial, EDatasmithTextureSlot::NOSLOT);
				}
			}

			// refract
			OpacityExpression = AddCompExpression(ShaderElement->GetTransComp(), UnrealMaterial, EDatasmithTextureSlot::NOSLOT, AssetsContext);
			if (OpacityExpression == nullptr)
			{
				if (RefraTexture != nullptr)
				{
					OpacityExpression = AddTextureExpression(RefraTexture, ShaderElement->GetTransTextureSampler(), 0, 0, UnrealMaterial, EDatasmithTextureSlot::NOSLOT);
				}
				else
				{
					OpacityExpression = AddRGBExpression(ShaderElement->GetTransparencyColor(), 0, 0, UnrealMaterial, EDatasmithTextureSlot::NOSLOT);
				}
			}

			AddGlassNode(RefleExpression, OpacityExpression, ShaderElement->GetIOR(), ShaderElement->GetIORRefra(), UnrealMaterial);
			if (ShaderElement->GetRoughness() == 0 && RoughnessTexture == nullptr)
			{
				AddConstExpression(0.01, 0, 0, UnrealMaterial, EDatasmithTextureSlot::ROUGHNESS);
			}

			break;
		}
		case EDatasmithMaterialMode::Metal:
		{
			UMaterialExpression* RefleExpression = nullptr;

			// in Metal color goes to diffuse and reflection has no effect

			if (AddCompExpression(ShaderElement->GetRefleComp(), UnrealMaterial, EDatasmithTextureSlot::DIFFUSE, AssetsContext) == nullptr)
			{
				if (SpecularTexture != nullptr)
				{
					RefleExpression = AddTextureExpression(SpecularTexture, ShaderElement->GetRefleTextureSampler(), 0, 0, UnrealMaterial, EDatasmithTextureSlot::DIFFUSE);
				}
				else
				{
					RefleExpression = AddRGBExpression(ShaderElement->GetReflectanceColor(), 0, 0, UnrealMaterial, EDatasmithTextureSlot::DIFFUSE);
				}
			}

			AddConstExpression(1.0, 0, 0, UnrealMaterial, EDatasmithTextureSlot::METALLIC);
			break;
		}
		case EDatasmithMaterialMode::MixedMetal:
		{
			// in Metal color goes to diffuse and reflection has no effect

			// specular
			UMaterialExpression* RefleExpression = AddCompExpression(ShaderElement->GetRefleComp(), UnrealMaterial, EDatasmithTextureSlot::SPECULAR, AssetsContext);
			if (RefleExpression == nullptr)
			{
				if (SpecularTexture != nullptr)
				{
					RefleExpression = AddTextureExpression(SpecularTexture, ShaderElement->GetRefleTextureSampler(), ShaderElement->GetIOR(), ShaderElement->GetIORk(), UnrealMaterial, EDatasmithTextureSlot::SPECULAR);
				}
				else
				{
					RefleExpression =
						AddRGBExpression(ShaderElement->GetReflectanceColor(), ShaderElement->GetIOR(), ShaderElement->GetIORk(), UnrealMaterial, EDatasmithTextureSlot::SPECULAR);
				}
			}

			UMaterialExpression* WeightedRefleExpression = Multiply(UnrealMaterial, RefleExpression, 0, RefleExpression, 0);
			ConnectToSlot(WeightedRefleExpression, UnrealMaterial, EDatasmithTextureSlot::METALLIC);

			// ShaderElement->IOR = 0;
			UMaterialExpression* ConstRefleExpression = AddCompExpression(ShaderElement->GetRefleComp(), UnrealMaterial, EDatasmithTextureSlot::NOSLOT, AssetsContext);
			if (ConstRefleExpression == nullptr)
			{
				if (SpecularTexture != nullptr)
				{
					ConstRefleExpression = AddTextureExpression(SpecularTexture, ShaderElement->GetRefleTextureSampler(), 0, 0, UnrealMaterial, EDatasmithTextureSlot::NOSLOT);
				}
				else
				{
					ConstRefleExpression = AddRGBExpression(ShaderElement->GetReflectanceColor(), 0, 0, UnrealMaterial, EDatasmithTextureSlot::NOSLOT);
				}
			}

			// diffuse
			UMaterialExpression* diffuseexpression = AddCompExpression(ShaderElement->GetDiffuseComp(), UnrealMaterial, EDatasmithTextureSlot::NOSLOT, AssetsContext);
			if (diffuseexpression == nullptr)
			{
				if (DiffuseTexture != nullptr)
				{
					diffuseexpression = AddTextureExpression(DiffuseTexture, ShaderElement->GetDiffTextureSampler(), 0, 0, UnrealMaterial, EDatasmithTextureSlot::NOSLOT);
				}
				else
				{
					diffuseexpression = AddRGBExpression(ShaderElement->GetDiffuseColor(), 0, 0, UnrealMaterial, EDatasmithTextureSlot::NOSLOT);
				}
			}

			UMaterialExpression* basecolorexpression = Lerp(UnrealMaterial, diffuseexpression, 0, true, ConstRefleExpression, 0, true, WeightedRefleExpression, 0);

			ConnectToSlot(basecolorexpression, UnrealMaterial, EDatasmithTextureSlot::DIFFUSE);

			break;
		}
	}

	// normal mapping
	UMaterialExpression* CompNormal = AddCompExpression(ShaderElement->GetNormalComp(), UnrealMaterial, EDatasmithTextureSlot::NORMAL, AssetsContext);
	if (CompNormal == nullptr)
	{
		if (NormalTexture != nullptr)
		{
			FDatasmithTextureSampler NormalSampler = ShaderElement->GetNormalTextureSampler();
			// invert on normal affects only Green channel and has been done on Texture load
			if (NormalSampler.bInvert)
			{
				NormalSampler.bInvert = false;
			}

			AddTextureExpression(NormalTexture, NormalSampler, 0.0, 0.0, UnrealMaterial, EDatasmithTextureSlot::NORMAL);
		}
	}

	if ((CompNormal!=nullptr || NormalTexture!=nullptr) && AdditionalBumpTexture != nullptr)
	{
		UMaterialExpressionTextureObject* TextureObject = AddTextureObjectExpression(AdditionalBumpTexture, UnrealMaterial);
		CreateParallaxOffset(UnrealMaterial, TextureObject, ShaderElement->GetBumpTextureSampler(), ShaderElement->GetBumpAmount());
	}

	ModulateNormalAmount(UnrealMaterial, ShaderElement->GetBumpAmount());
}

UMaterialExpressionMaterialFunctionCall* FDatasmithMaterialExpressions::BlendFunctions(UMaterial* UnrealMaterial, const FDatasmithAssetsImportContext& AssetsContext,
																					  const TSharedPtr< IDatasmithShaderElement >& ShaderTop, UMaterialExpressionMaterialFunctionCall* CallTop,
																					  UMaterialExpressionMaterialFunctionCall* CallBase, UMaterialFunction* BlendFunc)
{
	if (BlendFunc)
	{
		UMaterialExpression* WeightExpression = AddCompExpression(ShaderTop->GetWeightComp(), UnrealMaterial, EDatasmithTextureSlot::NOSLOT, AssetsContext);
		if (WeightExpression == nullptr)
		{
			UTexture* WeightTexture = FDatasmithImporterUtils::FindAsset< UTexture >( AssetsContext, ShaderTop->GetWeightTexture() );

			if (WeightTexture != nullptr)
			{
				FDatasmithTextureSampler WeightSampler = ShaderTop->GetWeightTextureSampler();
				WeightSampler.Multiplier = ShaderTop->GetWeightValue();
				WeightExpression = AddTextureExpression(WeightTexture, WeightSampler, 0, 0, UnrealMaterial, EDatasmithTextureSlot::NOSLOT);
			}
			else
			{
				if (ShaderTop->GetWeightColor().IsAlmostBlack() == false)
				{
					WeightExpression = AddRGBExpression(ShaderTop->GetWeightColor(), 0, 0, UnrealMaterial, EDatasmithTextureSlot::NOSLOT);
				}
				else
				{
					WeightExpression = AddConstExpression(ShaderTop->GetWeightValue(), 0, 0, UnrealMaterial, EDatasmithTextureSlot::NOSLOT);
				}
			}
		}

		UMaterialExpressionMaterialFunctionCall* ExpFuncCall = CreateMaterialExpression<UMaterialExpressionMaterialFunctionCall>(UnrealMaterial);
		ExpFuncCall->MaterialFunction = BlendFunc;

		ExpFuncCall->UpdateFromFunctionResource();

		CallBase->ConnectExpression(ExpFuncCall->GetInput(0), 0);
		CallTop->ConnectExpression(ExpFuncCall->GetInput(1), 0);

		if (WeightExpression)
		{
			WeightExpression->ConnectExpression(ExpFuncCall->GetInput(2), 0);
		}

		UMaterialEditorOnlyData* UnrealMaterialEditorOnly = UnrealMaterial->GetEditorOnlyData();
		UnrealMaterialEditorOnly->MaterialAttributes.Expression = ExpFuncCall;
		return ExpFuncCall;
	}
	else
	{
		FMessageDialog::Open(EAppMsgType::Ok, FText::FromString( TEXT("CANNOT LOAD MatLayerBlend_Standard") ));
		return nullptr;
	}
}

// any kind of material
UMaterialInterface* FDatasmithMaterialExpressions::CreateDatasmithMaterial(UPackage* Package, const TSharedPtr< IDatasmithMaterialElement >& MaterialElement, FDatasmithAssetsImportContext& AssetsContext,
																		  UMaterial* ExistingMaterial, EObjectFlags ObjectFlags)
{
	FString MaterialElementName = MaterialElement->GetName();

	if (MaterialElement->IsSingleShaderMaterial())
	{
		MaterialElement->GetShader(0)->SetName( *MaterialElementName );
		return CreateDatasmithMaterial(Package, MaterialElement->GetShader(0), AssetsContext, ExistingMaterial, ObjectFlags);
	}

	if (MaterialElement->IsClearCoatMaterial())
	{
		UMaterialInterface* TempMat = CreateDatasmithMaterial(Package, MaterialElement->GetShader(1), AssetsContext, ExistingMaterial, ObjectFlags);
		UMaterial* ActualMaterial = (UMaterial*)TempMat;
		ActualMaterial->SetShadingModel(EMaterialShadingModel::MSM_ClearCoat);

		CreateDatasmithMaterialCoat(AssetsContext, MaterialElement->GetShader(0), TempMat);
		UMaterialEditingLibrary::LayoutMaterialExpressions( ActualMaterial );

		return TempMat;
	}

	TArray<UMaterialFunction*> MatFunctions;
	for (int32 i = 0; i < MaterialElement->GetShadersCount(); i++)
	{
		MatFunctions.Add(CreateDatasmithMaterialFunc(Package, MaterialElement, MaterialElement->GetShader(i), AssetsContext, nullptr, ObjectFlags));
	}

	UMaterial* UnrealMaterial = nullptr;
	if (ExistingMaterial == nullptr)
	{
		FString FixedMaterialName = GenerateUniqueMaterialName( MaterialElement->GetLabel(), Package, AssetsContext.MaterialNameProvider );

		// Verify that the material could be created in final package
		FText FailReason;
		if (!FDatasmithImporterUtils::CanCreateAsset<UMaterial>( AssetsContext.MaterialsFinalPackage.Get(), FixedMaterialName, FailReason ))
		{
			AssetsContext.GetParentContext().LogError(FailReason);
			return nullptr;
		}

		UMaterialFactoryNew* MatFact = NewObject<UMaterialFactoryNew>();
		MatFact->AddToRoot();

		UnrealMaterial = (UMaterial*)MatFact->FactoryCreateNew(UMaterial::StaticClass(), Package, FName(*FixedMaterialName), ObjectFlags, nullptr, GWarn);
		UnrealMaterial->MarkPackageDirty();

		FAssetRegistryModule::AssetCreated(UnrealMaterial);
		Package->SetDirtyFlag(true);
		MatFact->RemoveFromRoot();
	}
	else
	{
		UnrealMaterial = ExistingMaterial;
		UMaterialEditorOnlyData* UnrealMaterialEditorOnly = UnrealMaterial->GetEditorOnlyData();

		UnrealMaterialEditorOnly->BaseColor.Expression = nullptr;
		UnrealMaterialEditorOnly->Metallic.Expression = nullptr;
		UnrealMaterialEditorOnly->Specular.Expression = nullptr;
		UnrealMaterialEditorOnly->Roughness.Expression = nullptr;
		UnrealMaterialEditorOnly->Normal.Expression = nullptr;
		UnrealMaterialEditorOnly->EmissiveColor.Expression = nullptr;
		UnrealMaterialEditorOnly->Opacity.Expression = nullptr;
		UnrealMaterialEditorOnly->OpacityMask.Expression = nullptr;
		UnrealMaterial->GetExpressionCollection().Empty();
	}
	UnrealMaterial->bUseMaterialAttributes = true;

	TArray<UMaterialExpressionMaterialFunctionCall*> FunctionCallArray;
	for (int32 i = 0; i < MatFunctions.Num(); i++)
	{
		FString FuncCallName = MatFunctions[i]->GetName() + TEXT("instanced");
		UMaterialExpressionMaterialFunctionCall* ExpFuncCall = CreateMaterialExpression<UMaterialExpressionMaterialFunctionCall>(UnrealMaterial);
		ExpFuncCall->MaterialFunction = MatFunctions[i];
		ExpFuncCall->UpdateFromFunctionResource();

		FunctionCallArray.Add(ExpFuncCall);
	}

	UMaterialFunction* BlendStandard =
		LoadObject<UMaterialFunction>(nullptr, TEXT("/Engine/Functions/MaterialLayerFunctions/MatLayerBlend_Standard.MatLayerBlend_Standard"), nullptr, LOAD_None, nullptr);

	UMaterialExpressionMaterialFunctionCall* PrevCall = BlendFunctions(UnrealMaterial, AssetsContext, MaterialElement->GetShader( MaterialElement->GetShadersCount() - 2 ),
		FunctionCallArray[FunctionCallArray.Num() - 2], FunctionCallArray[FunctionCallArray.Num() - 1], BlendStandard);

	for (int32 i = 2; i < MaterialElement->GetShadersCount(); i++)
	{
		PrevCall =
			BlendFunctions(UnrealMaterial, AssetsContext, MaterialElement->GetShader( MaterialElement->GetShadersCount() - i - 1 ), FunctionCallArray[FunctionCallArray.Num() - i - 1], PrevCall, BlendStandard);
	}

	UMaterialEditingLibrary::LayoutMaterialExpressions( UnrealMaterial );

	return UnrealMaterial;
}

// regular single material
UMaterialInterface* FDatasmithMaterialExpressions::CreateDatasmithMaterial(UPackage* Package, const TSharedPtr< IDatasmithShaderElement >& ShaderElement, FDatasmithAssetsImportContext& AssetsContext,
																		  UMaterial* ExistingMaterial, EObjectFlags ObjectFlags)
{

	FString FixedMaterialName = GenerateUniqueMaterialName( ShaderElement->GetLabel(), Package, AssetsContext.MaterialNameProvider );
	UMaterial* UnrealMaterial = nullptr;
	if (ExistingMaterial == nullptr)
	{
		// Verify that the material could be created in final package
		FText FailReason;
		if (!FDatasmithImporterUtils::CanCreateAsset<UMaterialInterface>( AssetsContext.MaterialsFinalPackage.Get(), FixedMaterialName, FailReason ))
		{
			AssetsContext.GetParentContext().LogError(FailReason);
			return nullptr;
		}

		UMaterialFactoryNew* MatFact = NewObject<UMaterialFactoryNew>();
		MatFact->AddToRoot();

		UnrealMaterial = (UMaterial*)MatFact->FactoryCreateNew(UMaterial::StaticClass(), Package, FName(*FixedMaterialName), ObjectFlags, nullptr, GWarn);
		UnrealMaterial->MarkPackageDirty();
		FAssetRegistryModule::AssetCreated(UnrealMaterial);
		Package->SetDirtyFlag(true);
		MatFact->RemoveFromRoot();
	}
	else
	{
		UnrealMaterial = ExistingMaterial;
		UMaterialEditorOnlyData* UnrealMaterialEditorOnly = UnrealMaterial->GetEditorOnlyData();
		UnrealMaterialEditorOnly->BaseColor.Expression = nullptr;
		UnrealMaterialEditorOnly->Metallic.Expression = nullptr;
		UnrealMaterialEditorOnly->Specular.Expression = nullptr;
		UnrealMaterialEditorOnly->Roughness.Expression = nullptr;
		UnrealMaterialEditorOnly->Normal.Expression = nullptr;
		UnrealMaterialEditorOnly->EmissiveColor.Expression = nullptr;
		UnrealMaterialEditorOnly->Opacity.Expression = nullptr;
		UnrealMaterialEditorOnly->OpacityMask.Expression = nullptr;
		UnrealMaterial->GetExpressionCollection().Empty();
	}

	CreateDatasmithMaterialHelper(Package, ShaderElement, AssetsContext, UnrealMaterial);

	if (ShaderElement->GetShaderUsage() == EDatasmithShaderUsage::LightFunction)
	{
		UnrealMaterial->MaterialDomain = EMaterialDomain::MD_LightFunction;
	}
	UnrealMaterial->bUseEmissiveForDynamicAreaLighting = ShaderElement->GetUseEmissiveForDynamicAreaLighting();

	UMaterialEditingLibrary::LayoutMaterialExpressions( UnrealMaterial );

	return UnrealMaterial;
}

// material function
UMaterialFunction* FDatasmithMaterialExpressions::CreateDatasmithMaterialFunc(UPackage* Package, const TSharedPtr< IDatasmithMaterialElement >& MaterialElement, const TSharedPtr< IDatasmithShaderElement >& ShaderElement,
																			  const FDatasmithAssetsImportContext& AssetsContext, UMaterialFunction* FindMaterialFunc, EObjectFlags ObjectFlags)
{
	FString MaterialElementName = MaterialElement->GetName();
	FString ShaderElementName = ShaderElement->GetName();

	FString FixedMaterialName = MaterialElementName + TEXT("_") + ShaderElementName + TEXT("_func");
	UMaterialFunction* UnrealMaterialFunc = nullptr;
	if (FindMaterialFunc == nullptr)
	{
		UnrealMaterialFunc = NewObject<UMaterialFunction>(Package, FName(*FixedMaterialName), ObjectFlags);
	}
	else
	{
		UnrealMaterialFunc = FindMaterialFunc;
	}

	UMaterialExpressionMakeMaterialAttributes* Attrib = FindOrAddAttributesFromMatFunc(UnrealMaterialFunc);

	Attrib->BaseColor.Expression = nullptr;
	Attrib->Metallic.Expression = nullptr;
	Attrib->Specular.Expression = nullptr;
	Attrib->Roughness.Expression = nullptr;
	Attrib->Normal.Expression = nullptr;
	Attrib->EmissiveColor.Expression = nullptr;
	Attrib->Opacity.Expression = nullptr;
	Attrib->OpacityMask.Expression = nullptr;
	UnrealMaterialFunc->GetExpressionCollection().Empty();

	CreateDatasmithMaterialHelper(Package, ShaderElement, AssetsContext, UnrealMaterialFunc);

	UMaterialEditingLibrary::LayoutMaterialFunctionExpressions( UnrealMaterialFunc );

	UnrealMaterialFunc->UpdateDependentFunctionCandidates();

	return UnrealMaterialFunc;
}

UMaterialInterface* FDatasmithMaterialExpressions::CreateDatasmithEnvironmentMaterial(UPackage* Package, const TSharedPtr< IDatasmithShaderElement >& ShaderElement,
																					  FDatasmithAssetsImportContext& AssetsContext, UMaterial* ExistingMaterial)
{
	FString FixedMaterialName = GenerateUniqueMaterialName(ShaderElement->GetLabel(), Package, AssetsContext.MaterialNameProvider);
	UMaterial* UnrealMaterial = nullptr;

	if (ExistingMaterial == nullptr)
	{
		UMaterialFactoryNew* MatFact = NewObject<UMaterialFactoryNew>();
		MatFact->AddToRoot();

		UnrealMaterial =
			(UMaterial*)MatFact->FactoryCreateNew(UMaterial::StaticClass(), Package, FName(*FixedMaterialName), RF_Standalone | RF_Public, nullptr, GWarn);

		UnrealMaterial->MarkPackageDirty();
		FAssetRegistryModule::AssetCreated(UnrealMaterial);
		Package->SetDirtyFlag(true);
		MatFact->RemoveFromRoot();
	}
	else
	{
		UnrealMaterial = ExistingMaterial;
		UMaterialEditorOnlyData* UnrealMaterialEditorOnly = UnrealMaterial->GetEditorOnlyData();
		UnrealMaterialEditorOnly->BaseColor.Expression = nullptr;
		UnrealMaterialEditorOnly->Metallic.Expression = nullptr;
		UnrealMaterialEditorOnly->Specular.Expression = nullptr;
		UnrealMaterialEditorOnly->Roughness.Expression = nullptr;
		UnrealMaterialEditorOnly->Normal.Expression = nullptr;
		UnrealMaterialEditorOnly->EmissiveColor.Expression = nullptr;
		UnrealMaterialEditorOnly->Opacity.Expression = nullptr;
		UnrealMaterialEditorOnly->OpacityMask.Expression = nullptr;
		UnrealMaterial->GetExpressionCollection().Empty();
	}

	Package->SetDirtyFlag(true);

	UTexture* EmissiveTexture = FDatasmithImporterUtils::FindAsset< UTexture >( AssetsContext, ShaderElement->GetEmitTexture() );

	if (EmissiveTexture == nullptr)
	{
		return nullptr;
	}

	bool bIsCubemap = EmissiveTexture->IsA<UTextureCube>();
	UnrealMaterial->SetShadingModel(EMaterialShadingModel::MSM_Unlit);
	UnrealMaterial->TwoSided = true;
	UMaterialExpressionTextureSample* TextureSample = nullptr;
	UMaterialExpression* TextureExpression = AddTextureExpression(EmissiveTexture, FDatasmithTextureSampler(), 0, 0, UnrealMaterial, EDatasmithTextureSlot::EMISSIVECOLOR, &TextureSample);

	UMaterialExpression* RotationAxis = AddRGBAExpression(FLinearColor(0.0, 0.0, 1.0, 0.0), 0, 0, UnrealMaterial, EDatasmithTextureSlot::NOSLOT);
	UMaterialExpression* Rotation = Constant(UnrealMaterial, ShaderElement->GetEmitTextureSampler().Rotation);
	UMaterialExpression* Pivot = Constant(UnrealMaterial, 0);

	UMaterialExpressionWorldPosition* WorldPos = CreateMaterialExpression<UMaterialExpressionWorldPosition>(UnrealMaterial);

	UMaterialExpressionRotateAboutAxis* RotAboutAxis = CreateMaterialExpression<UMaterialExpressionRotateAboutAxis>(UnrealMaterial);

	RotAboutAxis->NormalizedRotationAxis.Expression = RotationAxis;
	RotAboutAxis->RotationAngle.Expression = Rotation;
	RotAboutAxis->PivotPoint.Expression = Pivot;
	RotAboutAxis->Position.Expression = WorldPos;

	UMaterialExpression* BothAdded = Add(UnrealMaterial, RotAboutAxis, 0.0, WorldPos, 0.0);

	UMaterialExpressionNormalize* Normalize = CreateMaterialExpression<UMaterialExpressionNormalize>(UnrealMaterial);

	Normalize->VectorInput.Expression = BothAdded;

	if (!bIsCubemap && TextureSample)
	{
		// In this case, we want to connect the LongLatToUV material function to the UV input of the MaterialExpressionTextureSample created by the call to AddTextureExpression,
		// which is not necessarily the same as the return value
		UMaterialFunction* LongToUV = LoadObject<UMaterialFunction>(nullptr, TEXT("/Engine/Functions/Engine_MaterialFunctions01/Coordinates/LongLatToUV.LongLatToUV"), nullptr, LOAD_None, nullptr);
		UMaterialExpressionMaterialFunctionCall* ExpFuncCallLongUV = CreateMaterialExpression<UMaterialExpressionMaterialFunctionCall>(UnrealMaterial);
		ExpFuncCallLongUV->MaterialFunction = LongToUV;
		ExpFuncCallLongUV->UpdateFromFunctionResource();

		Normalize->ConnectExpression(ExpFuncCallLongUV->GetInput(0), 0);
		TextureSample->GetInput(0)->Expression = ExpFuncCallLongUV;
	}
	else
	{
		TextureExpression->GetInput(0)->Expression = Normalize;
	}

	UMaterialEditingLibrary::LayoutMaterialExpressions( UnrealMaterial );

	// let the material update itself if necessary
	UnrealMaterial->PreEditChange(nullptr);
	UnrealMaterial->PostEditChange();
	return UnrealMaterial;
}

UMaterialExpression* FDatasmithMaterialExpressions::CreateExpression( IDatasmithMaterialExpression* MaterialExpression, const FDatasmithAssetsImportContext& AssetsContext, UObject* UnrealMaterialOrFunction )
{
	if ( !MaterialExpression )
	{
		ensure( false );
		return nullptr;
	}

	if ( MaterialExpression->IsSubType( EDatasmithMaterialExpressionType::Texture ) )
	{
		return CreateTextureExpression( *static_cast< IDatasmithMaterialExpressionTexture* >( MaterialExpression ), AssetsContext, UnrealMaterialOrFunction );
	}
	else if ( MaterialExpression->IsSubType( EDatasmithMaterialExpressionType::TextureCoordinate ) )
	{
		return CreateTextureCoordinateExpression( *static_cast< IDatasmithMaterialExpressionTextureCoordinate* >( MaterialExpression ), AssetsContext, UnrealMaterialOrFunction );
	}
	else if ( MaterialExpression->IsSubType( EDatasmithMaterialExpressionType::FlattenNormal ) )
	{
		return CreateFlattenNormalExpression( *static_cast< IDatasmithMaterialExpressionFlattenNormal* >( MaterialExpression ), AssetsContext, UnrealMaterialOrFunction );
	}
	else if ( MaterialExpression->IsSubType( EDatasmithMaterialExpressionType::ConstantBool ) )
	{
		return CreateBoolExpression( *static_cast< IDatasmithMaterialExpressionBool* >( MaterialExpression ), AssetsContext, UnrealMaterialOrFunction );
	}
	else if ( MaterialExpression->IsSubType( EDatasmithMaterialExpressionType::ConstantColor ) )
	{
		return CreateColorExpression( *static_cast< IDatasmithMaterialExpressionColor* >( MaterialExpression ), AssetsContext, UnrealMaterialOrFunction );
	}
	else if ( MaterialExpression->IsSubType( EDatasmithMaterialExpressionType::ConstantScalar ) )
	{
		return CreateScalarExpression( *static_cast< IDatasmithMaterialExpressionScalar* >( MaterialExpression ), AssetsContext, UnrealMaterialOrFunction );
	}
	else if ( MaterialExpression->IsSubType( EDatasmithMaterialExpressionType::Generic ) )
	{
		return CreateGenericExpression( *static_cast< IDatasmithMaterialExpressionGeneric* >( MaterialExpression ), AssetsContext, UnrealMaterialOrFunction );
	}
	else if ( MaterialExpression->IsSubType( EDatasmithMaterialExpressionType::FunctionCall ) )
	{
		return CreateFunctionCallExpression( *static_cast< IDatasmithMaterialExpressionFunctionCall* >( MaterialExpression ), AssetsContext, UnrealMaterialOrFunction );
	}
	else if ( MaterialExpression->IsSubType( EDatasmithMaterialExpressionType::Custom ) )
	{
		return CreateCustomExpression( *static_cast< IDatasmithMaterialExpressionCustom* >( MaterialExpression ), AssetsContext, UnrealMaterialOrFunction );
	}
	else
	{
		ensure( false );
		return nullptr;
	}
}

void FDatasmithMaterialExpressions::ConnectExpression( const TSharedRef< IDatasmithUEPbrMaterialElement >& MaterialElement, TArray< TStrongObjectPtr< UMaterialExpression > >& MaterialExpressions,
	IDatasmithMaterialExpression* MaterialExpression, FExpressionInput* MaterialInput, int32 OutputIndex )
{
	if ( !MaterialExpression || !MaterialInput )
	{
		return;
	}

	ConnectAnyExpression( MaterialElement, MaterialExpressions, *MaterialExpression, MaterialInput, OutputIndex );
}

UMaterialExpression* FDatasmithMaterialExpressions::CreateTextureExpression( IDatasmithMaterialExpressionTexture& DatasmithTextureExpression, const FDatasmithAssetsImportContext& AssetsContext, UObject* UnrealMaterialOrFunction )
{
	UTexture* Texture = FDatasmithImporterUtils::FindAsset< UTexture >( AssetsContext, DatasmithTextureExpression.GetTexturePathName() );

	UMaterialExpressionTextureSampleParameter2D* TextureExpression = CreateMaterialExpression< UMaterialExpressionTextureSampleParameter2D >( UnrealMaterialOrFunction );
	TextureExpression->ParameterName = GenerateParamName<EDatasmithMaterialExpressionType::Texture>( DatasmithTextureExpression, GetNumberOfExpressionInMaterialOrFunction( UnrealMaterialOrFunction ) );

	if ( Texture )
	{
		TextureExpression->Group = DatasmithTextureExpression.GetGroupName();
		TextureExpression->Texture = Texture;
		TextureExpression->SamplerType = MaterialExpressionUtils::GetSamplerTypeForTexture( Texture );
	}

	return TextureExpression;
}

UMaterialExpression* FDatasmithMaterialExpressions::CreateTextureCoordinateExpression( IDatasmithMaterialExpressionTextureCoordinate& DatasmithTextureCoordinateExpression, const FDatasmithAssetsImportContext& AssetsContext, UObject* UnrealMaterialOrFunction )
{
	UMaterialExpressionTextureCoordinate* TextureCoordinate = CreateMaterialExpression< UMaterialExpressionTextureCoordinate >( UnrealMaterialOrFunction );
	TextureCoordinate->CoordinateIndex = DatasmithTextureCoordinateExpression.GetCoordinateIndex();
	TextureCoordinate->UTiling = DatasmithTextureCoordinateExpression.GetUTiling();
	TextureCoordinate->VTiling = DatasmithTextureCoordinateExpression.GetVTiling();

	return TextureCoordinate;
}

UMaterialExpression* FDatasmithMaterialExpressions::CreateFlattenNormalExpression( IDatasmithMaterialExpressionFlattenNormal& DatasmithFlattenNormal, const FDatasmithAssetsImportContext& AssetsContext, UObject* UnrealMaterialOrFunction )
{
	UMaterialFunction* FlattenFunction = LoadObject< UMaterialFunction >(nullptr, TEXT("/Engine/Functions/Engine_MaterialFunctions01/Texturing/FlattenNormal.FlattenNormal"), nullptr, LOAD_None, nullptr);

	UMaterialExpressionMaterialFunctionCall* ExpressionFunctionCall = CreateMaterialExpression< UMaterialExpressionMaterialFunctionCall >( UnrealMaterialOrFunction );
	ExpressionFunctionCall->MaterialFunction = FlattenFunction;
	ExpressionFunctionCall->UpdateFromFunctionResource();

	return ExpressionFunctionCall;
}

UMaterialExpression* FDatasmithMaterialExpressions::CreateBoolExpression( IDatasmithMaterialExpressionBool& DatasmithBool, const FDatasmithAssetsImportContext& AssetsContext, UObject* UnrealMaterialOrFunction )
{
	UMaterialExpressionStaticBoolParameter* ConstantBool = CreateMaterialExpression< UMaterialExpressionStaticBoolParameter >( UnrealMaterialOrFunction );
	ConstantBool->ParameterName = GenerateParamName<EDatasmithMaterialExpressionType::ConstantBool>( DatasmithBool, GetNumberOfExpressionInMaterialOrFunction( UnrealMaterialOrFunction ) );
	ConstantBool->DefaultValue = DatasmithBool.GetBool();
	ConstantBool->Group = DatasmithBool.GetGroupName();

	return ConstantBool;
}

UMaterialExpression* FDatasmithMaterialExpressions::CreateColorExpression( IDatasmithMaterialExpressionColor& DatasmithColor, const FDatasmithAssetsImportContext& AssetsContext, UObject* UnrealMaterialOrFunction )
{
	UMaterialExpression* Result = nullptr;

	if ( ShouldExpressionBeAParameter( DatasmithColor ) )
	{
		UMaterialExpressionVectorParameter* ConstantColor = CreateMaterialExpression< UMaterialExpressionVectorParameter >( UnrealMaterialOrFunction );
		ConstantColor->ParameterName = GenerateParamName<EDatasmithMaterialExpressionType::ConstantColor>( DatasmithColor, GetNumberOfExpressionInMaterialOrFunction( UnrealMaterialOrFunction ) );
		ConstantColor->DefaultValue = DatasmithColor.GetColor();
		ConstantColor->Group = DatasmithColor.GetGroupName();

		Result = ConstantColor;
	}
	else
	{
		UMaterialExpressionConstant3Vector* ConstantColor = CreateMaterialExpression< UMaterialExpressionConstant3Vector >( UnrealMaterialOrFunction );
		ConstantColor->Constant = DatasmithColor.GetColor();

		Result = ConstantColor;
	}

	return Result;
}

UMaterialExpression* FDatasmithMaterialExpressions::CreateScalarExpression( IDatasmithMaterialExpressionScalar& DatasmithScalar, const FDatasmithAssetsImportContext& AssetsContext, UObject* UnrealMaterialOrFunction )
{
	UMaterialExpression* Result = nullptr;

	if ( ShouldExpressionBeAParameter( DatasmithScalar ) )
	{
		UMaterialExpressionScalarParameter* ScalarExpression = CreateMaterialExpression< UMaterialExpressionScalarParameter >( UnrealMaterialOrFunction );
		ScalarExpression->ParameterName = GenerateParamName<EDatasmithMaterialExpressionType::ConstantScalar>( DatasmithScalar, GetNumberOfExpressionInMaterialOrFunction( UnrealMaterialOrFunction ) );
		ScalarExpression->DefaultValue = DatasmithScalar.GetScalar();
		ScalarExpression->Group = DatasmithScalar.GetGroupName();

		Result = ScalarExpression;
	}
	else
	{
		UMaterialExpressionConstant* ScalarExpression = CreateMaterialExpression< UMaterialExpressionConstant >( UnrealMaterialOrFunction );
		ScalarExpression->R = DatasmithScalar.GetScalar();

		Result = ScalarExpression;
	}

	return Result;
}

UMaterialExpression* FDatasmithMaterialExpressions::CreateCustomExpression(IDatasmithMaterialExpressionCustom& DatasmithCustom, const FDatasmithAssetsImportContext& AssetsContext, UObject* UnrealMaterialOrFunction)
{
	UMaterialExpressionCustom* CustomExpression = CreateMaterialExpression< UMaterialExpressionCustom >( UnrealMaterialOrFunction );
	CustomExpression->Code = DatasmithCustom.GetCode();

	auto DatasmithTypeToNative = [](EDatasmithShaderDataType DatasmithValue) -> ECustomMaterialOutputType
	{
		switch (DatasmithValue)
		{
			case EDatasmithShaderDataType::Float1: return ECustomMaterialOutputType::CMOT_Float1;
			case EDatasmithShaderDataType::Float2: return ECustomMaterialOutputType::CMOT_Float2;
			case EDatasmithShaderDataType::Float3: return ECustomMaterialOutputType::CMOT_Float3;
			case EDatasmithShaderDataType::Float4: return ECustomMaterialOutputType::CMOT_Float4;
			case EDatasmithShaderDataType::MaterialAttribute: return ECustomMaterialOutputType::CMOT_MaterialAttributes;
			default: return ECustomMaterialOutputType::CMOT_Float1; // log
		}
	};

	CustomExpression->OutputType = DatasmithTypeToNative(DatasmithCustom.GetOutputType());
	CustomExpression->Description = DatasmithCustom.GetDescription();

	if (DatasmithCustom.GetArgumentNameCount() > CustomExpression->Inputs.Num())
	{
		CustomExpression->Inputs.AddDefaulted(DatasmithCustom.GetArgumentNameCount() - CustomExpression->Inputs.Num());
	}
	for (int32 Index = 0; Index < DatasmithCustom.GetArgumentNameCount(); ++Index)
	{
		CustomExpression->Inputs[Index].InputName = DatasmithCustom.GetArgumentName(Index);
	}

	for (int32 IncludeFilePathIndex = 0; IncludeFilePathIndex < DatasmithCustom.GetIncludeFilePathCount(); ++IncludeFilePathIndex)
	{
		CustomExpression->IncludeFilePaths.Add(DatasmithCustom.GetIncludeFilePath(IncludeFilePathIndex));
	}

	for (int32 AdditionalDefineIndex = 0; AdditionalDefineIndex < DatasmithCustom.GetAdditionalDefineCount(); ++AdditionalDefineIndex)
	{
		FCustomDefine& Define = CustomExpression->AdditionalDefines.AddDefaulted_GetRef();
		FString DefineStr = DatasmithCustom.GetAdditionalDefine(AdditionalDefineIndex);

		int32 Index = -1;
		if (DefineStr.FindChar(TEXT('='), Index))
		{
			Define.DefineName = DefineStr.Left(Index);
			Define.DefineValue = DefineStr.RightChop(Index+1);
		}
		else
		{
			Define.DefineName = DefineStr;
		}
	}

	return CustomExpression;
}


UMaterialExpression* FDatasmithMaterialExpressions::CreateGenericExpression( IDatasmithMaterialExpressionGeneric& DatasmithGeneric, const FDatasmithAssetsImportContext& AssetsContext, UObject* UnrealMaterialOrFunction )
{
	UClass* ExpressionClass = GetGenericExpressionClass( DatasmithGeneric );

	if ( !ExpressionClass )
	{
		ensure( false );
		return nullptr;
	}

	UMaterialExpression* MaterialExpression = CreateMaterialExpression( UnrealMaterialOrFunction, ExpressionClass );

	if ( !MaterialExpression )
	{
		return nullptr;
	}

	bool bShouldHaveParameterName = ShouldExpressionBeAParameter( DatasmithGeneric );

	// Validate the new logic at least for a version or two (added in 4.27)
	check( bShouldHaveParameterName == MaterialExpression->HasAParameterName() )

	if ( bShouldHaveParameterName )
	{
		MaterialExpression->SetParameterName( GenerateParamName<EDatasmithMaterialExpressionType::Generic>( DatasmithGeneric, GetNumberOfExpressionInMaterialOrFunction(UnrealMaterialOrFunction) ) );
	}

	for ( int32 PropertyIndex = 0; PropertyIndex < DatasmithGeneric.GetPropertiesCount(); ++PropertyIndex )
	{
		const TSharedPtr< IDatasmithKeyValueProperty >& KeyValueProperty = DatasmithGeneric.GetProperty( PropertyIndex );

		if ( !KeyValueProperty )
		{
			continue;
		}

		const TCHAR* PropertyName = KeyValueProperty->GetName();
		FProperty* Property = MaterialExpression->GetClass()->FindPropertyByName( PropertyName );

		// fallback to search by display name
		if (Property == nullptr)
		{
			for (FProperty* PropertyIt = MaterialExpression->GetClass()->PropertyLink; PropertyIt != NULL; PropertyIt = PropertyIt->PropertyLinkNext)
			{
				if (PropertyIt->GetDisplayNameText().ToString() == PropertyName)
				{
					Property = PropertyIt;
				}
			}
		}

		EDatasmithKeyValuePropertyType PropertyType = KeyValueProperty->GetPropertyType();
		switch (PropertyType)
		{
			case EDatasmithKeyValuePropertyType::Texture:
			{
				FObjectProperty* ObjectProperty = CastField< FObjectProperty >( Property );

				if ( ObjectProperty )
				{
					UTexture* Texture = FDatasmithImporterUtils::FindAsset< UTexture >( AssetsContext, KeyValueProperty->GetValue() );

					if ( Texture )
					{
						ObjectProperty->SetPropertyValue( ObjectProperty->ContainerPtrToValuePtr< void >( MaterialExpression ), Texture );
					}
				}
				break;
			}

			default:
			{
				if ( Property )
				{
					Property->ImportText_Direct( KeyValueProperty->GetValue(), Property->ContainerPtrToValuePtr< void >( MaterialExpression ), nullptr, PPF_None);
				}
			}
		}
	}

	if ( UMaterialExpressionTextureBase* TextureExpression = Cast< UMaterialExpressionTextureBase >( MaterialExpression ) )
	{
		TextureExpression->AutoSetSampleType();
	}

	return MaterialExpression;
}

UMaterialExpression* FDatasmithMaterialExpressions::CreateFunctionCallExpression( class IDatasmithMaterialExpressionFunctionCall& DatasmithFunctionCall, const FDatasmithAssetsImportContext& AssetsContext, UObject* UnrealMaterialOrFunction )
{
	FString FunctionPath = FPackageName::ExportTextPathToObjectPath(DatasmithFunctionCall.GetFunctionPathName());
	UMaterialFunction* MaterialFunction = FDatasmithImporterUtils::FindAsset< UMaterialFunction >( AssetsContext, *FunctionPath );

	UMaterialExpressionMaterialFunctionCall* FunctionCall = CreateMaterialExpression< UMaterialExpressionMaterialFunctionCall >( UnrealMaterialOrFunction );
	FunctionCall->SetMaterialFunction( MaterialFunction );
	FunctionCall->UpdateFromFunctionResource();

	return FunctionCall;
}

void FDatasmithMaterialExpressions::ConnectAnyExpression( const TSharedRef< IDatasmithUEPbrMaterialElement >& MaterialElement, TArray< TStrongObjectPtr< UMaterialExpression > >& MaterialExpressions,
	IDatasmithMaterialExpression& DatasmithExpression, FExpressionInput* ExpressionInput, int32 OutputIndex )
{
	int32 ExpressionIndex = MaterialElement->GetExpressionIndex( &DatasmithExpression );

	if ( !MaterialExpressions.IsValidIndex( ExpressionIndex ) )
	{
		// TODO: Log error
		return;
	}

	TStrongObjectPtr< UMaterialExpression >& MaterialExpression = MaterialExpressions[ ExpressionIndex ];

	MaterialExpression->ConnectExpression( ExpressionInput, OutputIndex );

	for ( int32 InputIndex = 0; InputIndex < DatasmithExpression.GetInputCount(); ++InputIndex )
	{
		ConnectExpression( MaterialElement, MaterialExpressions, DatasmithExpression.GetInput( InputIndex )->GetExpression(), MaterialExpression->GetInput( InputIndex ), DatasmithExpression.GetInput( InputIndex )->GetOutputIndex() );
	}
}

void FDatasmithMaterialExpressions::CreateUEPbrMaterialGraph(const TSharedPtr< IDatasmithUEPbrMaterialElement >& MaterialElement, FDatasmithAssetsImportContext& AssetsContext, UObject* UnrealMaterialOrFunction)
{
	TArray< TStrongObjectPtr< UMaterialExpression > > MaterialExpressions;
	MaterialExpressions.Reserve( MaterialElement->GetExpressionsCount() );

	for ( int32 ExpressionIndex = 0; ExpressionIndex < MaterialElement->GetExpressionsCount(); ++ExpressionIndex )
	{
		UMaterialExpression* MaterialExpression = nullptr;

		if ( IDatasmithMaterialExpression* DatasmithExpression = MaterialElement->GetExpression( ExpressionIndex ) )
		{
			MaterialExpression = CreateExpression( DatasmithExpression, AssetsContext, UnrealMaterialOrFunction );
			check(MaterialExpression);
		}

		MaterialExpressions.Emplace( MaterialExpression );
	}

	if ( MaterialElement->GetUseMaterialAttributes() )
	{
		//We ignore all the other inputs if we are using MaterialAttributes.
		ConnectExpression( MaterialElement.ToSharedRef(), MaterialExpressions, MaterialElement->GetMaterialAttributes().GetExpression(), GetMaterialOrFunctionSlot( UnrealMaterialOrFunction, EDatasmithTextureSlot::MATERIALATTRIBUTES ), MaterialElement->GetMaterialAttributes().GetOutputIndex() );
		return;
	}

	ConnectExpression( MaterialElement.ToSharedRef(), MaterialExpressions, MaterialElement->GetBaseColor().GetExpression(), GetMaterialOrFunctionSlot( UnrealMaterialOrFunction, EDatasmithTextureSlot::DIFFUSE ), MaterialElement->GetBaseColor().GetOutputIndex() );

	ConnectExpression( MaterialElement.ToSharedRef(), MaterialExpressions, MaterialElement->GetMetallic().GetExpression(), GetMaterialOrFunctionSlot( UnrealMaterialOrFunction, EDatasmithTextureSlot::METALLIC ), MaterialElement->GetMetallic().GetOutputIndex() );

	ConnectExpression( MaterialElement.ToSharedRef(), MaterialExpressions, MaterialElement->GetSpecular().GetExpression(), GetMaterialOrFunctionSlot( UnrealMaterialOrFunction, EDatasmithTextureSlot::SPECULAR ), MaterialElement->GetSpecular().GetOutputIndex() );

	ConnectExpression( MaterialElement.ToSharedRef(), MaterialExpressions, MaterialElement->GetRoughness().GetExpression(), GetMaterialOrFunctionSlot( UnrealMaterialOrFunction, EDatasmithTextureSlot::ROUGHNESS ), MaterialElement->GetRoughness().GetOutputIndex() );

	ConnectExpression( MaterialElement.ToSharedRef(), MaterialExpressions, MaterialElement->GetEmissiveColor().GetExpression(), GetMaterialOrFunctionSlot( UnrealMaterialOrFunction, EDatasmithTextureSlot::EMISSIVECOLOR ), MaterialElement->GetEmissiveColor().GetOutputIndex() );

	ConnectExpression( MaterialElement.ToSharedRef(), MaterialExpressions, MaterialElement->GetAmbientOcclusion().GetExpression(), GetMaterialOrFunctionSlot( UnrealMaterialOrFunction, EDatasmithTextureSlot::AMBIANTOCCLUSION ), MaterialElement->GetAmbientOcclusion().GetOutputIndex() );

	if ( MaterialElement->GetOpacity().GetExpression() )
	{
		if ( GetUEPbrImportBlendMode(MaterialElement, AssetsContext) == BLEND_Translucent )
		{
			ConnectExpression( MaterialElement.ToSharedRef(), MaterialExpressions, MaterialElement->GetOpacity().GetExpression(), GetMaterialOrFunctionSlot( UnrealMaterialOrFunction, EDatasmithTextureSlot::OPACITY ), MaterialElement->GetOpacity().GetOutputIndex() );

			ConnectExpression( MaterialElement.ToSharedRef(), MaterialExpressions, MaterialElement->GetRefraction().GetExpression(), GetMaterialOrFunctionSlot( UnrealMaterialOrFunction, EDatasmithTextureSlot::REFRACTION ), MaterialElement->GetRefraction().GetOutputIndex() );
		}
		else
		{
			ConnectExpression( MaterialElement.ToSharedRef(), MaterialExpressions, MaterialElement->GetOpacity().GetExpression(), GetMaterialOrFunctionSlot( UnrealMaterialOrFunction, EDatasmithTextureSlot::OPACITYMASK ), MaterialElement->GetOpacity().GetOutputIndex() );
		}
	}

	ConnectExpression( MaterialElement.ToSharedRef(), MaterialExpressions, MaterialElement->GetNormal().GetExpression(), GetMaterialOrFunctionSlot( UnrealMaterialOrFunction, EDatasmithTextureSlot::NORMAL ), MaterialElement->GetNormal().GetOutputIndex() );

	if ( MaterialElement->GetShadingModel() == EDatasmithShadingModel::ClearCoat )
	{
		if ( MaterialElement->GetClearCoat().GetExpression() )
		{
			ConnectExpression(MaterialElement.ToSharedRef(), MaterialExpressions, MaterialElement->GetClearCoat().GetExpression(), GetMaterialOrFunctionSlot(UnrealMaterialOrFunction, EDatasmithTextureSlot::COATSPECULAR), MaterialElement->GetClearCoat().GetOutputIndex());
		}
		if (MaterialElement->GetClearCoatRoughness().GetExpression())
		{
			ConnectExpression(MaterialElement.ToSharedRef(), MaterialExpressions, MaterialElement->GetClearCoatRoughness().GetExpression(), GetMaterialOrFunctionSlot(UnrealMaterialOrFunction, EDatasmithTextureSlot::COATROUGHNESS), MaterialElement->GetClearCoat().GetOutputIndex());
		}
	}

	if (MaterialElement->GetWorldPositionOffset().GetExpression())
	{
		ConnectExpression(MaterialElement.ToSharedRef(), MaterialExpressions, MaterialElement->GetWorldPositionOffset().GetExpression(), GetMaterialOrFunctionSlot(UnrealMaterialOrFunction, EDatasmithTextureSlot::WORLDPOSITIONOFFSET), MaterialElement->GetWorldPositionOffset().GetOutputIndex());
	}

	// Connect expressions to any UMaterialExpressionCustomOutput since these aren't part of the predefined material outputs
	for ( int32 ExpressionIndex = 0; ExpressionIndex < MaterialElement->GetExpressionsCount(); ++ExpressionIndex )
	{
		if ( IDatasmithMaterialExpression* DatasmithExpression = MaterialElement->GetExpression( ExpressionIndex ) )
		{
			if ( UMaterialExpressionCustomOutput* MaterialOutputExpression = Cast< UMaterialExpressionCustomOutput >( MaterialExpressions[ ExpressionIndex ].Get() ) )
			{
				for ( int32 ExpressionInput = 0; ExpressionInput < DatasmithExpression->GetInputCount(); ++ExpressionInput )
				{
					if (FExpressionInput* MaterialOutputExpressionInput = MaterialOutputExpression->GetInput( ExpressionInput ) )
					{
						ConnectExpression(
							MaterialElement.ToSharedRef(),
							MaterialExpressions,
							DatasmithExpression->GetInput( ExpressionInput )->GetExpression(),
							MaterialOutputExpressionInput,
							DatasmithExpression->GetInput( ExpressionInput )->GetOutputIndex()
						);
					}
				}
			}
		}
	}
}

UMaterialFunction* FDatasmithMaterialExpressions::CreateUEPbrMaterialFunction(UPackage* Package, const TSharedPtr< IDatasmithUEPbrMaterialElement >& MaterialElement, FDatasmithAssetsImportContext& AssetsContext, UMaterial* ExistingMaterial, EObjectFlags ObjectFlags)
{
	FString MaterialFunctionName = GenerateUniqueMaterialName(MaterialElement->GetParentLabel(), Package, AssetsContext.MaterialFunctionNameProvider);

	FText FailReason;
	if (!FDatasmithImporterUtils::CanCreateAsset<UMaterialFunction>(Package, MaterialFunctionName, FailReason))
	{
		AssetsContext.GetParentContext().LogError(FailReason);
		return nullptr;
	}

	UMaterialFunction* UnrealMaterialFunc = NewObject<UMaterialFunction>(Package, FName(*MaterialFunctionName), ObjectFlags);

	CreateUEPbrMaterialGraph(MaterialElement, AssetsContext, UnrealMaterialFunc);

	UMaterialEditingLibrary::LayoutMaterialFunctionExpressions(UnrealMaterialFunc);

	UnrealMaterialFunc->UpdateDependentFunctionCandidates();

	return UnrealMaterialFunc;
}

EBlendMode FDatasmithMaterialExpressions::GetUEPbrImportBlendMode(const TSharedPtr< IDatasmithUEPbrMaterialElement >& MaterialElement, const FDatasmithAssetsImportContext& AssetsContext)
{
	EBlendMode BlendMode = static_cast<EBlendMode>(MaterialElement->GetBlendMode());

	if ( MaterialElement->GetOpacity().GetExpression() &&
		BlendMode != BLEND_Translucent && BlendMode != BLEND_Masked )
	{
		return EBlendMode::BLEND_Translucent; // force translucent
	}

	// If this material is dependent on function call expression, we need to check if those are translucent to apply
	// translucency attribute to this material too.
	for (int32 ExpressionIndex = 0; ExpressionIndex < MaterialElement->GetExpressionsCount(); ExpressionIndex++)
	{
		IDatasmithMaterialExpression* MaterialExpression = MaterialElement->GetExpression(ExpressionIndex);
		if (MaterialExpression && MaterialExpression->IsSubType(EDatasmithMaterialExpressionType::FunctionCall))
		{
			IDatasmithMaterialExpressionFunctionCall* MaterialExpressionFunctionCall = static_cast< IDatasmithMaterialExpressionFunctionCall* >(MaterialExpression);

			const TSharedRef<IDatasmithBaseMaterialElement>* MaterialFunction = FDatasmithFindAssetTypeHelper<UMaterialFunction>::GetImportedElementByName(AssetsContext, MaterialExpressionFunctionCall->GetFunctionPathName());
			if (MaterialFunction && (*MaterialFunction)->IsA(EDatasmithElementType::UEPbrMaterial))
			{
				const TSharedRef<IDatasmithUEPbrMaterialElement> MaterialFunctionElement = StaticCastSharedRef<IDatasmithUEPbrMaterialElement>(*MaterialFunction);
				if (GetUEPbrImportBlendMode(MaterialFunctionElement, AssetsContext) == EBlendMode::BLEND_Translucent)
				{
					return EBlendMode::BLEND_Translucent;
				}
			}
		}
	}

	return BlendMode;
}

UMaterialInterface* FDatasmithMaterialExpressions::CreateUEPbrMaterial(UPackage* Package, const TSharedPtr< IDatasmithUEPbrMaterialElement >& MaterialElement, FDatasmithAssetsImportContext& AssetsContext,
	UMaterial* ExistingMaterial, EObjectFlags ObjectFlags)
{
	FString MaterialName = GenerateUniqueMaterialName(MaterialElement->GetParentLabel(), Package, AssetsContext.MaterialInstanceNameProvider);

	// Verify that the material could be created in final package
	FText FailReason;
	if (!FDatasmithImporterUtils::CanCreateAsset<UMaterial>(Package, MaterialName, FailReason))
	{
		AssetsContext.GetParentContext().LogError(FailReason);
		return nullptr;
	}

	UMaterialFactoryNew* MatFact = NewObject<UMaterialFactoryNew>();
	MatFact->AddToRoot();

	UMaterial* UnrealMaterial = (UMaterial*)MatFact->FactoryCreateNew(UMaterial::StaticClass(), Package, *MaterialName, ObjectFlags, nullptr, GWarn);

	// Hack - work around for the issue of the fact that the ShaderModels is not initialized
	UnrealMaterial->SetShadingModel( MSM_DefaultLit );

	FAssetRegistryModule::AssetCreated(UnrealMaterial);
	MatFact->RemoveFromRoot();

	UnrealMaterial->bUseMaterialAttributes = MaterialElement->GetUseMaterialAttributes();
	UnrealMaterial->TwoSided = MaterialElement->GetTwoSided();
	UnrealMaterial->bIsThinSurface = MaterialElement->GetIsThinSurface();
	UnrealMaterial->BlendMode = GetUEPbrImportBlendMode(MaterialElement, AssetsContext);
	UnrealMaterial->OpacityMaskClipValue = MaterialElement->GetOpacityMaskClipValue();
	UnrealMaterial->TranslucencyLightingMode = static_cast<ETranslucencyLightingMode>(MaterialElement->GetTranslucencyLightingMode());

	CreateUEPbrMaterialGraph(MaterialElement, AssetsContext, UnrealMaterial);

	EDatasmithShadingModel MaterialShadingModel = MaterialElement->GetShadingModel();
	switch (MaterialShadingModel)
	{
		case EDatasmithShadingModel::Subsurface:
			UnrealMaterial->SetShadingModel(MSM_Subsurface);
			break;

		case EDatasmithShadingModel::ClearCoat:
			UnrealMaterial->SetShadingModel(MSM_ClearCoat);
			break;

		case EDatasmithShadingModel::ThinTranslucent:
			UnrealMaterial->SetShadingModel(MSM_ThinTranslucent);
			break;
		case EDatasmithShadingModel::Unlit:
			UnrealMaterial->SetShadingModel(MSM_Unlit);
			break;
		default:
			break;
	}

	if ( MaterialElement->GetOpacity().GetExpression() )
	{
		if ( MaterialElement->GetShadingModel() == EDatasmithShadingModel::ThinTranslucent )
		{
			UnrealMaterial->TranslucencyLightingMode = TLM_SurfacePerPixelLighting;
		}
		else
		{
			UnrealMaterial->TranslucencyLightingMode = ETranslucencyLightingMode::TLM_Surface;
		}
	}

	UnrealMaterial->UpdateCachedExpressionData();

	UMaterialEditingLibrary::LayoutMaterialExpressions( UnrealMaterial );
	UDatasmithAssetUserData::SetDatasmithUserDataValueForKey( UnrealMaterial, UDatasmithAssetUserData::UniqueIdMetaDataKey, MaterialElement->GetName() );

	return UnrealMaterial;
}

UMaterialInterface* FDatasmithMaterialExpressions::CreateUEPbrMaterialInstance(UPackage* Package, const TSharedPtr< IDatasmithUEPbrMaterialElement >& MaterialElement, FDatasmithAssetsImportContext& AssetsContext,
	UMaterialInterface* ParentMaterial, EObjectFlags ObjectFlags)
{
	FString MaterialName = GenerateUniqueMaterialName(MaterialElement->GetLabel(), Package, AssetsContext.MaterialNameProvider);

	// Verify that the material could be created in final package
	FText FailReason;
	if (!FDatasmithImporterUtils::CanCreateAsset<UMaterialInstanceConstant>(AssetsContext.MaterialsFinalPackage.Get(), MaterialName, FailReason))
	{
		AssetsContext.GetParentContext().LogError(FailReason);
		return nullptr;
	}

	UMaterialInstanceConstant* MaterialInstance = nullptr;

	UObject* ExistingObject = FDatasmithImporterUtils::FindObject< UObject >( Package, MaterialName );
	UMaterialInstanceConstant* ExistingMaterialInstance = Cast< UMaterialInstanceConstant >( ExistingObject );

	if ( ExistingObject && !ExistingMaterialInstance )
	{
		// TODO: Log error message
		return nullptr;
	}

	if ( !ExistingMaterialInstance )
	{
		TStrongObjectPtr< UMaterialInstanceConstantFactoryNew > MaterialFact( NewObject< UMaterialInstanceConstantFactoryNew >() );
		MaterialFact->InitialParent = ParentMaterial;

		MaterialInstance = Cast< UMaterialInstanceConstant >( MaterialFact->FactoryCreateNew( UMaterialInstanceConstant::StaticClass(), Package, FName(*MaterialName), ObjectFlags, nullptr, GWarn ) );

		FAssetRegistryModule::AssetCreated( MaterialInstance );
	}
	else
	{
		MaterialInstance = ExistingMaterialInstance;
	}

	UDatasmithMaterialInstanceTemplate* MaterialInstanceTemplate = NewObject< UDatasmithMaterialInstanceTemplate >( MaterialInstance );

	MaterialInstanceTemplate->ParentMaterial = ParentMaterial;

	for ( int32 ExpressionIndex = 0; ExpressionIndex < MaterialElement->GetExpressionsCount(); ++ExpressionIndex )
	{
		IDatasmithMaterialExpression* DatasmithExpression = MaterialElement->GetExpression( ExpressionIndex );

		if ( DatasmithExpression->IsSubType( EDatasmithMaterialExpressionType::Texture ) )
		{
			FName ParameterName = GenerateParamName<EDatasmithMaterialExpressionType::Texture>( *DatasmithExpression, ExpressionIndex + 1 );

			IDatasmithMaterialExpressionTexture* TextureExpression = static_cast< IDatasmithMaterialExpressionTexture* >( DatasmithExpression );
			UTexture* Texture = FDatasmithImporterUtils::FindAsset< UTexture >( AssetsContext, TextureExpression->GetTexturePathName() );
			MaterialInstanceTemplate->TextureParameterValues.Add( ParameterName, Texture );

			UTexture2D* Texture2D = Cast<UTexture2D>(Texture);
			UTexture* DefaultTexture = nullptr;
			if (Texture2D && MaterialInstance->GetMaterial()->GetTextureParameterDefaultValue(ParameterName, DefaultTexture) &&
				!Texture2D->VirtualTextureStreaming != !DefaultTexture->VirtualTextureStreaming)
			{
				if (Texture2D->VirtualTextureStreaming)
				{
					//If we are adding a virtual texture to the material instance and the parent material takes a standard texture, we need to convert that texture to a standard texture.
					AssetsContext.VirtualTexturesToConvert.Add(Texture2D);
				}
				else if(UTexture2D* Default2DTexture = Cast<UTexture2D>(DefaultTexture))
				{
					//We are adding a standard texture to a reference material that takes virtual textures, we'll need to convert the reference's material texture to standard.
					AssetsContext.VirtualTexturesToConvert.Add(Default2DTexture);
				}
			}
		}
		else if ( DatasmithExpression->IsSubType( EDatasmithMaterialExpressionType::ConstantBool ) )
		{
			FName ParameterName = GenerateParamName<EDatasmithMaterialExpressionType::ConstantBool>( *DatasmithExpression, ExpressionIndex + 1 );

			IDatasmithMaterialExpressionBool* BoolExpression = static_cast< IDatasmithMaterialExpressionBool* >( DatasmithExpression );

			MaterialInstanceTemplate->StaticParameters.StaticSwitchParameters.Add( ParameterName ) = BoolExpression->GetBool();
		}
		else if ( DatasmithExpression->IsSubType( EDatasmithMaterialExpressionType::ConstantColor ) )
		{
			FName ParameterName = GenerateParamName<EDatasmithMaterialExpressionType::ConstantColor>( *DatasmithExpression, ExpressionIndex + 1 );

			IDatasmithMaterialExpressionColor* ColorExpression = static_cast< IDatasmithMaterialExpressionColor* >( DatasmithExpression );

			MaterialInstanceTemplate->VectorParameterValues.Add( ParameterName ) = ColorExpression->GetColor();
		}
		else if ( DatasmithExpression->IsSubType( EDatasmithMaterialExpressionType::ConstantScalar ) )
		{
			FName ParameterName = GenerateParamName<EDatasmithMaterialExpressionType::ConstantScalar>( *DatasmithExpression, ExpressionIndex + 1 );

			IDatasmithMaterialExpressionScalar* ScalarExpression = static_cast< IDatasmithMaterialExpressionScalar* >( DatasmithExpression );

			MaterialInstanceTemplate->ScalarParameterValues.Add( ParameterName ) = ScalarExpression->GetScalar();
		}
		else if ( DatasmithExpression->IsSubType( EDatasmithMaterialExpressionType::Generic ) )
		{
			IDatasmithMaterialExpressionGeneric* GenericExpression = static_cast< IDatasmithMaterialExpressionGeneric* >( DatasmithExpression );

			// Instancing is only supported for generic expressions with a single value so that there's no ambiguity on which is the parameter
			if ( GenericExpression->GetPropertiesCount() == 1 )
			{
				FName ParameterName = GenerateParamName<EDatasmithMaterialExpressionType::Generic>( *DatasmithExpression, ExpressionIndex + 1 );

				const TSharedPtr< IDatasmithKeyValueProperty >& KeyValueProperty = GenericExpression->GetProperty( 0 );

				if ( KeyValueProperty )
				{
					switch ( KeyValueProperty->GetPropertyType() )
					{
					case EDatasmithKeyValuePropertyType::Bool:
						{
							bool bValue;
							LexFromString( bValue, KeyValueProperty->GetValue() );

							MaterialInstanceTemplate->StaticParameters.StaticSwitchParameters.Add( ParameterName ) = bValue;
						}
						break;
					case EDatasmithKeyValuePropertyType::Color:
						{
							FLinearColor ColorValue;
							ColorValue.InitFromString( KeyValueProperty->GetValue() );

							MaterialInstanceTemplate->VectorParameterValues.Add( ParameterName ) = ColorValue;
						}
						break;
					case EDatasmithKeyValuePropertyType::Float:
						{
							float Value;
							LexFromString( Value, KeyValueProperty->GetValue() );

							MaterialInstanceTemplate->ScalarParameterValues.Add( ParameterName ) = Value;
						}
						break;
					case EDatasmithKeyValuePropertyType::Texture:
						{
							UTexture* Texture = FDatasmithImporterUtils::FindAsset< UTexture >( AssetsContext, KeyValueProperty->GetValue() );
							MaterialInstanceTemplate->TextureParameterValues.Add( ParameterName, Texture );

							UTexture2D* Texture2D = Cast<UTexture2D>(Texture);
							UTexture* DefaultTexture = nullptr;
							if (Texture2D && MaterialInstance->GetMaterial()->GetTextureParameterDefaultValue(ParameterName, DefaultTexture) &&
								!Texture2D->VirtualTextureStreaming != !DefaultTexture->VirtualTextureStreaming)
							{
								if (Texture2D->VirtualTextureStreaming)
								{
									//If we are adding a virtual texture to the material instance and the parent material takes a standard texture, we need to convert that texture to a standard texture.
									AssetsContext.VirtualTexturesToConvert.Add(Texture2D);
								}
								else if (UTexture2D* Default2DTexture = Cast<UTexture2D>(DefaultTexture))
								{
									//We are adding a standard texture to a reference material that takes virtual textures, we'll need to convert the reference's material texture to standard.
									AssetsContext.VirtualTexturesToConvert.Add(Default2DTexture);
								}
							}
						}
						break;
					}
				}
			}
		}
	}

	MaterialInstanceTemplate->Apply( MaterialInstance );

	return MaterialInstance;
}