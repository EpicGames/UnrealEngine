// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithMaterialImporter.h"

#include "DatasmithImporterModule.h"
#include "DatasmithMaterialElements.h"
#include "DatasmithMaterialExpressions.h"

#include "ReferenceMaterials/DatasmithReferenceMaterial.h"
#include "ReferenceMaterials/DatasmithReferenceMaterialManager.h"
#include "ReferenceMaterials/DatasmithReferenceMaterialSelector.h"
#include "ObjectTemplates/DatasmithMaterialInstanceTemplate.h"
#include "Utility/DatasmithImporterUtils.h"

#include "AssetRegistry/AssetRegistryModule.h"

#include "Engine/Texture.h"
#include "Engine/Texture2D.h"

#include "Materials/Material.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialInstanceConstant.h"

#include "ObjectTools.h"

#define LOCTEXT_NAMESPACE "DatasmithMaterialImporter"

namespace DatasmithMaterialImporterUtils
{
	int32 ComputeMaterialExpressionHash( IDatasmithMaterialExpression* MaterialExpression, TSet<IDatasmithMaterialExpression*>& VisitedExpressions );

	int32 ComputeExpressionInputHash( IDatasmithExpressionInput* ExpressionInput, TSet<IDatasmithMaterialExpression*>& VisitedExpressions )
	{
		if ( !ExpressionInput )
		{
			return 0;
		}

		uint32 Hash = 0;

		if ( ExpressionInput->GetExpression() )
		{
			int32 ExpressionHash = ComputeMaterialExpressionHash( ExpressionInput->GetExpression(), VisitedExpressions );
			Hash = HashCombine( Hash, ExpressionHash );
		}

		Hash = HashCombine( Hash, GetTypeHash( ExpressionInput->GetOutputIndex() ) );

		return Hash;
	}

	int32 ComputeMaterialExpressionHash( IDatasmithMaterialExpression* MaterialExpression, TSet<IDatasmithMaterialExpression*>& VisitedExpressions )
	{
		VisitedExpressions.Add(MaterialExpression);
		uint32 Hash = GetTypeHash( MaterialExpression->GetExpressionType() );
		Hash = HashCombine( Hash, GetTypeHash( FString( MaterialExpression->GetName() ) ) );

		if ( MaterialExpression->IsSubType( EDatasmithMaterialExpressionType::TextureCoordinate ) )
		{
			IDatasmithMaterialExpressionTextureCoordinate* TextureCoordinate = static_cast< IDatasmithMaterialExpressionTextureCoordinate* >( MaterialExpression );
			Hash = HashCombine( Hash, GetTypeHash( TextureCoordinate->GetCoordinateIndex() ) );
			Hash = HashCombine( Hash, GetTypeHash( TextureCoordinate->GetUTiling() ) );
			Hash = HashCombine( Hash, GetTypeHash( TextureCoordinate->GetVTiling() ) );
		}
		else if ( MaterialExpression->IsSubType( EDatasmithMaterialExpressionType::ConstantColor ) )
		{
			if ( FCString::Strlen( MaterialExpression->GetName() ) == 0 )
			{
				IDatasmithMaterialExpressionColor* ColorExpression = static_cast< IDatasmithMaterialExpressionColor* >( MaterialExpression );

				Hash = HashCombine( Hash, GetTypeHash( ColorExpression->GetColor() ) );
			}
		}
		else if ( MaterialExpression->IsSubType( EDatasmithMaterialExpressionType::ConstantScalar ) )
		{
			if ( FCString::Strlen( MaterialExpression->GetName() ) == 0 )
			{
				IDatasmithMaterialExpressionScalar* ScalarExpression = static_cast< IDatasmithMaterialExpressionScalar* >( MaterialExpression );

				Hash = HashCombine( Hash, GetTypeHash( ScalarExpression->GetScalar() ) );
			}
		}
		else if ( MaterialExpression->IsSubType( EDatasmithMaterialExpressionType::Generic ) )
		{
			IDatasmithMaterialExpressionGeneric* GenericExpression = static_cast< IDatasmithMaterialExpressionGeneric* >( MaterialExpression );

			UClass* ExpressionClass = FindFirstObject<UClass>(*(FString(TEXT("MaterialExpression")) + GenericExpression->GetExpressionName()), EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("ComputeMaterialExpressionHash"));

			UMaterialExpression* MaterialCDO = nullptr;

			if ( ExpressionClass )
			{
				MaterialCDO = ExpressionClass->GetDefaultObject< UMaterialExpression >();
			}

			for ( int32 PropertyIndex = 0; PropertyIndex < GenericExpression->GetPropertiesCount(); ++PropertyIndex )
			{
				const TSharedPtr< IDatasmithKeyValueProperty >& KeyValue = GenericExpression->GetProperty( PropertyIndex );

				if ( KeyValue )
				{
					Hash = HashCombine( Hash, FCrc::Strihash_DEPRECATED( KeyValue->GetName() ) );
					Hash = HashCombine( Hash, GetTypeHash( KeyValue->GetPropertyType() ) );

					// Only hash values if it's not the parameter
					// Currently, if we're setting values on multiple properties, we're not sure which one is the parameter so we hash them all
					if ( MaterialCDO && ( !MaterialCDO->HasAParameterName() || GenericExpression->GetPropertiesCount() > 1 ) )
					{
						Hash = HashCombine( Hash, FCrc::Strihash_DEPRECATED( KeyValue->GetValue() ) );
					}
				}
			}
		}
		else if ( MaterialExpression->IsSubType( EDatasmithMaterialExpressionType::FunctionCall ) )
		{
			// Hash the path to the function as calling different functions should result in different hash values
			IDatasmithMaterialExpressionFunctionCall* FunctionCallExpression = static_cast< IDatasmithMaterialExpressionFunctionCall* >( MaterialExpression );

			Hash = HashCombine( Hash, FCrc::Strihash_DEPRECATED( FunctionCallExpression->GetFunctionPathName() ) );
		}

		for ( int32 InputIndex = 0; InputIndex < MaterialExpression->GetInputCount(); ++InputIndex )
		{
			IDatasmithExpressionInput* DatasmithExpressionInput = MaterialExpression->GetInput( InputIndex );
			if(ensure(!DatasmithExpressionInput || (MaterialExpression != DatasmithExpressionInput->GetExpression()))) // Prevent infinite recursion when an expression has itself as input
			{
				Hash = HashCombine( Hash, ComputeExpressionInputHash( DatasmithExpressionInput, VisitedExpressions ) );
			}
		}

		return Hash;
	}

	int32 ComputeMaterialHash( TSharedRef< IDatasmithUEPbrMaterialElement > MaterialElement )
	{
		TSet<IDatasmithMaterialExpression*> VisitedExpressions;
		int32 Hash = GetTypeHash( MaterialElement->GetTwoSided() );

		Hash = HashCombine( Hash, GetTypeHash( MaterialElement->GetUseMaterialAttributes() ) );
		Hash = HashCombine( Hash, GetTypeHash( MaterialElement->GetBlendMode() ) );
		Hash = HashCombine( Hash, GetTypeHash( MaterialElement->GetIsThinSurface() ) );
		Hash = HashCombine( Hash, GetTypeHash( MaterialElement->GetShadingModel() ) );

		Hash = HashCombine( Hash, ComputeExpressionInputHash( &MaterialElement->GetBaseColor(), VisitedExpressions ) );
		Hash = HashCombine( Hash, ComputeExpressionInputHash( &MaterialElement->GetMetallic(), VisitedExpressions ) );
		Hash = HashCombine( Hash, ComputeExpressionInputHash( &MaterialElement->GetSpecular(), VisitedExpressions ) );
		Hash = HashCombine( Hash, ComputeExpressionInputHash( &MaterialElement->GetRoughness(), VisitedExpressions ) );
		Hash = HashCombine( Hash, ComputeExpressionInputHash( &MaterialElement->GetEmissiveColor(), VisitedExpressions ) );
		Hash = HashCombine( Hash, ComputeExpressionInputHash( &MaterialElement->GetOpacity(), VisitedExpressions ) );
		Hash = HashCombine( Hash, ComputeExpressionInputHash( &MaterialElement->GetNormal(), VisitedExpressions ) );
		Hash = HashCombine( Hash, ComputeExpressionInputHash( &MaterialElement->GetRefraction(), VisitedExpressions ) );
		Hash = HashCombine( Hash, ComputeExpressionInputHash( &MaterialElement->GetAmbientOcclusion(), VisitedExpressions ) );
		Hash = HashCombine( Hash, ComputeExpressionInputHash( &MaterialElement->GetClearCoat(), VisitedExpressions ) );
		Hash = HashCombine( Hash, ComputeExpressionInputHash( &MaterialElement->GetClearCoatRoughness(), VisitedExpressions ) );
		Hash = HashCombine( Hash, ComputeExpressionInputHash( &MaterialElement->GetWorldPositionOffset(), VisitedExpressions ) );
		Hash = HashCombine( Hash, ComputeExpressionInputHash( &MaterialElement->GetMaterialAttributes(), VisitedExpressions ) );

		TSet<IDatasmithMaterialExpression*> AllExpressions;
		for (int32 ExpressionIndex = 0; ExpressionIndex < MaterialElement->GetExpressionsCount(); ++ExpressionIndex)
		{
			AllExpressions.Add(MaterialElement->GetExpression(ExpressionIndex));
		}

		TSet<IDatasmithMaterialExpression*> NotVisitedExpressions = AllExpressions.Difference(VisitedExpressions);
		NotVisitedExpressions.Remove(nullptr);
		// we don't want to depend on the order of nodes
		NotVisitedExpressions.Sort([](IDatasmithMaterialExpression& A, IDatasmithMaterialExpression& B){ return FCString::Strcmp(A.GetName(), B.GetName()) < 0; });

		for (IDatasmithMaterialExpression* Expression : NotVisitedExpressions)
		{
			TSet<IDatasmithMaterialExpression*> _;
			Hash = HashCombine(Hash, ComputeMaterialExpressionHash(Expression, _));
		}

		return Hash;
	}
}

UMaterialFunction* FDatasmithMaterialImporter::CreateMaterialFunction( FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithBaseMaterialElement >& BaseMaterialElement )
{
	UMaterialFunction* MaterialFunction = nullptr;

	if ( BaseMaterialElement->IsA( EDatasmithElementType::UEPbrMaterial ) )
	{
		const TSharedRef< IDatasmithUEPbrMaterialElement > MaterialElement = StaticCastSharedRef< IDatasmithUEPbrMaterialElement >( BaseMaterialElement );
		UPackage* MaterialPackage = ImportContext.AssetsContext.MaterialFunctionsImportPackage.Get();
		MaterialFunction = FDatasmithMaterialExpressions::CreateUEPbrMaterialFunction( MaterialPackage, MaterialElement, ImportContext.AssetsContext, nullptr, ImportContext.ObjectFlags );
	}
	else
	{
		//Only UEPbr materials should end up here.
		check(false)
		return nullptr;
	}

	if ( MaterialFunction != nullptr )
	{
		ImportContext.ImportedMaterialFunctions.Add( BaseMaterialElement ) = MaterialFunction;
		ImportContext.ImportedMaterialFunctionsByName.Add( BaseMaterialElement->GetName(), BaseMaterialElement );
	}

	return MaterialFunction;
}

UMaterialInterface* FDatasmithMaterialImporter::CreateMaterial( FDatasmithImportContext& ImportContext,
	const TSharedRef< IDatasmithBaseMaterialElement >& BaseMaterialElement, UMaterialInterface* ExistingMaterial )
{
	UMaterialInterface* Material = nullptr;

	if ( BaseMaterialElement->IsA( EDatasmithElementType::Material ) )
	{
		const TSharedRef< IDatasmithMaterialElement >& MaterialElement = StaticCastSharedRef< IDatasmithMaterialElement >( BaseMaterialElement );

		UPackage* MaterialPackage = ImportContext.AssetsContext.MaterialsImportPackage.Get();

		Material = FDatasmithMaterialExpressions::CreateDatasmithMaterial(MaterialPackage, MaterialElement, ImportContext.AssetsContext, nullptr, ImportContext.ObjectFlags);
	}
	else if ( BaseMaterialElement->IsA( EDatasmithElementType::MaterialInstance ) )
	{
		const TSharedRef< IDatasmithMaterialInstanceElement >& ReferenceMaterialElement = StaticCastSharedRef< IDatasmithMaterialInstanceElement >( BaseMaterialElement );
		Material = ImportMaterialInstance( ImportContext, ReferenceMaterialElement, ExistingMaterial );
	}
	else if ( BaseMaterialElement->IsA( EDatasmithElementType::DecalMaterial ) )
	{
		const TSharedRef< IDatasmithDecalMaterialElement >& DecalMaterialElement = StaticCastSharedRef< IDatasmithDecalMaterialElement >( BaseMaterialElement );
		Material = ImportDecalMaterial( ImportContext, DecalMaterialElement, ExistingMaterial );
	}
	else if ( BaseMaterialElement->IsA( EDatasmithElementType::UEPbrMaterial ) )
	{
		const TSharedRef< IDatasmithUEPbrMaterialElement > MaterialElement = StaticCastSharedRef< IDatasmithUEPbrMaterialElement >( BaseMaterialElement );
		if ( MaterialElement->GetMaterialFunctionOnly() )
		{
			//No need to instantiate a MaterialElement that is only used as a material function
			return nullptr;
		}

		int32 MaterialHash = DatasmithMaterialImporterUtils::ComputeMaterialHash( MaterialElement );

		if ( !ImportContext.ImportedParentMaterials.Contains( MaterialHash ) )
		{
			UMaterialInterface* ParentMaterial = FDatasmithMaterialExpressions::CreateUEPbrMaterial( ImportContext.AssetsContext.ReferenceMaterialsImportPackage.Get(), MaterialElement, ImportContext.AssetsContext, nullptr, ImportContext.ObjectFlags );

			if (ParentMaterial == nullptr)
			{
				return nullptr;
			}

			ImportContext.ImportedParentMaterials.Add( MaterialHash ) = ParentMaterial;
		}

		// Always create a material instance
		{
			Material = FDatasmithMaterialExpressions::CreateUEPbrMaterialInstance( ImportContext.AssetsContext.MaterialsImportPackage.Get(), MaterialElement, ImportContext.AssetsContext,
				Cast< UMaterial >( ImportContext.ImportedParentMaterials[ MaterialHash ] ), ImportContext.ObjectFlags );
		}
	}

	if (Material != nullptr)
	{
		ImportContext.ImportedMaterials.Add( BaseMaterialElement ) = Material;
	}

	return Material;
}

UMaterialInterface* FDatasmithMaterialImporter::ImportMaterialInstance( FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithMaterialInstanceElement >& MaterialElement, UMaterialInterface* ExistingMaterial )
{
	// Verify existing material is of the right class for further processing
	UMaterialInstanceConstant* FoundConstantMaterial = Cast<UMaterialInstanceConstant>(ExistingMaterial);

	FString Host = FDatasmithReferenceMaterialManager::Get().GetHostFromString(ImportContext.Scene->GetHost());
	TSharedPtr< FDatasmithReferenceMaterialSelector > MaterialSelector = FDatasmithReferenceMaterialManager::Get().GetSelector( *Host );

	if (!MaterialSelector.IsValid())
	{
		FText FailReason = FText::Format(LOCTEXT("NoSelectorForHost", "No Material selector found for Host {0}. Skipping material {1} ..."), FText::FromString(Host), FText::FromString(MaterialElement->GetName()));
		ImportContext.LogError(FailReason);
		return nullptr;
	}

	const FDatasmithReferenceMaterial* ParentMaterial = nullptr;
	FDatasmithReferenceMaterial CustomReferenceMaterial; // ReferenceMaterial might point on this so keep them in the same scope

	if ( MaterialElement->GetMaterialType() == EDatasmithReferenceMaterialType::Custom )
	{
		CustomReferenceMaterial.FromSoftObjectPath( FSoftObjectPath( MaterialElement->GetCustomMaterialPathName() ) );
		if (!CustomReferenceMaterial.IsValid())
		{
			ImportContext.LogError(FText::Format(LOCTEXT("NoReferenceForPath", "No compatible asset for path '{0}'. Skipping material {1} ..."), FText::FromString(MaterialElement->GetCustomMaterialPathName()), FText::FromString(MaterialElement->GetName())));
			return nullptr;
		}

		ParentMaterial = &CustomReferenceMaterial;
	}
	else
	{
		if ( MaterialSelector->IsValid() )
		{
			ParentMaterial = &MaterialSelector->GetReferenceMaterial(MaterialElement);
		}
		else
		{
			FText FailReason = FText::Format(LOCTEXT("NoValidSelectorForHost", "No valid Material selector found for Host {0}. Skipping material {1} ..."), FText::FromString(Host), FText::FromString(MaterialElement->GetName()));
			ImportContext.LogError(FailReason);
			return nullptr;
		}
	}

	if ( ParentMaterial && ParentMaterial->IsValid() )
	{
		UPackage* DestinationPackage = ImportContext.AssetsContext.MaterialsFinalPackage.Get();
		int32 CharBudget = FDatasmithImporterUtils::GetAssetNameMaxCharCount(DestinationPackage);

		const FString MaterialLabel = MaterialElement->GetLabel();
		const TCHAR* NameSource = MaterialLabel.Len() > 0 ? MaterialElement->GetLabel(): MaterialElement->GetName();
		const FString MaterialName = ImportContext.AssetsContext.MaterialNameProvider.GenerateUniqueName(NameSource, CharBudget);

		// Verify that the material could be created in final package
		FText FailReason;
		if (!FDatasmithImporterUtils::CanCreateAsset<UMaterialInstanceConstant>( DestinationPackage, MaterialName, FailReason ))
		{
			ImportContext.LogError(FailReason);
			return nullptr;
		}

		UMaterialInstanceConstant* MaterialInstance = FoundConstantMaterial;

		if (MaterialInstance == nullptr)
		{
			MaterialInstance = NewObject<UMaterialInstanceConstant>(ImportContext.AssetsContext.MaterialsImportPackage.Get(), *MaterialName, ImportContext.ObjectFlags);
			MaterialInstance->Parent = ParentMaterial->GetMaterial();

			FAssetRegistryModule::AssetCreated(MaterialInstance);
		}
		else
		{
			MaterialInstance = DuplicateObject< UMaterialInstanceConstant >(MaterialInstance, ImportContext.AssetsContext.MaterialsImportPackage.Get(), *MaterialName);
			IDatasmithImporterModule::Get().ResetOverrides(MaterialInstance); // Don't copy the existing overrides
		}

		UDatasmithMaterialInstanceTemplate* MaterialInstanceTemplate = NewObject< UDatasmithMaterialInstanceTemplate >( MaterialInstance );

		MaterialInstanceTemplate->ParentMaterial = MaterialInstance->Parent;

		// Find matching reference material parameters
		for (int i = 0; i < MaterialElement->GetPropertiesCount(); ++i)
		{
			const TSharedPtr< IDatasmithKeyValueProperty > Property = MaterialElement->GetProperty(i);
			FString PropertyName(Property->GetName());

			// Vector Params
			if ( ParentMaterial->VectorParams.Contains(PropertyName) )
			{
				FLinearColor Color;
				if ( MaterialSelector->GetColor( Property, Color ) )
				{
					MaterialInstanceTemplate->VectorParameterValues.Add( FName(*PropertyName), Color );
				}
			}
			// Scalar Params
			else if ( ParentMaterial->ScalarParams.Contains(PropertyName) )
			{
				float Value;
				if ( MaterialSelector->GetFloat( Property, Value ) )
				{
					MaterialInstanceTemplate->ScalarParameterValues.Add( FName(*PropertyName), Value );
				}
			}
			// Bool Params
			else if (ParentMaterial->BoolParams.Contains(PropertyName))
			{
				bool bValue;
				if ( MaterialSelector->GetBool( Property, bValue ) )
				{
					MaterialInstanceTemplate->StaticParameters.StaticSwitchParameters.Add( FName( Property->GetName() ), bValue );
				}
			}
			// Texture Params
			else if (ParentMaterial->TextureParams.Contains(PropertyName))
			{
				FString TexturePath;
				if ( MaterialSelector->GetTexture( Property, TexturePath ) )
				{
					FString TextureName = FPackageName::IsValidObjectPath(TexturePath) ? TexturePath
						: ObjectTools::SanitizeObjectName(FPaths::GetBaseFilename( TexturePath ));

					UTexture* Texture = FDatasmithImporterUtils::FindAsset< UTexture >( ImportContext.AssetsContext, *TextureName );
					MaterialInstanceTemplate->TextureParameterValues.Add( FName(*PropertyName), Texture );

					//If we are adding a virtual texture to a non-virtual texture streamer then we will need to convert back that Virtual texture.
					UTexture* DefaultTextureValue = nullptr;
					UTexture2D* Texture2D = Cast<UTexture2D>(Texture);
					if (Texture2D && Texture2D->VirtualTextureStreaming &&
						MaterialInstance->GetTextureParameterDefaultValue(FName(*PropertyName), DefaultTextureValue) && DefaultTextureValue)
					{
						if (!DefaultTextureValue->VirtualTextureStreaming)
						{
							ImportContext.AssetsContext.VirtualTexturesToConvert.Add(Texture2D);
						}
					}
				}
			}
		}

		MaterialInstanceTemplate->Apply( MaterialInstance );

		MaterialSelector->FinalizeMaterialInstance(MaterialElement, MaterialInstance);

		return MaterialInstance;
	}

	return nullptr;
}

UMaterialInterface* FDatasmithMaterialImporter::ImportDecalMaterial( FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithDecalMaterialElement >& MaterialElement, UMaterialInterface* ExistingMaterial )
{
	FDatasmithAssetsImportContext& AssetsContext = ImportContext.AssetsContext;

	// Verify existing material is of the right class for further processing
	UMaterialInstanceConstant* FoundConstantMaterial = Cast<UMaterialInstanceConstant>(ExistingMaterial);

	UMaterial* DecalMaterial = Cast< UMaterial >(FSoftObjectPath("/DatasmithContent/Materials/M_DatasmithDecal.M_DatasmithDecal").TryLoad());

	if ( DecalMaterial )
	{
		UPackage* DestinationPackage = AssetsContext.MaterialsFinalPackage.Get();
		int32 CharBudget = FDatasmithImporterUtils::GetAssetNameMaxCharCount(DestinationPackage);

		const FString MaterialLabel = MaterialElement->GetLabel();
		const TCHAR* NameSource = MaterialLabel.Len() > 0 ? MaterialElement->GetLabel(): MaterialElement->GetName();
		const FString MaterialName = AssetsContext.MaterialNameProvider.GenerateUniqueName(NameSource, CharBudget);

		// Verify that the material could be created in final package
		FText FailReason;
		if (!FDatasmithImporterUtils::CanCreateAsset<UMaterialInstanceConstant>( DestinationPackage, MaterialName, FailReason ))
		{
			ImportContext.LogError(FailReason);
			return nullptr;
		}

		UMaterialInstanceConstant* MaterialInstance = FoundConstantMaterial;

		if (MaterialInstance == nullptr)
		{
			MaterialInstance = NewObject<UMaterialInstanceConstant>(AssetsContext.MaterialsImportPackage.Get(), *MaterialName, ImportContext.ObjectFlags);
			MaterialInstance->Parent = DecalMaterial;

			FAssetRegistryModule::AssetCreated(MaterialInstance);
		}
		else
		{
			MaterialInstance = DuplicateObject< UMaterialInstanceConstant >(MaterialInstance, AssetsContext.MaterialsImportPackage.Get(), *MaterialName);
			IDatasmithImporterModule::Get().ResetOverrides(MaterialInstance); // Don't copy the existing overrides
		}

		UDatasmithMaterialInstanceTemplate* MaterialInstanceTemplate = NewObject< UDatasmithMaterialInstanceTemplate >( MaterialInstance );

		MaterialInstanceTemplate->ParentMaterial = MaterialInstance->Parent;

		TFunction<void(const TCHAR*, const TCHAR*)> ApplyTexture;
		ApplyTexture = [&AssetsContext, &MaterialInstance, &MaterialInstanceTemplate](const TCHAR* PropertyName, const TCHAR* TexturePathName) -> void
		{
			if (UTexture* Texture = FDatasmithImporterUtils::FindAsset< UTexture >(AssetsContext, TexturePathName))
			{
				MaterialInstanceTemplate->TextureParameterValues.Add( FName(PropertyName), Texture );

				//If we are adding a virtual texture to a non-virtual texture streamer then we will need to convert back that Virtual texture.
				UTexture* DefaultTextureValue = nullptr;
				UTexture2D* Texture2D = Cast<UTexture2D>(Texture);
				if (Texture2D && Texture2D->VirtualTextureStreaming &&
					MaterialInstance->GetTextureParameterDefaultValue(FName(PropertyName), DefaultTextureValue) && DefaultTextureValue)
				{
					if (!DefaultTextureValue->VirtualTextureStreaming)
					{
						AssetsContext.VirtualTexturesToConvert.Add(Texture2D);
					}
				}
			}
		};

		ApplyTexture(TEXT("DecalTexture"), MaterialElement->GetDiffuseTexturePathName());
		ApplyTexture(TEXT("NormalTexture"), MaterialElement->GetNormalTexturePathName());

		MaterialInstanceTemplate->Apply( MaterialInstance );

		return MaterialInstance;
	}

	return nullptr;
}

int32 FDatasmithMaterialImporter::GetMaterialRequirements(UMaterialInterface * MaterialInterface)
{
	if (MaterialInterface == nullptr || MaterialInterface->GetMaterial() == nullptr)
	{
		return EMaterialRequirements::RequiresNothing;
	}
	// Currently all Datasmith materials require at least normals and tangents
	// @todo: Adjust initial value and logic based on future materials' requirements
	int32 MaterialRequirement = EMaterialRequirements::RequiresNormals | EMaterialRequirements::RequiresTangents;
	return MaterialRequirement;
}

#undef LOCTEXT_NAMESPACE
