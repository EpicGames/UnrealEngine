// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithPostProcessImporter.h"

#include "DatasmithActorImporter.h"
#include "DatasmithImportContext.h"
#include "IDatasmithSceneElements.h"
#include "ObjectTemplates/DatasmithCineCameraComponentTemplate.h"
#include "ObjectTemplates/DatasmithPostProcessVolumeTemplate.h"

#include "Camera/CameraComponent.h"
#include "Engine/PostProcessVolume.h"

FDatasmithPostProcessSettingsTemplate FDatasmithPostProcessImporter::CopyDatasmithPostProcessToUEPostProcess(const TSharedPtr< IDatasmithPostProcessElement >& Src)
{
	FDatasmithPostProcessSettingsTemplate PostProcessSettingsTemplate;

	if ( !FMath::IsNearlyEqual( Src->GetTemperature(), 6500.f ) )
	{
		PostProcessSettingsTemplate.bOverride_WhiteTemp = true;
		PostProcessSettingsTemplate.WhiteTemp = Src->GetTemperature();
	}

	if ( Src->GetVignette() > 0.f )
	{
		PostProcessSettingsTemplate.bOverride_VignetteIntensity = true;
		PostProcessSettingsTemplate.VignetteIntensity = Src->GetVignette();
	}

	if ( !FMath::IsNearlyEqual( Src->GetSaturation(), 1.f ) )
	{
		PostProcessSettingsTemplate.bOverride_ColorSaturation = true;
		PostProcessSettingsTemplate.ColorSaturation.W = Src->GetSaturation();
	}

	if ( Src->GetCameraISO() > 0.f || Src->GetCameraShutterSpeed() > 0.f || Src->GetDepthOfFieldFstop() > 0.f )
	{
		PostProcessSettingsTemplate.bOverride_AutoExposureMethod = true;
		PostProcessSettingsTemplate.AutoExposureMethod = EAutoExposureMethod::AEM_Manual;

		if ( Src->GetCameraISO() > 0.f )
		{
			PostProcessSettingsTemplate.bOverride_CameraISO = true;
			PostProcessSettingsTemplate.CameraISO = Src->GetCameraISO();
		}

		if ( Src->GetCameraShutterSpeed() > 0.f )
		{
			PostProcessSettingsTemplate.bOverride_CameraShutterSpeed = true;
			PostProcessSettingsTemplate.CameraShutterSpeed = Src->GetCameraShutterSpeed();
		}

		if ( Src->GetDepthOfFieldFstop() > 0.f )
		{
			PostProcessSettingsTemplate.bOverride_DepthOfFieldFstop = true;
			PostProcessSettingsTemplate.DepthOfFieldFstop = Src->GetDepthOfFieldFstop();
		}
	}

	return PostProcessSettingsTemplate;
}

AActor* FDatasmithPostProcessImporter::ImportPostProcessVolume( const TSharedRef< IDatasmithPostProcessVolumeElement >& PostProcessVolumeElement, FDatasmithImportContext& ImportContext, EDatasmithImportActorPolicy ImportActorPolicy )
{
	APostProcessVolume* PostProcessVolume = Cast< APostProcessVolume >( FDatasmithActorImporter::ImportActor( APostProcessVolume::StaticClass(), PostProcessVolumeElement, ImportContext, ImportActorPolicy ) );

	if ( !PostProcessVolume )
	{
		return nullptr;
	}

	UDatasmithPostProcessVolumeTemplate* PostProcessVolumeTemplate = NewObject< UDatasmithPostProcessVolumeTemplate >( PostProcessVolume->GetRootComponent() );
	PostProcessVolumeTemplate->bEnabled = PostProcessVolumeElement->GetEnabled();
	PostProcessVolumeTemplate->bUnbound = PostProcessVolumeElement->GetUnbound();

	PostProcessVolumeTemplate->Settings = CopyDatasmithPostProcessToUEPostProcess( PostProcessVolumeElement->GetSettings() );

	PostProcessVolumeTemplate->Apply( PostProcessVolume );

	return PostProcessVolume;
}
