// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithStaticMeshImporter.h"

#include "DatasmithImportContext.h"
#include "DatasmithImporterModule.h"
#include "DatasmithMaterialImporter.h"
#include "DatasmithMeshUObject.h"
#include "IDatasmithSceneElements.h"
#include "ObjectTemplates/DatasmithStaticMeshTemplate.h"
#include "DatasmithPayload.h"
#include "Utility/DatasmithImporterUtils.h"
#include "Utility/DatasmithMeshHelper.h"

#include "Algo/AnyOf.h"
#include "Async/Async.h"
#include "Engine/StaticMesh.h"
#include "LayoutUV.h"
#include "MeshBuild.h"
#include "MeshUtilities.h"
#include "Misc/FeedbackContext.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "OverlappingCorners.h"
#include "PhysicsEngine/AggregateGeom.h"
#include "PhysicsEngine/BodySetup.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"
#include "Templates/UniquePtr.h"
#include "UObject/Package.h"
#include "UVTools/UVGenerationFlattenMapping.h"
#include "UVTools/UVGenerationUtils.h"

#define LOCTEXT_NAMESPACE "DatasmithStaticMeshImporter"

namespace DatasmithStaticMeshImporterImpl
{
	float Get2DSurface(const FVector4& Dimensions)
	{
		if (Dimensions[0] >= Dimensions[1] && Dimensions[2] >= Dimensions[1])
		{
			return Dimensions[0]*Dimensions[2];
		}
		if (Dimensions[0] >= Dimensions[2] && Dimensions[1] >= Dimensions[2])
		{
			return Dimensions[0]*Dimensions[1];
		}
		return Dimensions[2]*Dimensions[1];
	}

	float CalcBlendWeight(const FVector4& Dimensions, float MaxArea, float Max2DSurface)
	{
		float Current2DSurface = Get2DSurface(Dimensions);
		float Weight = FMath::Sqrt((Dimensions[3] / MaxArea)) + FMath::Sqrt(Current2DSurface / Max2DSurface);
		return Weight;
	}

	int32 GetLightmapSize(float Weight, int32 MinimumSize, int32 MaximumSize)
	{
		Weight = FMath::Clamp(Weight, 0.f, 1.f);

		float WeightedSize = (float(MinimumSize)*(1.0f - Weight)) + Weight*float(MaximumSize);

		int32 LightmapSizeValue = MinimumSize;
		while (WeightedSize > LightmapSizeValue)
		{
			LightmapSizeValue *= 2;
		}

		return FMath::Clamp( LightmapSizeValue, 0, 4096 );
	}

	TMap< int32, FString > GetMaterials( TSharedRef< IDatasmithMeshElement > MeshElement )
	{
		TMap< int32, FString > Materials;
		for ( int i = 0; i < MeshElement->GetMaterialSlotCount(); ++i )
		{
			Materials.Add( MeshElement->GetMaterialSlotAt(i)->GetId(), MeshElement->GetMaterialSlotAt(i)->GetName() );
		}
		return Materials;
	}

	int32 GetStaticMeshBuildRequirements( const FDatasmithAssetsImportContext& AssetsContext, TSharedRef< IDatasmithMeshElement > MeshElement )
	{
		int32 BuildRequirements = EMaterialRequirements::RequiresNormals | EMaterialRequirements::RequiresTangents;;
		for ( const TPair<int32, FString>& Material : GetMaterials( MeshElement ) )
		{
			if ( AssetsContext.MaterialsRequirements.Contains( *Material.Value ) )
			{
				BuildRequirements |= AssetsContext.MaterialsRequirements[ *Material.Value ];
			}
		}
		return BuildRequirements;
	}
}

void FDatasmithStaticMeshImporter::CleanupMeshDescriptions(TArray<FMeshDescription>& MeshDescriptions)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithStaticMeshImporter::CleanupMeshDescriptions)

	for (FMeshDescription& MeshDescription : MeshDescriptions)
	{
		TSet<FPolygonID> PolygonsToDelete;
		FStaticMeshAttributes Attributes(MeshDescription);
		const TVertexAttributesRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
		if (VertexPositions.IsValid())
		{
			for (const FVertexID VertexID : MeshDescription.Vertices().GetElementIDs())
			{
				// Ensure that no vertices contains NaN since it can wreck havoc in other algorithms (i.e. MikkTSpace)
				if (VertexPositions[VertexID].ContainsNaN())
				{
					for (FPolygonID PolygonID : MeshDescription.GetVertexConnectedPolygons(VertexID))
					{
						PolygonsToDelete.Add(PolygonID);
					}
				}
			}
		}

		if (MeshDescription.Polygons().Num() == PolygonsToDelete.Num())
		{
			MeshDescription.Empty();
			continue;
		}

		// Remove UV channels containing UV coordinates with Nan.
		// This crashes the editor on Mac, see UE-106145.
		TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();
		for ( int32 Index = VertexInstanceUVs.GetNumChannels() - 1; Index >= 0 ; --Index)
		{
			for (FVertexInstanceID InstanceID : MeshDescription.VertexInstances().GetElementIDs())
			{
				FVector2f UVCoords = VertexInstanceUVs.Get(InstanceID, Index);
				if (UVCoords.ContainsNaN())
				{
					VertexInstanceUVs.RemoveChannel(Index);
					break;
				}
			}
		}

		// Add at least one UV channel if all incoming ones contained coordinates with NaN
		if (VertexInstanceUVs.GetNumChannels() == 0)
		{
			DatasmithMeshHelper::CreateDefaultUVs(MeshDescription);
		}

		if (PolygonsToDelete.Num() > 0)
		{
			TArray<FEdgeID> OrphanedEdges;
			TArray<FVertexInstanceID> OrphanedVertexInstances;
			TArray<FPolygonGroupID> OrphanedPolygonGroups;
			TArray<FVertexID> OrphanedVertices;
			for (FPolygonID PolygonID : PolygonsToDelete)
			{
				MeshDescription.DeletePolygon(PolygonID, &OrphanedEdges, &OrphanedVertexInstances, &OrphanedPolygonGroups);
			}
			for (FPolygonGroupID PolygonGroupID : OrphanedPolygonGroups)
			{
				MeshDescription.DeletePolygonGroup(PolygonGroupID);
			}
			for (FVertexInstanceID VertexInstanceID : OrphanedVertexInstances)
			{
				MeshDescription.DeleteVertexInstance(VertexInstanceID, &OrphanedVertices);
			}
			for (FEdgeID EdgeID : OrphanedEdges)
			{
				MeshDescription.DeleteEdge(EdgeID, &OrphanedVertices);
			}
			for (FVertexID VertexID : OrphanedVertices)
			{
				MeshDescription.DeleteVertex(VertexID);
			}

			FElementIDRemappings Remappings;
			MeshDescription.Compact(Remappings);
		}

		if (PolygonsToDelete.Num() > 0 || MeshDescription.Triangles().Num() == 0)
		{
			MeshDescription.TriangulateMesh();
			if (MeshDescription.Triangles().Num() == 0)
			{
				MeshDescription.Empty();
				continue;
			}
		}

		// Fix invalid vertex normals and tangents
		// We need polygon info because ComputeTangentsAndNormals uses it to repair the invalid vertex normals/tangents
		// Can't calculate just the required polygons as ComputeTangentsAndNormals is parallel and we can't guarantee thread-safe access patterns
		FStaticMeshOperations::ComputeTriangleTangentsAndNormals(MeshDescription);
		FStaticMeshOperations::ComputeTangentsAndNormals(MeshDescription, EComputeNTBsFlags::UseMikkTSpace);
	}
}

UStaticMesh* FDatasmithStaticMeshImporter::ImportStaticMesh(
	const TSharedRef< IDatasmithMeshElement > MeshElement
	, FDatasmithMeshElementPayload& Payload
	, EObjectFlags ObjectFlags
	, const FDatasmithStaticMeshImportOptions& ImportOptions
	, FDatasmithAssetsImportContext& AssetsContext
	, UStaticMesh* ExistingMesh)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithStaticMeshImporter::ImportStaticMesh);

	// 1. import data
	TArray< FMeshDescription >& MeshDescriptions = Payload.LodMeshes;

	// Update MeshElement's LOD count with the actual number of mesh descriptions
	MeshElement->SetLODCount( MeshDescriptions.Num() );
	if ( MeshDescriptions.Num() == 0 )
	{
		return nullptr;
	}

	// Destructive cleanup of the mesh descriptions to avoid passing invalid data to the rest of the editor
	CleanupMeshDescriptions(MeshDescriptions);

	// 2. find the destination
	UStaticMesh* ResultStaticMesh = nullptr;
	int32 MaxNameCharCount = FDatasmithImporterUtils::GetAssetNameMaxCharCount(AssetsContext.StaticMeshesFinalPackage.Get());
	FString StaticMeshName = AssetsContext.StaticMeshNameProvider.GenerateUniqueName( MeshElement->GetLabel(), MaxNameCharCount );

	// Verify that the static mesh could be created in final package
	FText FailReason;
	if (!FDatasmithImporterUtils::CanCreateAsset<UStaticMesh>( AssetsContext.StaticMeshesFinalPackage.Get(), StaticMeshName, FailReason ))
	{
		AssetsContext.GetParentContext().LogError(FailReason);
		return nullptr;
	}

	UPackage* Outer = AssetsContext.StaticMeshesImportPackage.Get();
	if ( ExistingMesh )
	{
		if ( ExistingMesh->GetOuter() != Outer )
		{
			// We don't need to copy over the mesh BulkData as it is going to be recreated anyway, this also prevent ExistingMesh from being invalidated.
			const bool bIgnoreBulkData = true;
			ResultStaticMesh = FDatasmithImporterUtils::DuplicateStaticMesh( ExistingMesh, Outer, *StaticMeshName, bIgnoreBulkData);
			IDatasmithImporterModule::Get().ResetOverrides( ResultStaticMesh ); // Don't copy the existing overrides
		}
		else
		{
			ResultStaticMesh = ExistingMesh;
		}

		ResultStaticMesh->SetFlags( ObjectFlags );
	}
	else
	{
		ResultStaticMesh = NewObject< UStaticMesh >( Outer, *StaticMeshName, ObjectFlags );
	}

	// 3. Write data to destination
	int32 LODIndex = 0;
	for (FMeshDescription& MeshDesc : MeshDescriptions)
	{
		DatasmithMeshHelper::FillUStaticMesh(ResultStaticMesh, LODIndex, MoveTemp(MeshDesc));
		LODIndex++;
	}

	// 4. Collisions
	TArray< FVector3f > VertexPositions;
	DatasmithMeshHelper::ExtractVertexPositions(Payload.CollisionMesh, VertexPositions);
	if ( VertexPositions.Num() == 0 )
	{
		VertexPositions = MoveTemp(Payload.CollisionPointCloud);
	}

	if ( VertexPositions.Num() > 0 )
	{
		ProcessCollision( ResultStaticMesh, VertexPositions );
	}

	return ResultStaticMesh;
}

bool FDatasmithStaticMeshImporter::ShouldRecomputeNormals(const FMeshDescription& MeshDescription, int32 BuildRequirements)
{
	TVertexInstanceAttributesConstRef<FVector3f> Normals = FStaticMeshConstAttributes(MeshDescription).GetVertexInstanceNormals();
	check(Normals.IsValid());
	return Algo::AnyOf(MeshDescription.VertexInstances().GetElementIDs(), [&](const FVertexInstanceID& InstanceID) { return !Normals[InstanceID].IsNormalized(); }, Algo::NoRef);
}

bool FDatasmithStaticMeshImporter::ShouldRecomputeTangents(const FMeshDescription& MeshDescription, int32 BuildRequirements)
{
	TVertexInstanceAttributesConstRef<FVector3f> Tangents = FStaticMeshConstAttributes(MeshDescription).GetVertexInstanceTangents();
	check(Tangents.IsValid());
	return Algo::AnyOf(MeshDescription.VertexInstances().GetElementIDs(), [&](const FVertexInstanceID& InstanceID) { return !Tangents[InstanceID].IsNormalized(); }, Algo::NoRef);
}

bool FDatasmithStaticMeshImporter::PreBuildStaticMesh( UStaticMesh* StaticMesh )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithStaticMeshImporter::PreBuildStaticMesh);

	// The input static mesh must at least have its LOD 0 valid
	if (!StaticMesh->IsMeshDescriptionValid(0))
	{
		return false;
	}

	for (int32 LodIndex = 0; LodIndex < StaticMesh->GetNumSourceModels(); ++LodIndex)
	{
		if (!StaticMesh->IsMeshDescriptionValid(LodIndex))
		{
			continue;
		}

		FMeshBuildSettings& BuildSettings = StaticMesh->GetSourceModel(LodIndex).BuildSettings;
		FMeshDescription& MeshDescription = *StaticMesh->GetMeshDescription(LodIndex);

		if (BuildSettings.bGenerateLightmapUVs && !DatasmithMeshHelper::HasUVData(MeshDescription, BuildSettings.SrcLightmapIndex))
		{
			//If no UV data exist at the source index we generate unwrapped UVs.
			//Do this before calling DatasmithMeshHelper::CreateDefaultUVs() as the UVs may be unwrapped at channel 0.
			UUVGenerationFlattenMapping::GenerateUVs(MeshDescription, BuildSettings.SrcLightmapIndex, BuildSettings.bRemoveDegenerates);
		}

		// We should always have some UV data in channel 0 because it is used in the mesh tangent calculation during the build.
		if (!DatasmithMeshHelper::HasUVData(MeshDescription, 0))
		{
			DatasmithMeshHelper::CreateDefaultUVs(MeshDescription);
		}

		if (DatasmithMeshHelper::IsMeshValid(MeshDescription, FVector3f(BuildSettings.BuildScale3D)))
		{
			if (BuildSettings.bGenerateLightmapUVs)
			{
				DatasmithMeshHelper::RequireUVChannel(MeshDescription, BuildSettings.DstLightmapIndex);
				UVGenerationUtils::SetupGeneratedLightmapUVResolution(StaticMesh, LodIndex);
			}

			UStaticMesh::FCommitMeshDescriptionParams Params;

			// We will call MarkPackageDirty() ourself from the main thread
			Params.bMarkPackageDirty = false;

			// Ensure we get DDC benefits when we import already cached content
			Params.bUseHashAsGuid = true;

			StaticMesh->CommitMeshDescription(LodIndex, Params);

			// Get rid of the memory used now that its committed into its bulk form.
			// Will be reloaded from bulk data when building the mesh if not present in memory.
			StaticMesh->ClearMeshDescription(LodIndex);
		}
		else
		{
			return false;
		}
	}

	return true;
}

void FDatasmithStaticMeshImporter::BuildStaticMesh( UStaticMesh* StaticMesh )
{
	BuildStaticMeshes({ StaticMesh });
}

void FDatasmithStaticMeshImporter::BuildStaticMeshes(const TArray< UStaticMesh* >& StaticMeshes, TFunction<bool(UStaticMesh*)> ProgressCallback)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithStaticMeshImporter::BuildStaticMeshes);

	UStaticMesh::BatchBuild(StaticMeshes, true, ProgressCallback);
}

void FDatasmithStaticMeshImporter::ProcessCollision(UStaticMesh* StaticMesh, const TArray< FVector3f >& VertexPositions)
{
	// The following code is copied from StaticMeshEdit AddConvexGeomFromVertices (inaccessible outside UnrealEd)
	if ( !StaticMesh )
	{
		return;
	}

	StaticMesh->SetCustomizedCollision(true);
	StaticMesh->CreateBodySetup();
	if ( !ensure(StaticMesh->GetBodySetup()) )
	{
		return;
	}

	// Convex elements must be removed first since the re-import process uses the same flow
	FKAggregateGeom& AggGeom = StaticMesh->GetBodySetup()->AggGeom;
	AggGeom.ConvexElems.Reset();
	FKConvexElem& ConvexElem = AggGeom.ConvexElems.AddDefaulted_GetRef();

	ConvexElem.VertexData.Reserve( VertexPositions.Num() );
	for ( const FVector3f& Position : VertexPositions )
	{
		ConvexElem.VertexData.Add( FVector(Position) );
	}

	ConvexElem.UpdateElemBox();
}

void FDatasmithStaticMeshImporter::PreBuildStaticMeshes( FDatasmithImportContext& ImportContext )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithStaticMeshImporter::PreBuildStaticMeshes);

	TUniquePtr<FScopedSlowTask> ProgressPtr;
	if ( ImportContext.FeedbackContext )
	{
		ProgressPtr = MakeUnique<FScopedSlowTask>(ImportContext.ImportedStaticMeshes.Num(), LOCTEXT("PreBuildStaticMeshes", "Setting up UVs..."), true, *ImportContext.FeedbackContext);
		ProgressPtr->MakeDialog(true);
	}

	IMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked< IMeshUtilities >( "MeshUtilities" );

	TArray<UStaticMesh*> SortedMesh;
	for (const auto& Kvp : ImportContext.ImportedStaticMeshes)
	{
		if (Kvp.Value)
		{
			SortedMesh.Add(Kvp.Value);
		}
	}

	// Start with the biggest mesh first to help balancing tasks on threads
	SortedMesh.Sort(
		[](const UStaticMesh& Lhs, const UStaticMesh& Rhs)
		{
			int32 LhsVerticesNum = Lhs.IsMeshDescriptionValid(0) ? Lhs.GetMeshDescription(0)->Vertices().Num() : 0;
			int32 RhsVerticesNum = Rhs.IsMeshDescriptionValid(0) ? Rhs.GetMeshDescription(0)->Vertices().Num() : 0;

			return LhsVerticesNum > RhsVerticesNum;
		}
	);

	// Setup tasks in such a way that we're able to update the progress for tasks finishing in any order
	TAtomic<uint32> TasksDone(0);
	TArray<TFuture<bool>> TasksIsMeshValidResult;
	for (UStaticMesh* StaticMesh : SortedMesh)
	{
		TasksIsMeshValidResult.Add(
			Async(
				EAsyncExecution::LargeThreadPool,
				[StaticMesh]() { return PreBuildStaticMesh(StaticMesh); },
				[&TasksDone]() { TasksDone++; }
			)
		);
	}

	FScopedSlowTask* Progress = ProgressPtr.Get();

	// Ensure UI stays responsive by updating the progress even when the number of tasks hasn't changed
	for (int32 OldTasksDone = 0, NewTasksDone = 0; OldTasksDone != SortedMesh.Num(); OldTasksDone = NewTasksDone)
	{
		NewTasksDone = TasksDone.Load();
		if ( Progress )
		{
			Progress->EnterProgressFrame(NewTasksDone - OldTasksDone, FText::FromString(FString::Printf(TEXT("Packing UVs and computing tangents for static mesh %d/%d ..."), NewTasksDone, SortedMesh.Num())));
			FPlatformProcess::Sleep(0.01f);
		}
	}

	for (int32 Index = 0; Index < TasksIsMeshValidResult.Num(); ++Index)
	{
		UStaticMesh* StaticMesh = SortedMesh[Index];

		if (TasksIsMeshValidResult[Index].Get())
		{
			SortedMesh[Index]->MarkPackageDirty();
		}
		else
		{
			// Log invalid meshes and remove mesh reference from imported meshes and mesh actors
			for (const auto& Kvp : ImportContext.ImportedStaticMeshes)
			{
				if (Kvp.Value == StaticMesh)
				{
					ImportContext.LogError( FText::Format( LOCTEXT("PreBuildMesh", "Static mesh {0} is invalid since it contains only degenerated triangles or no triangle."), FText::FromName( StaticMesh->GetFName() ) ) );

					ImportContext.ImportedStaticMeshes.Remove(Kvp.Key);
					break;
				}
			}
		}
	}
}

class FMeshMaterialSlotBuilder
{
public:
	struct FSectionInfo
	{
		enum { InvalidIndex = -1 };

		int32 LodIndex;
		int32 SectionIndex;        // index of this group (in given LOD's mesh)
		FPolygonGroupID SectionId; // id stored on the triangles
		FName MaterialSlotName;    // per PolygonGroup FName attribute
		int32 PolyCount;           // Number of poly
		int32 StaticMaterialsIndex = InvalidIndex; // index in the FStaticMesh::StaticMaterials
	};

	FMeshMaterialSlotBuilder(UStaticMesh& StaticMesh, int32 MeshElementLODCount)
	{
		// Populate section LODs declared in IDatasmithMeshElement
		for (int32 LodIndex = 0; LodIndex < MeshElementLODCount; ++LodIndex)
		{
			if (FMeshDescription* MeshDescription = StaticMesh.GetMeshDescription(LodIndex))
			{
				FStaticMeshAttributes Attributes(*MeshDescription);
				TPolygonGroupAttributesConstRef<FName> MaterialSlotNameAttribute = Attributes.GetPolygonGroupMaterialSlotNames();
				int32 SectionIndex = 0;
				RawSections.Reserve(MeshDescription->PolygonGroups().Num());
				for (FPolygonGroupID PolygonGroupID : MeshDescription->PolygonGroups().GetElementIDs())
				{
					FSectionInfo& ThisSection = RawSections.Add_GetRef({});
					ThisSection.LodIndex = LodIndex;
					ThisSection.SectionIndex = SectionIndex++;
					ThisSection.SectionId = PolygonGroupID;
					ThisSection.MaterialSlotName = MaterialSlotNameAttribute[PolygonGroupID];
					ThisSection.PolyCount = MeshDescription->GetPolygonGroupPolygonIDs(PolygonGroupID).Num();
				}
			}
		}

		// Prepares StaticMaterials Slot informations from sections found in the mesh
		for (FSectionInfo& Section : RawSections)
		{
			// get associated slot (but skip empty polygon groups)
			Section.StaticMaterialsIndex = Section.PolyCount <= 0 ? FSectionInfo::InvalidIndex : SlotNames.Add(Section.MaterialSlotName).AsInteger();
		}
	}

	FMeshSectionInfoMap GenerateSectionInfoMap() const
	{
		FMeshSectionInfoMap InfoMap;
		for (const FSectionInfo& Section : RawSections)
		{
			if (Section.StaticMaterialsIndex != FSectionInfo::InvalidIndex)
			{
				InfoMap.Set(Section.LodIndex, Section.SectionIndex, FMeshSectionInfo(Section.StaticMaterialsIndex));
			}
		}
		return InfoMap;
	}

	const TArray<FSectionInfo>& GetSectionsInfos() const
	{
		return RawSections;
	}

	const TSet<FName>& GetSlotNames() const
	{
		return SlotNames;
	}


private:
	TArray<FSectionInfo> RawSections;
	TSet<FName> SlotNames;
};

void FDatasmithStaticMeshImporter::SetupStaticMesh( FDatasmithAssetsImportContext& AssetsContext, TSharedRef< IDatasmithMeshElement > MeshElement, UStaticMesh* StaticMesh,
	const FDatasmithStaticMeshImportOptions& StaticMeshImportOptions, float LightmapWeight )
{
	if (StaticMesh == nullptr)
	{
		return;
	}

	// The input static mesh must at least have its LOD 0 valid
	if (!StaticMesh->IsMeshDescriptionValid(0))
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithStaticMeshImporter::SetupStaticMesh);

	FMeshMaterialSlotBuilder Sections(*StaticMesh, MeshElement->GetLODCount());
	UDatasmithStaticMeshTemplate* StaticMeshTemplate = NewObject< UDatasmithStaticMeshTemplate >( StaticMesh );

	int32 MinLightmapSize = FDatasmithStaticMeshImportOptions::ConvertLightmapEnumToValue( StaticMeshImportOptions.MinLightmapResolution );
	int32 MaxLightmapSize = FDatasmithStaticMeshImportOptions::ConvertLightmapEnumToValue( StaticMeshImportOptions.MaxLightmapResolution );

	int32 LightmapSize = DatasmithStaticMeshImporterImpl::GetLightmapSize( LightmapWeight, MinLightmapSize, MaxLightmapSize );
	StaticMesh->SetLightingGuid();
	StaticMesh->InitResources();

	StaticMeshTemplate->LightMapResolution = LightmapSize;
	StaticMeshTemplate->LightMapCoordinateIndex = 2;

	// Get build requirement for the assigned materials
	const int32 BuildRequirements = DatasmithStaticMeshImporterImpl::GetStaticMeshBuildRequirements( AssetsContext, MeshElement );

	TSet< int32 > MaterialIndexToImportIndex;

	// Only setup LODs declared by the mesh element
	for ( int32 LodIndex = 0; LodIndex < MeshElement->GetLODCount(); ++LodIndex)
	{
		if (!StaticMesh->IsMeshDescriptionValid(LodIndex))
		{
			continue;
		}

		FMeshDescription& MeshDescription = *StaticMesh->GetMeshDescription(LodIndex);

		// UV Channels
		int32 SourceIndex = 0;
		int32 DestinationIndex = 1;
		bool bUseImportedLightmap = false;
		bool bGenerateLightmapUVs = StaticMeshImportOptions.bGenerateLightmapUVs;
		const int32 FirstOpenUVChannel = UVGenerationUtils::GetNextOpenUVChannel(StaticMesh, LodIndex);

		// if a custom lightmap coordinate index was imported, disable lightmap generation
		if (DatasmithMeshHelper::HasUVData(MeshDescription, MeshElement->GetLightmapCoordinateIndex()))
		{
			bUseImportedLightmap = true;
			bGenerateLightmapUVs = false;
			DestinationIndex = MeshElement->GetLightmapCoordinateIndex();
		}
		else
		{
			if (MeshElement->GetLightmapCoordinateIndex() >= 0)
			{
				//A custom lightmap index value was set but the data was invalid.
				FFormatNamedArguments FormatArgs;
				FormatArgs.Add(TEXT("LightmapCoordinateIndex"), FText::FromString(FString::FromInt(MeshElement->GetLightmapCoordinateIndex())));
				FormatArgs.Add(TEXT("MeshName"), FText::FromName(StaticMesh->GetFName()));
				AssetsContext.GetParentContext().LogError(FText::Format(LOCTEXT("InvalidLightmapSourceUVError", "The lightmap coordinate index '{LightmapCoordinateIndex}' used for the mesh '{MeshName}' is invalid. A valid lightmap coordinate index was set instead."), FormatArgs));
			}

			DestinationIndex = FirstOpenUVChannel;
		}

		// Set the source lightmap index to the imported mesh data lightmap source if any, otherwise use the first open channel.
		if (DatasmithMeshHelper::HasUVData(MeshDescription, MeshElement->GetLightmapSourceUV()))
		{
			SourceIndex = MeshElement->GetLightmapSourceUV();
		}
		else
		{
			//If the lightmap source index was not set, we set it to the first open UV channel as it will be generated.
			//Also, it's okay to set both the source and the destination to be the same index as they are for different containers.
			SourceIndex = FirstOpenUVChannel;
		}

		if (bGenerateLightmapUVs)
		{
			if (!FMath::IsWithin(SourceIndex, 0, (int32)MAX_MESH_TEXTURE_COORDS_MD))
			{
				AssetsContext.GetParentContext().LogError(FText::Format(LOCTEXT("InvalidLightmapSourceIndexError", "Lightmap generation error for mesh {0}: Specified source is invalid {1}. Cannot find an available fallback source channel."), FText::FromName(StaticMesh->GetFName()), MeshElement->GetLightmapSourceUV()));
				bGenerateLightmapUVs = false;
			}
			else if (!FMath::IsWithin(DestinationIndex, 0, (int32)MAX_MESH_TEXTURE_COORDS_MD))
			{
				AssetsContext.GetParentContext().LogError(FText::Format(LOCTEXT("InvalidLightmapDestinationIndexError", "Lightmap generation error for mesh {0}: Cannot find an available destination channel."), FText::FromName(StaticMesh->GetFName())));
				bGenerateLightmapUVs = false;
			}

			if (!bGenerateLightmapUVs)
			{
				AssetsContext.GetParentContext().LogWarning(FText::Format(LOCTEXT("LightmapUVsWontBeGenerated", "Lightmap UVs for mesh {0} won't be generated."), FText::FromName(StaticMesh->GetFName())));
			}
		}

		FDatasmithMeshBuildSettingsTemplate BuildSettingsTemplate;
		BuildSettingsTemplate.Load(StaticMesh->GetSourceModel(LodIndex).BuildSettings);

		BuildSettingsTemplate.bUseMikkTSpace = true;
		BuildSettingsTemplate.bRecomputeNormals = FDatasmithStaticMeshImporter::ShouldRecomputeNormals(MeshDescription, BuildRequirements );
		BuildSettingsTemplate.bRecomputeTangents = FDatasmithStaticMeshImporter::ShouldRecomputeTangents(MeshDescription, BuildRequirements );
		BuildSettingsTemplate.bRemoveDegenerates = StaticMeshImportOptions.bRemoveDegenerates;
		BuildSettingsTemplate.bUseHighPrecisionTangentBasis = true;
		BuildSettingsTemplate.bUseFullPrecisionUVs = true;
		BuildSettingsTemplate.bGenerateLightmapUVs = bGenerateLightmapUVs;
		BuildSettingsTemplate.SrcLightmapIndex = SourceIndex;
		BuildSettingsTemplate.DstLightmapIndex = DestinationIndex;
		BuildSettingsTemplate.MinLightmapResolution = MinLightmapSize;

		StaticMeshTemplate->BuildSettings.Add( MoveTemp( BuildSettingsTemplate ) );

		if ( bUseImportedLightmap || StaticMeshImportOptions.bGenerateLightmapUVs )
		{
			//If we are generating the lightmap or are using an imported lightmap the DstLightmapIndex will already be set to the proper index.
			StaticMeshTemplate->LightMapCoordinateIndex = BuildSettingsTemplate.DstLightmapIndex;
		}
		else
		{
			//No lightmap UV provided or generated, set the LightmapCoordinateIndex to default 0 value which will always contain contain a basic UV.
			StaticMeshTemplate->LightMapCoordinateIndex = 0;
		}
	}

	// Build a FMeshSectionInfoMap so that it computes the proper key for us
	FMeshSectionInfoMap MeshSectionInfoMap = Sections.GenerateSectionInfoMap();
	StaticMeshTemplate->SectionInfoMap.Load( MeshSectionInfoMap );

	// Create one StaticMaterial per section. Actual materials will be assigned later. This is just so that we create all the slots.
	for (const FMeshMaterialSlotBuilder::FSectionInfo& SectionInfo : Sections.GetSectionsInfos())
	{
		FDatasmithStaticMaterialTemplate StaticMaterial;
		StaticMaterial.MaterialSlotName = SectionInfo.MaterialSlotName;

		StaticMeshTemplate->StaticMaterials.Add(MoveTemp(StaticMaterial));
	}

	ApplyMaterialsToStaticMesh(AssetsContext, MeshElement, StaticMeshTemplate);
	StaticMeshTemplate->Apply(StaticMesh);
}

TMap< TSharedRef< IDatasmithMeshElement >, float > FDatasmithStaticMeshImporter::CalculateMeshesLightmapWeights( const TSharedRef< IDatasmithScene >& SceneElement )
{
	TMap< TSharedRef< IDatasmithMeshElement >, float > LightmapWeights;

	float MaxArea = 0.0f;
	float Max2DSurface = 0.0f;

	// Compute the max values based on all meshes in the Datasmith Scene

	for ( int32 MeshIndex = 0; MeshIndex < SceneElement->GetMeshesCount(); ++MeshIndex )
	{
		TSharedPtr< IDatasmithMeshElement > MeshElement = SceneElement->GetMesh( MeshIndex );

		FVector4 Dimensions;
		Dimensions.Set( MeshElement->GetWidth(), MeshElement->GetDepth(), MeshElement->GetHeight(), MeshElement->GetArea() );

		MaxArea = FMath::Max( MaxArea, MeshElement->GetArea() );
		Max2DSurface = FMath::Max(Max2DSurface, DatasmithStaticMeshImporterImpl::Get2DSurface( Dimensions ));
	}

	float MaxWeight = 0.0f;

	for ( int32 MeshIndex = 0; MeshIndex < SceneElement->GetMeshesCount(); ++MeshIndex )
	{
		TSharedPtr< IDatasmithMeshElement > MeshElement = SceneElement->GetMesh( MeshIndex );

		FVector4 Dimensions( MeshElement->GetWidth(), MeshElement->GetDepth(), MeshElement->GetHeight(), MeshElement->GetArea() );

		MaxWeight = FMath::Max( MaxWeight, DatasmithStaticMeshImporterImpl::CalcBlendWeight( Dimensions, MaxArea, Max2DSurface ) );
	}

	for ( int32 MeshIndex = 0; MeshIndex < SceneElement->GetMeshesCount(); ++MeshIndex )
	{
		TSharedPtr< IDatasmithMeshElement > MeshElement = SceneElement->GetMesh( MeshIndex );

		FVector4 Dimensions( MeshElement->GetWidth(), MeshElement->GetDepth(), MeshElement->GetHeight(), MeshElement->GetArea() );

		float LightmapWeight = DatasmithStaticMeshImporterImpl::CalcBlendWeight( Dimensions, MaxArea, Max2DSurface ) / MaxWeight;

		LightmapWeights.Add( MeshElement.ToSharedRef() ) = LightmapWeight;
	}

	return LightmapWeights;
}

void FDatasmithStaticMeshImporter::ApplyMaterialsToStaticMesh( const FDatasmithAssetsImportContext& AssetsContext, const TSharedRef< IDatasmithMeshElement >& MeshElement, UDatasmithStaticMeshTemplate* StaticMeshTemplate )
{
	for ( TPair<int32, FString>& Material : DatasmithStaticMeshImporterImpl::GetMaterials( MeshElement ) )
	{
		UMaterialInterface* MaterialInterface = FDatasmithImporterUtils::FindAsset< UMaterialInterface >( AssetsContext, *Material.Value );

		FName MaterialSlotName = DatasmithMeshHelper::DefaultSlotName(Material.Key);

		FDatasmithStaticMaterialTemplate* MaterialTemplate = Algo::FindByPredicate( StaticMeshTemplate->StaticMaterials, [ &MaterialSlotName ]( const FDatasmithStaticMaterialTemplate& Current ) -> bool
		{
			return Current.MaterialSlotName == MaterialSlotName;
		});

		if ( MaterialTemplate )
		{
			MaterialTemplate->MaterialSlotName = MaterialInterface  ? *MaterialInterface->GetName() : TEXT("Undefined");
			MaterialTemplate->MaterialInterface = MaterialInterface;
		}
	}
}

#undef LOCTEXT_NAMESPACE
