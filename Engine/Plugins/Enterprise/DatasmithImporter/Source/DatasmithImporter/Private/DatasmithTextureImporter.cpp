// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithTextureImporter.h"

#include "DatasmithMaterialExpressions.h"
#include "IDatasmithSceneElements.h"
#include "DatasmithImportContext.h"
#include "Utility/DatasmithImporterUtils.h"
#include "Utility/DatasmithTextureResize.h"

#include "AssetRegistry/AssetRegistryModule.h"

#include "Async/AsyncWork.h"

#include "Editor.h"
#include "EditorFramework/AssetImportData.h"

#include "Engine/Texture2D.h"
#include "Engine/TextureCube.h"
#include "Engine/TextureLightProfile.h"

#include "Factories/TextureFactory.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"

#include "Logging/LogMacros.h"

#include "Misc/FileHelper.h"

#include "Modules/ModuleManager.h"

#include "InterchangeManager.h"
#include "ObjectTools.h"
#include "RHI.h"
#include "InterchangeSourceData.h"
#include "InterchangeTexture2DFactoryNode.h"
#include "InterchangeTextureNode.h"

#define LOCTEXT_NAMESPACE "DatasmithTextureImport"

namespace
{
	static bool ResizeTexture(const TCHAR* Filename, const TCHAR* ResizedFilename, bool bCreateNormal, FDatasmithImportContext& ImportContext)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ResizeTexture);

		EDSTextureUtilsError ErrorCode = FDatasmithTextureResize::ResizeTexture(Filename, ResizedFilename, EDSResizeTextureMode::NearestPowerOfTwo, GMaxTextureDimensions, bCreateNormal);

		switch (ErrorCode)
		{
		case EDSTextureUtilsError::FileNotFound:
		{
			ImportContext.LogError( FText::Format( LOCTEXT( "FileNotFound", "Unable to find Texture file {0}."), FText::FromString( Filename ) ) );
			return false;
		}
		case EDSTextureUtilsError::InvalidFileType:
		{
			ImportContext.LogError( FText::Format( LOCTEXT( "InvalidFileType", "Cannot determine type of image file {0}."), FText::FromString( Filename ) ) );
			return false;
		}
		case EDSTextureUtilsError::FileReadIssue:
		{
			ImportContext.LogError( FText::Format( LOCTEXT( "FileReadIssue", "Cannot open image file {0}."), FText::FromString( Filename ) ) );
			return false;
		}
		case EDSTextureUtilsError::InvalidData:
		{
			ImportContext.LogError( FText::Format( LOCTEXT( "InvalidData", "Image file {0} contains invalid data."), FText::FromString( Filename ) ) );
			return false;
		}
		case EDSTextureUtilsError::FreeImageNotFound:
		{
			ImportContext.LogError( LOCTEXT( "FreeImageNotFound", "FreeImage.dll couldn't be found. Texture resizing won't be done.") );
			break;
		}
		default:
			break;
		}

		return true;
	}
}


FDatasmithTextureImporter::FDatasmithTextureImporter(FDatasmithImportContext& InImportContext)
	: ImportContext(InImportContext)
	, TextureFact( NewObject< UTextureFactory >() )
{
	TextureFact->SuppressImportOverwriteDialog();

	// Avoid recomputing DDC when importing the same texture more than once
	TextureFact->bUseHashAsGuid = true;

	TempDir = FPaths::Combine(FPaths::ProjectIntermediateDir(), TEXT("DatasmithTextureImport"));
	IFileManager::Get().MakeDirectory(*TempDir);
}

FDatasmithTextureImporter::~FDatasmithTextureImporter()
{
	// Clean up all transient files created during the process
	IFileManager::Get().DeleteDirectory(*TempDir, false, true);
}

bool FDatasmithTextureImporter::ResizeTextureElement(const TSharedPtr<IDatasmithTextureElement>& TextureElement, FString& ResizedFilename)
{
	FString Filename = TextureElement->GetFile();

	if (Filename.IsEmpty() || !FPaths::FileExists(Filename))
	{
		UE_LOGF(LogDatasmithImport, Warning, "Unable to find Texture file %ls", *Filename);
		return false;
	}

	FString Extension;
	if (!FDatasmithTextureResize::GetBestTextureExtension(*Filename, Extension))
	{
		return false;
	}
	// Convert HDR image to EXR if not used as environment map
	else if (Extension == TEXT(".hdr") && TextureElement->GetTextureMode() != EDatasmithTextureMode::Other)
	{
		Extension = TEXT(".exr");
	}

	ResizedFilename = FPaths::Combine(TempDir, LexToString(FGuid::NewGuid()) + Extension);

	const bool bGenerateNormalMap = ( TextureElement->GetTextureMode() == EDatasmithTextureMode::Bump );

	const bool bResult = ResizeTexture(*Filename, *ResizedFilename, bGenerateNormalMap, ImportContext);

	if ( bResult && bGenerateNormalMap )
	{
		TextureElement->SetTextureMode( EDatasmithTextureMode::Normal );
	}

	return bResult;
}

bool FDatasmithTextureImporter::GetTextureData(const TSharedPtr<IDatasmithTextureElement>& TextureElement, TArray<uint8>& TextureData, FString& Extension)
{
	const FString Filename = TextureElement->GetFile();
	if (!Filename.IsEmpty())
	{
		// load from a file path

		FString ImageFileName;
		if (!ResizeTextureElement(TextureElement, ImageFileName))
		{
			return false;
		}

		// try opening from absolute path
		if (!(FFileHelper::LoadFileToArray(TextureData, *ImageFileName) && TextureData.Num() > 0))
		{
			ImportContext.LogWarning(FText::Format(LOCTEXT("UnableToFindTexture", "Unable to find Texture file {0}."), FText::FromString(Filename)));
			return false;
		}

		Extension         = FPaths::GetExtension(ImageFileName).ToLower();
	}
	else
	{
		// load from memory source
		EDatasmithTextureFormat TextureFormat;
		uint32                  TextureSize;

		const uint8* PtrTextureData = TextureElement->GetData(TextureSize, TextureFormat);

		if (PtrTextureData == nullptr || !TextureSize)
		{
			return false;
		}

		TextureData.SetNumUninitialized(TextureSize);
		FPlatformMemory::Memcpy(TextureData.GetData(), PtrTextureData, TextureSize);

		switch (TextureFormat)
		{
			case EDatasmithTextureFormat::PNG:
				Extension = TEXT("png");
				break;
			case EDatasmithTextureFormat::JPEG:
				Extension = TEXT("jpeg");
				break;
			default:
				check(false);
		}
	}

	return true;
}


UTexture* FDatasmithTextureImporter::CreateTexture(const TSharedPtr<IDatasmithTextureElement>& TextureElement, const TArray<uint8>& TextureData, const FString& Extension)
{
	if (TextureElement->GetTextureMode() == EDatasmithTextureMode::Ies)
	{
		return CreateIESTexture(TextureElement);
	}

	const FString TextureLabel = TextureElement->GetLabel();
	UPackage* DestinationPackage = ImportContext.AssetsContext.TexturesFinalPackage.Get();
	int32 AssetNameMaxLength = FDatasmithImporterUtils::GetAssetNameMaxCharCount(DestinationPackage);
	const FString TextureName = ImportContext.AssetsContext.TextureNameProvider.GenerateUniqueName(TextureLabel.Len() > 0 ? TextureLabel : TextureElement->GetName(), AssetNameMaxLength) ;

	// Verify that the texture could be created in final package
	FText FailReason;
	if (!FDatasmithImporterUtils::CanCreateAsset<UTexture2D>(DestinationPackage, TextureName, FailReason))
	{
		ImportContext.LogError(FailReason);
		return nullptr;
	}

	// Reset options that are not always overwritten because single TextureFact instance is reused for every imported texture 
	TextureFact->LODGroup = TEXTUREGROUP_World;
	TextureFact->CompressionSettings = TC_Default;
	TextureFact->bFlipNormalMapGreenChannel = false;

	// Make sure to set the proper LODGroup as it's used to determine the CompressionSettings when using TEXTUREGROUP_WorldNormalMap
	switch (TextureElement->GetTextureMode())
	{
	case EDatasmithTextureMode::Diffuse:
		TextureFact->LODGroup = TEXTUREGROUP_World;
		break;
	case EDatasmithTextureMode::Specular:
		TextureFact->LODGroup = TEXTUREGROUP_WorldSpecular;
		break;
	case EDatasmithTextureMode::Bump:
	case EDatasmithTextureMode::Normal:
		TextureFact->LODGroup = TEXTUREGROUP_WorldNormalMap;
		TextureFact->CompressionSettings = TC_Normalmap;
		break;
	case EDatasmithTextureMode::NormalGreenInv:
		TextureFact->LODGroup = TEXTUREGROUP_WorldNormalMap;
		TextureFact->CompressionSettings = TC_Normalmap;
		TextureFact->bFlipNormalMapGreenChannel = true;
		break;
	}

	const EDatasmithColorSpace DatasmithColorSpace = TextureElement->GetSRGB();
	ETextureSourceColorSpace FactoryColorSpace = ETextureSourceColorSpace::Auto;

	if (DatasmithColorSpace == EDatasmithColorSpace::sRGB)
	{
		FactoryColorSpace = ETextureSourceColorSpace::SRGB;
	}
	else if (DatasmithColorSpace == EDatasmithColorSpace::Linear)
	{
		FactoryColorSpace = ETextureSourceColorSpace::Linear;
	}

	TextureFact->ColorSpaceMode = FactoryColorSpace;

	const float RGBCurve = TextureElement->GetRGBCurve();

	const uint8* PtrTextureData    = TextureData.GetData();
	const uint8* PtrTextureDataEnd = PtrTextureData + TextureData.Num();
	const FString Filename = TextureElement->GetFile();
	UPackage* TextureOuter = ImportContext.AssetsContext.TexturesImportPackage.Get();

	// This has to be called explicitly each time we create a texture since the flag gets reset in FactoryCreateBinary
	TextureFact->SuppressImportOverwriteDialog();

	UTexture* Texture =
		Cast<UTexture>(TextureFact->FactoryCreateBinary(UTexture2D::StaticClass(), TextureOuter, *TextureName, ImportContext.ObjectFlags /*& ~RF_Public*/, nullptr,
															*Extension, PtrTextureData, PtrTextureDataEnd, GWarn));
	if (Texture != nullptr)
	{
		Texture->PreEditChange(nullptr);

		static_assert(TextureAddress::TA_Wrap == (int)EDatasmithTextureAddress::Wrap && TextureAddress::TA_Mirror == (int)EDatasmithTextureAddress::Mirror, "Texture Address enum doesn't match!" );

		FMD5Hash Hash = TextureElement->CalculateElementHash(false);

		TextureFilter TexFilter = TextureFilter::TF_Default;

		switch ( TextureElement->GetTextureFilter() )
		{
		case EDatasmithTextureFilter::Nearest:
			TexFilter = TextureFilter::TF_Nearest;
			break;
		case EDatasmithTextureFilter::Bilinear:
			TexFilter = TextureFilter::TF_Bilinear;
			break;
		case EDatasmithTextureFilter::Trilinear:
			TexFilter = TextureFilter::TF_Trilinear;
			break;
		case EDatasmithTextureFilter::Default:
		default:
			TexFilter = TextureFilter::TF_Default;
			break;
		}

		Texture->Filter = TexFilter;

		if (UTexture2D* Texture2D = Cast<UTexture2D>(Texture))
		{
			Texture2D->AddressX = (TextureAddress)TextureElement->GetTextureAddressX();
			Texture2D->AddressY = (TextureAddress)TextureElement->GetTextureAddressY();
		}

		// Update import data
		Texture->AssetImportData->Update(Filename, &Hash);

		// Notify the asset registry
		FAssetRegistryModule::AssetCreated(Texture);

		if (FMath::IsNearlyEqual(RGBCurve, 1.0f) == false && RGBCurve > 0.f)
		{
			Texture->AdjustRGBCurve = RGBCurve;
		}

		// Double check that the texture color space matches what we requested
		EDatasmithColorSpace ColorSpace = TextureElement->GetSRGB();
		if (!Texture->SRGB && ColorSpace == EDatasmithColorSpace::sRGB)
		{
			Texture->SRGB = true;
		}
		else if (Texture->SRGB && ColorSpace == EDatasmithColorSpace::Linear)
		{
			Texture->SRGB = false;
		}

		Texture->PostEditChange();
	}

	return Texture;
}

UTexture* FDatasmithTextureImporter::CreateIESTexture(const TSharedPtr<IDatasmithTextureElement>& TextureElement)
{
	UTextureLightProfile* IESTexture = nullptr;

	FString Filename(TextureElement->GetFile());
	if (Filename.IsEmpty() || !FPaths::FileExists(Filename))
	{
		UE_LOGF(LogDatasmithImport, Error, "Unable to find ies file %ls", *Filename);

		return nullptr;
	}

	FString Extension = FPaths::GetExtension(Filename).ToLower();
	FString TextureName(TextureElement->GetName());

	// try opening from absolute path
	TArray<uint8> TextureData;
	if (!(FFileHelper::LoadFileToArray(TextureData, *Filename) && TextureData.Num() > 0))
	{
		UE_LOGF(LogDatasmithImport, Warning, "Unable to find Texture file %ls", *Filename);
		return nullptr;
	}

	TextureFact->SuppressImportOverwriteDialog();

	TextureFact->LODGroup = TEXTUREGROUP_IESLightProfile;
	TextureFact->CompressionSettings = TC_HDR;

	const uint8* PtrTexture = TextureData.GetData();

	IESTexture = (UTextureLightProfile*)TextureFact->FactoryCreateBinary(UTextureLightProfile::StaticClass(),
							ImportContext.AssetsContext.TexturesImportPackage.Get(),
							*TextureName, ImportContext.ObjectFlags, nullptr,
							*Extension, PtrTexture,	PtrTexture + TextureData.Num(), GWarn);
	if (IESTexture != nullptr)
	{
		IESTexture->AssetImportData->Update(Filename);

		// Notify the asset registry
		FAssetRegistryModule::AssetCreated(IESTexture);

		IESTexture->MarkPackageDirty();
	}

	return IESTexture;
}

UE::Interchange::FAssetImportResultRef FDatasmithTextureImporter::CreateTextureAsync(const TSharedPtr<IDatasmithTextureElement>& TextureElement)
{
	UE::Interchange::FScopedSourceData ScopedSourceData( TextureElement->GetFile() );

	if ( !UInterchangeManager::GetInterchangeManager().CanTranslateSourceData( ScopedSourceData.GetSourceData() ) )
	{
		return MakeShared< UE::Interchange::FImportResult, ESPMode::ThreadSafe >();
	}

	const FString ContentPath = ImportContext.AssetsContext.TexturesImportPackage->GetPathName();

	FImportAssetParameters ImportAssetParameters;
	ImportAssetParameters.bIsAutomated = true; // From the InterchangeManager point of view, this is considered an automated import

	UDatasmithTexturePipeline* TexturePipeline = NewObject< UDatasmithTexturePipeline >();
	TexturePipeline->TextureElement = TextureElement;

	ImportAssetParameters.OverridePipelines.Add( TexturePipeline );

	return UInterchangeManager::GetInterchangeManager().ImportAssetAsync( ContentPath, ScopedSourceData.GetSourceData(), ImportAssetParameters );
}

void UDatasmithTexturePipeline::ExecutePipeline(UInterchangeBaseNodeContainer* BaseNodeContainer, const TArray<UInterchangeSourceData*>& SourceDatas, const FString& ContentBasePath)
{
	if ( !TextureElement.IsValid() )
	{
		return;
	}

	TArray< FString > Nodes;
	BaseNodeContainer->GetRoots( Nodes );

	if ( Nodes.Num() <= 0)
	{
		return;
	}

	const UInterchangeTextureNode* TextureTranslatedNode = Cast<const UInterchangeTextureNode >( BaseNodeContainer->GetNode( Nodes[0] ) );
	if (!ensure(TextureTranslatedNode))
	{
		return;
	}

	FString DisplayLabel = TextureTranslatedNode->GetDisplayLabel();
	FString NodeUID = UInterchangeTextureFactoryNode::GetTextureFactoryNodeUidFromTextureNodeUid(TextureTranslatedNode->GetUniqueID());
	UInterchangeTexture2DFactoryNode* TextureFactoryNode = NewObject<UInterchangeTexture2DFactoryNode>(BaseNodeContainer, NAME_None);
	if (!ensure(TextureFactoryNode))
	{
		return;
	}
	//Creating a UTexture2D
	TextureFactoryNode->InitializeTextureNode(NodeUID, DisplayLabel, TextureTranslatedNode->GetDisplayLabel(), BaseNodeContainer);
	TextureFactoryNode->SetCustomTranslatedTextureNodeUid(TextureTranslatedNode->GetUniqueID());

	TOptional< bool > bFlipNormalMapGreenChannel;
	TOptional< TextureMipGenSettings > MipGenSettings;
	TOptional< TextureGroup > LODGroup;
	TOptional< TextureCompressionSettings > CompressionSettings;

	// Make sure to set the proper LODGroup as it's used to determine the CompressionSettings when using TEXTUREGROUP_WorldNormalMap
	switch ( TextureElement->GetTextureMode() )
	{
	case EDatasmithTextureMode::Diffuse:
		LODGroup = TEXTUREGROUP_World;
		break;
	case EDatasmithTextureMode::Specular:
		LODGroup = TEXTUREGROUP_WorldSpecular;
		break;
	case EDatasmithTextureMode::Bump:
	case EDatasmithTextureMode::Normal:
		LODGroup = TEXTUREGROUP_WorldNormalMap;
		CompressionSettings = TC_Normalmap;
		break;
	case EDatasmithTextureMode::NormalGreenInv:
		LODGroup = TEXTUREGROUP_WorldNormalMap;
		CompressionSettings = TC_Normalmap;
		bFlipNormalMapGreenChannel = true;
		break;
	}

	const TOptional< float > RGBCurve = [ this ]() -> TOptional< float >
	{
		const float ElementRGBCurve = TextureElement->GetRGBCurve();

		if ( FMath::IsNearlyEqual( ElementRGBCurve, 1.0f ) == false && ElementRGBCurve > 0.f )
		{
			return ElementRGBCurve;
		}
		else
		{
			return {};
		}
	}();

	static_assert(TextureAddress::TA_Wrap == (int)EDatasmithTextureAddress::Wrap && TextureAddress::TA_Mirror == (int)EDatasmithTextureAddress::Mirror, "Texture Address enum doesn't match!" );

	TOptional< TextureFilter > TexFilter;

	switch ( TextureElement->GetTextureFilter() )
	{
	case EDatasmithTextureFilter::Nearest:
		TexFilter = TextureFilter::TF_Nearest;
		break;
	case EDatasmithTextureFilter::Bilinear:
		TexFilter = TextureFilter::TF_Bilinear;
		break;
	case EDatasmithTextureFilter::Trilinear:
		TexFilter = TextureFilter::TF_Trilinear;
		break;
	}

	TOptional< bool > bSrgb;

	EDatasmithColorSpace ColorSpace = TextureElement->GetSRGB();
	if ( ColorSpace == EDatasmithColorSpace::sRGB )
	{
		bSrgb = true;
	}
	else if ( ColorSpace == EDatasmithColorSpace::Linear )
	{
		bSrgb = false;
	}

	TextureFactoryNode->SetCustomAddressX( (TextureAddress)TextureElement->GetTextureAddressX() );
	TextureFactoryNode->SetCustomAddressY( (TextureAddress)TextureElement->GetTextureAddressY() );

	if ( bSrgb.IsSet() )
	{
		TextureFactoryNode->SetCustomSRGB( bSrgb.GetValue() );
	}

	if ( bFlipNormalMapGreenChannel.IsSet() )
	{
		TextureFactoryNode->SetCustombFlipGreenChannel( bFlipNormalMapGreenChannel.GetValue() );
	}

	if ( MipGenSettings.IsSet() )
	{
		TextureFactoryNode->SetCustomMipGenSettings( MipGenSettings.GetValue() );
	}

	if ( LODGroup.IsSet() )
	{
		TextureFactoryNode->SetCustomLODGroup( LODGroup.GetValue() );
	}

	if ( CompressionSettings.IsSet() )
	{
		TextureFactoryNode->SetCustomLODGroup( CompressionSettings.GetValue() );
	}

	if ( RGBCurve.IsSet() )
	{
		TextureFactoryNode->SetCustomAdjustRGBCurve( RGBCurve.GetValue() );
	}

	if ( TexFilter.IsSet() )
	{
		TextureFactoryNode->SetCustomFilter( TexFilter.GetValue() );
	}
}

#undef LOCTEXT_NAMESPACE
