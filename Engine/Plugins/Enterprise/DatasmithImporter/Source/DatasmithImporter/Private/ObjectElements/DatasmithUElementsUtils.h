// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "DatasmithVariantElements.h"
#include "IDatasmithSceneElements.h"
#include "UObject/UObjectGlobals.h"

class IDatasmithLevelVariantSetsElement;

class FDatasmithUElementsUtils
{

	template <typename T>
	struct TFindOrAddTraits;

	template <typename T>
	struct TFindOrAddTraits<T*>
	{
		using TElement = T;
	};

	template <typename T>
	struct TFindOrAddTraits<TObjectPtr<T>>
	{
		using TElement = T;
	};

public:
	template<typename IElement,
					 typename TMapValue,
					 typename UElement = typename TFindOrAddTraits<TMapValue>::TElement,
					 typename SetOp>
	static UElement* FindOrAddElement(UObject* Outer, TMap<TWeakPtr<IElement>, TMapValue>& Map, const TSharedPtr<IElement>& InElement, SetOp Assign)
	{
		if (InElement.IsValid())
		{
			UElement* FoundElement = Map.FindRef(InElement);
			if (FoundElement)
			{
				return FoundElement;
			}

			FoundElement = NewObject<UElement>(Outer);
			Assign(FoundElement);
			Map.Add(InElement, FoundElement);
			return FoundElement;
		}

		return nullptr;
	}

	/**
	 * Utility to help iterating over the nested structure of the variant objects
	 * Examples:
	 * ForVariantElement<IDatasmithVariantElement>(SceneElement, [](TSharedPtr<IDatasmithVariantElement> Var)
	 * {
	 *     // Do something with all variants in the scene
	 *
	 *     return true; // It will keep iterating while the lambda returns true
	 * });
	 *
	 * ForVariantElement<IDatasmithBasePropertyCaptureElement>(SomeVariantSet, [](TSharedPtr<IDatasmithBasePropertyCaptureElement> Prop)
	 * {
	 *     // Do something with all properties of a variant set
	 *
	 *     return true; // It will keep iterating while the lambda returns true
	 * });
	 */
	template<typename TargetType, typename Func>
	static inline bool ForVariantElement(const TSharedPtr<IDatasmithScene>& RootObject, Func Function)
	{
		for (int32 Index = 0; Index < RootObject->GetLevelVariantSetsCount(); ++Index)
		{
			bool bKeepIterating;
			if constexpr (std::is_same_v<TargetType, IDatasmithLevelVariantSetsElement>)
			{
				bKeepIterating = Function(RootObject->GetLevelVariantSets(Index));
			}
			else
			{
				bKeepIterating = ForVariantElement<TargetType>(RootObject->GetLevelVariantSets(Index), Function);
			}
			if (!bKeepIterating)
			{
				return false;
			}
		}
		return true;
	}
	template<typename TargetType, typename Func>
	static inline bool ForVariantElement(const TSharedPtr<IDatasmithLevelVariantSetsElement>& RootObject, Func Function)
	{
		for (int32 Index = 0; Index < RootObject->GetVariantSetsCount(); ++Index)
		{
			bool bKeepIterating;
			if constexpr (std::is_same_v<TargetType, IDatasmithVariantSetElement>)
			{
				bKeepIterating = Function(RootObject->GetVariantSet(Index));
			}
			else
			{
				bKeepIterating = ForVariantElement<TargetType>(RootObject->GetVariantSet(Index), Function);
			}
			if (!bKeepIterating)
			{
				return false;
			}
		}
		return true;
	}

	template<typename TargetType, typename Func>
	static inline bool ForVariantElement(const TSharedPtr<IDatasmithVariantSetElement>& RootObject, Func Function)
	{
		for (int32 Index = 0; Index < RootObject->GetVariantsCount(); ++Index)
		{
			bool bKeepIterating;
			if constexpr (std::is_same_v<TargetType, IDatasmithVariantElement>)
			{
				bKeepIterating = Function(RootObject->GetVariant(Index));
			}
			else
			{
				bKeepIterating = ForVariantElement<TargetType>(RootObject->GetVariant(Index), Function);
			}
			if (!bKeepIterating)
			{
				return false;
			}
		}
		return true;
	}
	template<typename TargetType, typename Func>
	static inline bool ForVariantElement(const TSharedPtr<IDatasmithVariantElement>& RootObject, Func Function)
	{
		for (int32 Index = 0; Index < RootObject->GetActorBindingsCount(); ++Index)
		{
			bool bKeepIterating;
			if constexpr (std::is_same_v<TargetType, IDatasmithActorBindingElement>)
			{
				bKeepIterating = Function(RootObject->GetActorBinding(Index));
			}
			else
			{
				bKeepIterating = ForVariantElement<TargetType>(RootObject->GetActorBinding(Index), Function);
			}
			if (!bKeepIterating)
			{
				return false;
			}
		}
		return true;
	}
	template<typename TargetType, typename Func>
	static inline typename TEnableIf<std::is_same_v<TargetType, IDatasmithBasePropertyCaptureElement>, bool>::Type ForVariantElement(const TSharedPtr<IDatasmithActorBindingElement>& RootObject, Func Function)
	{
		for (int32 Index = 0; Index < RootObject->GetPropertyCapturesCount(); ++Index)
		{
			bool bKeepIterating = Function(RootObject->GetPropertyCapture(Index));
			if (!bKeepIterating)
			{
				return false;
			}
		}
		return true;
	}
};
