// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectElements/DatasmithUObjectElements.h"

#include "DatasmithDefinitions.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithUElementsUtils.h"
#include "DatasmithUtils.h"
#include "IDatasmithSceneElements.h"
#include "ObjectElements/DatasmithUSceneElement.h"
#include "Utility/DatasmithImporterUtils.h"

#include "Engine/StaticMesh.h"

#define DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(TIElementType, functionName, returnCode) \
	TSharedPtr<TIElementType> Element = functionName.Pin(); \
	if (!Element.IsValid()) \
	{ \
		UE_LOGF(LogDatasmithImport, Error, "The '%ls' Object Element is not valid anymore", *GetName()); \
		return returnCode; \
	}

#define DATASMITHOBJECTELEMENT_GETARGUMENT_AND_EARLYREATURN(TIElementType, functionName, returnCode, InArg, TypeForMessage) \
	if (!InArg) \
	{ \
		UE_LOGF(LogDatasmithImport, Error, "Can't remove invalid %ls", TypeForMessage); \
		return returnCode; \
	} \
	if (!InArg->IsElementValid()) \
	{ \
		UE_LOGF(LogDatasmithImport, Error, "The %ls is not from this Scene", TypeForMessage); \
		return returnCode; \
	} \
	TSharedPtr<TIElementType> ArgElement = InArg->functionName.Pin(); \
	if (!ArgElement.IsValid()) \
	{ \
		UE_LOGF(LogDatasmithImport, Error, "The '%ls' Object Element is not valid anymore", *InArg->GetName()); \
		return returnCode; \
	}

#define DATASMITHOBJECTELEMENT_ISELEMENTVALID(Element) \
	if (!Super::IsElementValid()) \
	{ \
		return false; \
	} \
	UDatasmithSceneElementBase* Scene = Cast<UDatasmithSceneElementBase>(GetOuter()); \
	if (!Scene || !Scene->IsElementValid(Element)) \
	{ \
		return false; \
	}

/**
 * UDatasmithObjectElement
 */
FString UDatasmithObjectElement::GetElementName() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithElement, GetIDatasmithElement(), FString());
	return FString(Element->GetName());
}

FString UDatasmithObjectElement::GetLabel() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithElement, GetIDatasmithElement(), FString());
	return Element->GetLabel();
}

void UDatasmithObjectElement::SetLabel(const FString& InLabel)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithElement, GetIDatasmithElement(), );
	Element->SetLabel(*InLabel);
}

bool UDatasmithObjectElement::IsElementValid() const
{
	if (!GetIDatasmithElement().IsValid())
	{
		return false;
	}

	return true;
}

/**
 * UDatasmithActorElement
 */
FVector UDatasmithActorElement::GetTranslation() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithActorElement, GetIDatasmithActorElement(), FVector::ZeroVector);
	return Element->GetTranslation();
}

void UDatasmithActorElement::SetTranslation(FVector Value)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithActorElement, GetIDatasmithActorElement(), );
	Element->SetTranslation(Value);
}

FVector UDatasmithActorElement::GetScale() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithActorElement, GetIDatasmithActorElement(), FVector::ZeroVector);
	return Element->GetScale();
}

void UDatasmithActorElement::SetScale(FVector Value)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithActorElement, GetIDatasmithActorElement(), );
	Element->SetScale(Value);
}

FQuat UDatasmithActorElement::GetRotation() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithActorElement, GetIDatasmithActorElement(), FQuat::Identity);
	return Element->GetRotation();
}

void UDatasmithActorElement::SetRotation(FQuat Value)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithActorElement, GetIDatasmithActorElement(), );
	Element->SetRotation(Value);
}

FString UDatasmithActorElement::GetLayer() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithActorElement, GetIDatasmithActorElement(), FString());
	return Element->GetLayer();
}

void UDatasmithActorElement::SetLayer(const FString& InLayer)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithActorElement, GetIDatasmithActorElement(), );
	Element->SetLayer(*InLayer);
}

TArray<FString> UDatasmithActorElement::GetTags() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithActorElement, GetIDatasmithActorElement(), TArray<FString>());

	const int32 NumberOfTags = Element->GetTagsCount();
	TArray<FString> Result;
	if (NumberOfTags > 0)
	{
		Result.Reserve(NumberOfTags);
		for (int32 Index = 0; Index < NumberOfTags; ++Index)
		{
			Result.Add(FString(Element->GetTag(Index)));
		}
	}
	return Result;
}

void UDatasmithActorElement::SetTags(const TArray<FString>& InTags)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithActorElement, GetIDatasmithActorElement(), );
	Element->ResetTags();
	for (const FString& Str : InTags)
	{
		Element->AddTag(*Str);
	}
}

void UDatasmithActorElement::AddChild(UDatasmithActorElement* InChild)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithActorElement, GetIDatasmithActorElement(), );
	DATASMITHOBJECTELEMENT_GETARGUMENT_AND_EARLYREATURN(IDatasmithActorElement, GetIDatasmithActorElement(), , InChild, TEXT("Actor"));
	Element->AddChild(ArgElement);
}

int32 UDatasmithActorElement::GetChildrenCount() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithActorElement, GetIDatasmithActorElement(), 0);
	return Element->GetChildrenCount();
}

TArray<UDatasmithActorElement*> UDatasmithActorElement::GetChildren() const
{
	TArray<UDatasmithActorElement*> Result;
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithActorElement, GetIDatasmithActorElement(), Result);

	int32 ActorCount = GetChildrenCount();
	UDatasmithSceneElementBase* Scene = Cast<UDatasmithSceneElementBase>(GetOuter());
	if (Scene && ActorCount > 0)
	{
		Result.Reserve(ActorCount);
		for (int32 Index = 0; Index < ActorCount; ++Index)
		{
			TSharedPtr<IDatasmithActorElement> MeshActorElement = Element->GetChild(Index);
			UDatasmithActorElement* NewActorElement = Scene->FindOrAddActorElement(MeshActorElement);
			if (NewActorElement)
			{
				Result.Add(NewActorElement);
			}
		}
	}

	return Result;
}

void UDatasmithActorElement::RemoveChild(UDatasmithActorElement* InChild)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithActorElement, GetIDatasmithActorElement(), );
	DATASMITHOBJECTELEMENT_GETARGUMENT_AND_EARLYREATURN(IDatasmithActorElement, GetIDatasmithActorElement(), , InChild, TEXT("Actor"));
	Element->RemoveChild(ArgElement);
}

bool UDatasmithActorElement::GetVisibility() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithActorElement, GetIDatasmithActorElement(), false);
	return Element->GetVisibility();
}

void UDatasmithActorElement::SetVisibility(bool bInVisibility)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithActorElement, GetIDatasmithActorElement(), );
	Element->SetVisibility(bInVisibility);
}


/**
 * UDatasmithMeshElement
 */
FString UDatasmithMeshElement::GetFile() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithMeshElement, MeshElement, FString());
	return FString(Element->GetFile());
}

float UDatasmithMeshElement::GetBoundingBoxWidth() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithMeshElement, MeshElement, 0.f);
	return Element->GetWidth();
}

float UDatasmithMeshElement::GetBoundingBoxHeight() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithMeshElement, MeshElement, 0.f);
	return Element->GetHeight();
}

float UDatasmithMeshElement::GetBoundingBoxDepth() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithMeshElement, MeshElement, 0.f);
	return Element->GetDepth();
}

FVector UDatasmithMeshElement::GetBoundingBoxSize() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithMeshElement, MeshElement, FVector::ZeroVector);
	return FVector(Element->GetWidth(), Element->GetHeight(), Element->GetDepth());
}

float UDatasmithMeshElement::GetLightMapArea() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithMeshElement, MeshElement, 0.f);
	return Element->GetArea();
}

int32 UDatasmithMeshElement::GetLightmapCoordinateIndex() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithMeshElement, MeshElement, -1);
	return Element->GetLightmapCoordinateIndex();
}

void UDatasmithMeshElement::SetLightmapCoordinateIndex(int32 UVChannel)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithMeshElement, MeshElement,);
	return Element->SetLightmapCoordinateIndex(UVChannel);
}

int32 UDatasmithMeshElement::GetLightmapSourceUV() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithMeshElement, MeshElement, 0);
	return Element->GetLightmapSourceUV();
}

void UDatasmithMeshElement::SetLightmapSourceUV(int32 UVChannel)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithMeshElement, MeshElement, );
	return Element->SetLightmapSourceUV(UVChannel);
}

void UDatasmithMeshElement::SetMaterial(const FString& MaterialName, int32 SlotId)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithMeshElement, MeshElement, );
	return Element->SetMaterial(*MaterialName, SlotId);
}

FString UDatasmithMeshElement::GetMaterial(int32 SlotId)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithMeshElement, MeshElement, FString());
	return Element->GetMaterial(SlotId);
}


bool UDatasmithMeshElement::IsElementValid() const
{
	DATASMITHOBJECTELEMENT_ISELEMENTVALID(MeshElement)
	return true;
}

/**
 * UDatasmithMeshActorElement
 */
void UDatasmithMeshActorElement::AddMaterialOverride(UDatasmithMaterialIDElement* Material)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithMeshActorElement, MeshActorElemement, );
	DATASMITHOBJECTELEMENT_GETARGUMENT_AND_EARLYREATURN(IDatasmithMaterialIDElement, GetDatasmithMaterialIDElement(), , Material, TEXT("MaterialID"));
	Element->AddMaterialOverride(ArgElement);
}

int32 UDatasmithMeshActorElement::GetMaterialOverridesCount() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithMeshActorElement, MeshActorElemement, 0);
	return Element->GetMaterialOverridesCount();
}

TArray<UDatasmithMaterialIDElement*> UDatasmithMeshActorElement::GetMaterials() const
{
	TArray<UDatasmithMaterialIDElement*> Result;
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithMeshActorElement, MeshActorElemement, Result);

	int32 MaterialCount = GetMaterialOverridesCount();
	UDatasmithSceneElementBase* Scene = Cast<UDatasmithSceneElementBase>(GetOuter());
	if (Scene && MaterialCount > 0)
	{
		Result.Reserve(MaterialCount);
		for (int32 Index = 0; Index < MaterialCount; ++Index)
		{
			TSharedPtr<IDatasmithMaterialIDElement> MaterialElement = Element->GetMaterialOverride(Index);
			Result.Add(Scene->FindOrAddElement(MaterialElement));
		}
	}

	return Result;
}

void UDatasmithMeshActorElement::RemoveMaterialOverride(UDatasmithMaterialIDElement* Material)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithMeshActorElement, MeshActorElemement, );
	DATASMITHOBJECTELEMENT_GETARGUMENT_AND_EARLYREATURN(IDatasmithMaterialIDElement, GetDatasmithMaterialIDElement(), , Material, TEXT("MaterialID"));
	Element->RemoveMaterialOverride(ArgElement);
}

FString UDatasmithMeshActorElement::GetStaticMeshPathName() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithMeshActorElement, MeshActorElemement, FString());
	return FString(Element->GetStaticMeshPathName());
}

void UDatasmithMeshActorElement::SetStaticMeshPathName(const FString& InStaticMeshName)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithMeshActorElement, MeshActorElemement, );
	Element->SetStaticMeshPathName(*InStaticMeshName);
}

UDatasmithMeshElement* UDatasmithMeshActorElement::GetMeshElement()
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithMeshActorElement, MeshActorElemement, nullptr);

	UDatasmithSceneElementBase* Scene = Cast<UDatasmithSceneElementBase>(GetOuter());
	if (Scene)
	{
		const FString StaticMeshPathName = Element->GetStaticMeshPathName();
		return Scene->GetMeshByPathName(StaticMeshPathName);
	}
	return nullptr;
}

bool UDatasmithMeshActorElement::IsStaticMeshPathRelative() const
{
	const FString MeshPathName = GetStaticMeshPathName();
	return !MeshPathName.IsEmpty() && FPaths::IsRelative(MeshPathName);
}

FVector UDatasmithMeshActorElement::GetBoundingBoxSize() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithMeshActorElement, MeshActorElemement, FVector::ZeroVector);

	UDatasmithSceneElementBase* Scene = Cast<UDatasmithSceneElementBase>(GetOuter());
	FVector BoundingBoxSize = FVector::ZeroVector;
	bool bFound = false;
	if (Scene)
	{
		const FString StaticMeshPathName = Element->GetStaticMeshPathName();
		UDatasmithMeshElement* MeshElement = Scene->GetMeshByPathName(StaticMeshPathName);
		if (MeshElement)
		{
			bFound = true;
			BoundingBoxSize = MeshElement->GetBoundingBoxSize();
		}
		else if (!IsStaticMeshPathRelative())
		{
			UStaticMesh* StaticMesh = Cast<UStaticMesh>(FSoftObjectPath(StaticMeshPathName).TryLoad());
			if (StaticMesh)
			{
				bFound = true;
				BoundingBoxSize = StaticMesh->GetBoundingBox().GetSize();
			}
		}
	}

	if (!bFound)
	{
		UE_LOGF(LogDatasmithImport, Warning, "The MeshPart was not found.");
		return FVector::ZeroVector;
	}

	FVector ActorScale = Element->GetScale();
	BoundingBoxSize *= ActorScale;
	return BoundingBoxSize;
}

bool UDatasmithMeshActorElement::IsElementValid() const
{
	DATASMITHOBJECTELEMENT_ISELEMENTVALID(MeshActorElemement)
	return true;
}


/**
 * UDatasmithLightActorElement
 */
bool UDatasmithLightActorElement::IsEnabled() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithLightActorElement, LightActorElement, false);
	return Element->IsEnabled();
}

void UDatasmithLightActorElement::SetEnabled(bool bIsEnabled)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithLightActorElement, LightActorElement, );
	return Element->SetEnabled(bIsEnabled);
}

float UDatasmithLightActorElement::GetIntensity() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithLightActorElement, LightActorElement, 0.f);
	return Element->GetIntensity();
}

void UDatasmithLightActorElement::SetIntensity(float Intensity)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithLightActorElement, LightActorElement, );
	return Element->SetIntensity(Intensity);
}

FLinearColor UDatasmithLightActorElement::GetColor() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithLightActorElement, LightActorElement, FLinearColor::Black);
	return Element->GetColor();
}

void UDatasmithLightActorElement::SetColor(FLinearColor Color)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithLightActorElement, LightActorElement, );
	return Element->SetColor(Color);
}

float UDatasmithLightActorElement::GetTemperature() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithLightActorElement, LightActorElement, 0.f);
	return Element->GetTemperature();
}

void UDatasmithLightActorElement::SetTemperature(float Temperature)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithLightActorElement, LightActorElement, );
	return Element->SetTemperature(Temperature);
}

bool UDatasmithLightActorElement::GetUseTemperature() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithLightActorElement, LightActorElement, false);
	return Element->GetUseTemperature();
}

void UDatasmithLightActorElement::SetUseTemperature(bool bUseTemperature)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithLightActorElement, LightActorElement, );
	return Element->SetUseTemperature(bUseTemperature);
}

FString UDatasmithLightActorElement::GetIesFile() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithLightActorElement, LightActorElement, FString());
	return Element->GetIesFile();
}

void UDatasmithLightActorElement::SetIesFile(const FString& IesFile)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithLightActorElement, LightActorElement, );
	return Element->SetIesFile(*IesFile);
}

bool UDatasmithLightActorElement::GetUseIes() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithLightActorElement, LightActorElement, false);
	return Element->GetUseIes();
}

void UDatasmithLightActorElement::SetUseIes(bool bUseIes)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithLightActorElement, LightActorElement, );
	return Element->SetUseIes(bUseIes);
}

float UDatasmithLightActorElement::GetIesBrightnessScale() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithLightActorElement, LightActorElement, 0.f);
	return Element->GetIesBrightnessScale();
}

void UDatasmithLightActorElement::SetIesBrightnessScale(float IesBrightnessScale)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithLightActorElement, LightActorElement, );
	return Element->SetIesBrightnessScale(IesBrightnessScale);
}

bool UDatasmithLightActorElement::GetUseIesBrightness() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithLightActorElement, LightActorElement, false);
	return Element->GetUseIesBrightness();
}

void UDatasmithLightActorElement::SetUseIesBrightness(bool bUseIesBrightness)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithLightActorElement, LightActorElement, );
	return Element->SetUseIesBrightness(bUseIesBrightness);
}

bool UDatasmithLightActorElement::IsElementValid() const
{
	DATASMITHOBJECTELEMENT_ISELEMENTVALID(LightActorElement)
	return true;
}

/**
 * UDatasmithCameraActorElement
 */
float UDatasmithCameraActorElement::GetSensorWidth() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithCameraActorElement, CameraActorElement, 0.f);
	return Element->GetSensorWidth();
}

void UDatasmithCameraActorElement::SetSensorWidth(float SensorWidth)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithCameraActorElement, CameraActorElement, );
	return Element->SetSensorWidth(SensorWidth);
}

float UDatasmithCameraActorElement::GetSensorAspectRatio() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithCameraActorElement, CameraActorElement, 0.f);
	return Element->GetSensorAspectRatio();
}

void UDatasmithCameraActorElement::SetSensorAspectRatio(float SensorAspectRatio)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithCameraActorElement, CameraActorElement, );
	return Element->SetSensorAspectRatio(SensorAspectRatio);
}

float UDatasmithCameraActorElement::GetFocusDistance() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithCameraActorElement, CameraActorElement, 0.f);
	return Element->GetFocusDistance();
}

void UDatasmithCameraActorElement::SetFocusDistance(float FocusDistance)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithCameraActorElement, CameraActorElement, );
	return Element->SetFocusDistance(FocusDistance);
}

float UDatasmithCameraActorElement::GetFStop() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithCameraActorElement, CameraActorElement, 0.f);
	return Element->GetFStop();
}

void UDatasmithCameraActorElement::SetFStop(float FStop)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithCameraActorElement, CameraActorElement, );
	return Element->SetFStop(FStop);
}

float UDatasmithCameraActorElement::GetFocalLength() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithCameraActorElement, CameraActorElement, 0.f);
	return Element->GetFocalLength();
}

void UDatasmithCameraActorElement::SetFocalLength(float FocalLength)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithCameraActorElement, CameraActorElement, );
	return Element->SetFocalLength(FocalLength);
}

FString UDatasmithCameraActorElement::GetLookAtActor() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithCameraActorElement, CameraActorElement, FString());
	return Element->GetLookAtActor();
}

void UDatasmithCameraActorElement::SetLookAtActor(const FString& ActorPathName)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithCameraActorElement, CameraActorElement, );
	return Element->SetLookAtActor(*ActorPathName);
}

bool UDatasmithCameraActorElement::GetLookAtAllowRoll() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithCameraActorElement, CameraActorElement, false);
	return Element->GetLookAtAllowRoll();
}

void UDatasmithCameraActorElement::SetLookAtAllowRoll(bool bAllow)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithCameraActorElement, CameraActorElement, );
	return Element->SetLookAtAllowRoll(bAllow);
}

/**
 * UDatasmithCustomActorElement
 */
FString UDatasmithCustomActorElement::GetClassOrPathName() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithCustomActorElement, CustomActorElement, TEXT(""));
	return Element->GetClassOrPathName();
}

void UDatasmithCustomActorElement::SetClassOrPathName(const FString& InPathName)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithCustomActorElement, CustomActorElement,);
	Element->SetClassOrPathName( *InPathName );
}

int32 UDatasmithCustomActorElement::GetPropertiesCount() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithCustomActorElement, CustomActorElement, 0);
	return Element->GetPropertiesCount();
}

UDatasmithKeyValueProperty* UDatasmithCustomActorElement::GetProperty(int32 i)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithCustomActorElement, CustomActorElement, nullptr);
	return FindOrAddElement( Element->GetProperty(i) );
}

UDatasmithKeyValueProperty* UDatasmithCustomActorElement::GetPropertyByName(const FString& InName)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithCustomActorElement, CustomActorElement, nullptr);
	return FindOrAddElement( Element->GetPropertyByName( *InName ) );
}

void UDatasmithCustomActorElement::AddProperty(UDatasmithKeyValueProperty* Property)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithCustomActorElement, CustomActorElement,);

	TSharedPtr< IDatasmithKeyValueProperty > PropertyElement = Property->GetDatasmithKeyValueProperty().Pin();

	if ( PropertyElement.IsValid() )
	{
		Element->AddProperty( PropertyElement );
	}
}

void UDatasmithCustomActorElement::RemoveProperty(UDatasmithKeyValueProperty* Property)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithCustomActorElement, CustomActorElement,);

	TSharedPtr< IDatasmithKeyValueProperty > PropertyElement = Property->GetDatasmithKeyValueProperty().Pin();

	if ( PropertyElement.IsValid() )
	{
		Element->RemoveProperty( PropertyElement );
	}
}

bool UDatasmithCustomActorElement::IsElementValid() const
{
	DATASMITHOBJECTELEMENT_ISELEMENTVALID(CustomActorElement);
	return true;
}

bool UDatasmithCustomActorElement::IsElementValid(const TWeakPtr<IDatasmithKeyValueProperty>& Element) const
{
	const TSharedPtr<IDatasmithKeyValueProperty> KeyValueProperty = Element.Pin();

	if ( KeyValueProperty.IsValid() )
	{
		const TSharedPtr< IDatasmithCustomActorElement > CustomActor = CustomActorElement.Pin();

		if ( CustomActor.IsValid() )
		{
			for ( int32 i = 0; i < CustomActor->GetPropertiesCount(); ++i )
			{
				if ( CustomActor->GetProperty(i) == KeyValueProperty )
				{
					return true;
				}
			}
		}
	}

	return false;
}

UDatasmithKeyValueProperty* UDatasmithCustomActorElement::FindOrAddElement(const TSharedPtr<IDatasmithKeyValueProperty>& InElement)
{
	return FDatasmithUElementsUtils::FindOrAddElement(this, Properties, InElement, [&InElement](UDatasmithKeyValueProperty* Element)
	{
		Element->SetDatasmithKeyValueProperty(InElement);
	});
}

UDatasmithPostProcessElement* UDatasmithCameraActorElement::GetPostProcess()
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithCameraActorElement, CameraActorElement, nullptr);

	UDatasmithSceneElementBase* Scene = Cast<UDatasmithSceneElementBase>(GetOuter());
	if (Scene)
	{
		TSharedPtr<IDatasmithPostProcessElement> PostProcessElement = Element->GetPostProcess();
		return Scene->FindOrAddElement(PostProcessElement);
	}
	return nullptr;
}

bool UDatasmithCameraActorElement::IsElementValid() const
{
	DATASMITHOBJECTELEMENT_ISELEMENTVALID(CameraActorElement);
	return true;
}


/**
 * UDatasmithBaseMaterialElement
 **/

bool UDatasmithBaseMaterialElement::IsElementValid() const
{
	DATASMITHOBJECTELEMENT_ISELEMENTVALID(BaseMaterialElemement);
	return true;
}

/**
 * UDatasmithMaterialIDElement
 **/
int32 UDatasmithMaterialIDElement::GetId() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithMaterialIDElement, MaterialElemement, 0);
	return Element->GetId();
}

bool UDatasmithMaterialIDElement::IsElementValid() const
{
	DATASMITHOBJECTELEMENT_ISELEMENTVALID(MaterialElemement);
	return true;
}


/**
 * UDatasmithPostProcessElement
 */
float UDatasmithPostProcessElement::GetTemperature() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithPostProcessElement, PostProcessElemement, 0.f);
	return Element->GetTemperature();
}

void UDatasmithPostProcessElement::SetTemperature(float InValue)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithPostProcessElement, PostProcessElemement, );
	return Element->SetTemperature(InValue);
}

FLinearColor UDatasmithPostProcessElement::GetColorFilter() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithPostProcessElement, PostProcessElemement, FLinearColor::Black);
	return Element->GetColorFilter();
}

void UDatasmithPostProcessElement::SetColorFilter(FLinearColor InValue)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithPostProcessElement, PostProcessElemement, );
	return Element->SetColorFilter(InValue);
}

float UDatasmithPostProcessElement::GetVignette() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithPostProcessElement, PostProcessElemement, 0.f);
	return Element->GetVignette();
}

void UDatasmithPostProcessElement::SetVignette(float InValue)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithPostProcessElement, PostProcessElemement, );
	return Element->SetVignette(InValue);
}

float UDatasmithPostProcessElement::GetDof() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithPostProcessElement, PostProcessElemement, 0.f);
	return Element->GetDof();
}

void UDatasmithPostProcessElement::SetDof(float InValue)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithPostProcessElement, PostProcessElemement, );
	return Element->SetDof(InValue);
}

float UDatasmithPostProcessElement::GetMotionBlur() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithPostProcessElement, PostProcessElemement, 0.f);
	return Element->GetMotionBlur();
}

void UDatasmithPostProcessElement::SetMotionBlur(float InValue)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithPostProcessElement, PostProcessElemement, );
	return Element->SetMotionBlur(InValue);
}

float UDatasmithPostProcessElement::GetSaturation() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithPostProcessElement, PostProcessElemement, 0.f);
	return Element->GetSaturation();
}

void UDatasmithPostProcessElement::SetSaturation(float InValue)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithPostProcessElement, PostProcessElemement, );
	return Element->SetSaturation(InValue);
}

float UDatasmithPostProcessElement::GetCameraISO() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithPostProcessElement, PostProcessElemement, -1.f);
	return Element->GetCameraISO();
}

void UDatasmithPostProcessElement::SetCameraISO(float CameraISO)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithPostProcessElement, PostProcessElemement, );
	return Element->SetCameraISO(CameraISO);
}

float UDatasmithPostProcessElement::GetCameraShutterSpeed() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithPostProcessElement, PostProcessElemement, -1.f);
	return Element->GetCameraShutterSpeed();
}

void UDatasmithPostProcessElement::SetCameraShutterSpeed(float CameraShutterSpeed)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithPostProcessElement, PostProcessElemement, );
	return Element->SetCameraShutterSpeed(CameraShutterSpeed);
}

bool UDatasmithPostProcessElement::IsElementValid() const
{
	DATASMITHOBJECTELEMENT_ISELEMENTVALID(PostProcessElemement);
	return true;
}

/**
 * UDatasmithTextureElement
 */
FString UDatasmithTextureElement::GetFile() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithTextureElement, TextureElemement, FString());
	return FString(Element->GetFile());
}

void UDatasmithTextureElement::SetFile(const FString& InValue)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithTextureElement, TextureElemement, );
	return Element->SetFile(*InValue);
}

EDatasmithTextureMode UDatasmithTextureElement::GetTextureMode() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithTextureElement, TextureElemement, EDatasmithTextureMode::Other);
	return Element->GetTextureMode();
}

void UDatasmithTextureElement::SetTextureMode(EDatasmithTextureMode InValue)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithTextureElement, TextureElemement, );
	return Element->SetTextureMode(InValue);
}

bool UDatasmithTextureElement::GetAllowResize() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithTextureElement, TextureElemement, false);
	return Element->GetAllowResize();
}

void UDatasmithTextureElement::SetAllowResize(bool InValue)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithTextureElement, TextureElemement, );
	return Element->SetAllowResize(InValue);
}

float UDatasmithTextureElement::GetRGBCurve() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithTextureElement, TextureElemement, 0.f);
	return Element->GetRGBCurve();
}

void UDatasmithTextureElement::SetRGBCurve(float InValue)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithTextureElement, TextureElemement, );
	return Element->SetRGBCurve(InValue);
}

EDatasmithColorSpace UDatasmithTextureElement::GetColorSpace() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithTextureElement, TextureElemement, EDatasmithColorSpace::Default);
	return Element->GetSRGB();
}

void UDatasmithTextureElement::SetColorSpace(EDatasmithColorSpace Option)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithTextureElement, TextureElemement, );
	return Element->SetSRGB(Option);
}

bool UDatasmithTextureElement::IsElementValid() const
{
	DATASMITHOBJECTELEMENT_ISELEMENTVALID(TextureElemement);
	return true;
}

/**
 * UDatasmithKeyValueProperty
 */
EDatasmithKeyValuePropertyType UDatasmithKeyValueProperty::GetPropertyType() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithKeyValueProperty, KeyValueProperty, EDatasmithKeyValuePropertyType::Bool);
	return Element->GetPropertyType();
}

void UDatasmithKeyValueProperty::SetPropertyType(EDatasmithKeyValuePropertyType InType)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithKeyValueProperty, KeyValueProperty, );
	Element->SetPropertyType(InType);
}

FString UDatasmithKeyValueProperty::GetValue() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithKeyValueProperty, KeyValueProperty, TEXT(""));
	return Element->GetValue();
}

void UDatasmithKeyValueProperty::SetValue(const FString& Value)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithKeyValueProperty, KeyValueProperty, );
	Element->SetValue(*Value);
}

bool UDatasmithKeyValueProperty::IsElementValid() const
{
	if ( !Super::IsElementValid() )
	{
		return false;
	}

	UDatasmithMetaDataElement* MetaData = Cast<UDatasmithMetaDataElement>( GetOuter() );
	UDatasmithCustomActorElement* CustomActor = Cast<UDatasmithCustomActorElement>( GetOuter() );

	if ( ( !MetaData || !MetaData->IsElementValid( KeyValueProperty ) ) &&
		 ( !CustomActor || !CustomActor->IsElementValid( KeyValueProperty ) ) )
	{
		return false;
	}

	return true;
}

/**
 * UDatasmithMetaDataElement
 */
int32 UDatasmithMetaDataElement::GetPropertiesCount() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithMetaDataElement, MetaDataElement, 0);
	return Element->GetPropertiesCount();
}

UDatasmithKeyValueProperty* UDatasmithMetaDataElement::GetProperty(int32 i)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithMetaDataElement, MetaDataElement, nullptr);
	return FindOrAddElement( Element->GetProperty(i) );
}

UDatasmithKeyValueProperty* UDatasmithMetaDataElement::GetPropertyByName(const FString& InName)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithMetaDataElement, MetaDataElement, nullptr);
	return FindOrAddElement( Element->GetPropertyByName( *InName ) );
}

UDatasmithObjectElement* UDatasmithMetaDataElement::GetAssociatedElement() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithMetaDataElement, MetaDataElement, nullptr);
	UDatasmithSceneElementBase* Scene = Cast<UDatasmithSceneElementBase>(GetOuter());
	if (Scene)
	{
		const TSharedPtr<IDatasmithElement>& AssociatedElement = Element->GetAssociatedElement();
		return Scene->FindOrAddElement(AssociatedElement);
	}
	return nullptr;
}

void UDatasmithMetaDataElement::GetProperties(TArray<FString>& OutKeys, TArray<FString>& OutValues)
{
	OutKeys.Reset();
	OutValues.Reset();

	int32 NumProperties = GetPropertiesCount();
	for (int32 i = 0; i < NumProperties; ++i)
	{
		if (UDatasmithKeyValueProperty* Property = GetProperty(i))
		{
			TSharedPtr<IDatasmithKeyValueProperty> Kvp = Property->GetDatasmithKeyValueProperty().Pin();
			if (Kvp.IsValid())
			{
				OutKeys.Add(Kvp->GetName());
				OutValues.Add(Kvp->GetValue());
			}
		}
	}
}

bool UDatasmithMetaDataElement::IsElementValid() const
{
	DATASMITHOBJECTELEMENT_ISELEMENTVALID(MetaDataElement);
	return true;
}

bool UDatasmithMetaDataElement::IsElementValid(const TWeakPtr<IDatasmithKeyValueProperty>& Element) const
{
	const TSharedPtr<IDatasmithKeyValueProperty> KeyValueProperty = Element.Pin();

	if ( KeyValueProperty.IsValid() )
	{
		const TSharedPtr<IDatasmithMetaDataElement > MetaData = MetaDataElement.Pin();

		if ( MetaData.IsValid() )
		{
			for ( int32 i = 0; i < MetaData->GetPropertiesCount(); ++i )
			{
				if ( MetaData->GetProperty(i) == KeyValueProperty )
				{
					return true;
				}
			}
		}
	}

	return false;
}

UDatasmithKeyValueProperty* UDatasmithMetaDataElement::FindOrAddElement(const TSharedPtr<IDatasmithKeyValueProperty>& InElement)
{
	return FDatasmithUElementsUtils::FindOrAddElement(this, Properties, InElement, [&InElement](UDatasmithKeyValueProperty* Element)
	{
		Element->SetDatasmithKeyValueProperty(InElement);
	});
}

/*
 * UDatasmithBasePropertyCaptureElement
 */
void UDatasmithBasePropertyCaptureElement::SetPropertyPath(const FString& Path) const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithBasePropertyCaptureElement, DatasmithElement, );
	Element->SetPropertyPath(Path);
}

FString UDatasmithBasePropertyCaptureElement::GetPropertyPath() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithBasePropertyCaptureElement, DatasmithElement, FString());
	return Element->GetPropertyPath();
}

void UDatasmithBasePropertyCaptureElement::SetCategory(EDatasmithPropertyCategory Category)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithBasePropertyCaptureElement, DatasmithElement, );
	Element->SetCategory(Category);
}

EDatasmithPropertyCategory UDatasmithBasePropertyCaptureElement::GetCategory() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithBasePropertyCaptureElement, DatasmithElement, EDatasmithPropertyCategory::Undefined);
	return Element->GetCategory();
}

bool UDatasmithBasePropertyCaptureElement::IsElementValid() const
{
	DATASMITHOBJECTELEMENT_ISELEMENTVALID(DatasmithElement);
	return true;
}

/*
 * UDatasmithPropertyCaptureElement
 */
template<typename T>
T GetValueImpl(TWeakPtr<IDatasmithBasePropertyCaptureElement>& Element)
{
	T Result{};

	TSharedPtr<IDatasmithPropertyCaptureElement> PinnedElement = StaticCastSharedPtr<IDatasmithPropertyCaptureElement>(Element.Pin());
	if (PinnedElement.IsValid())
	{
		const TArray<uint8>& Bytes = PinnedElement->GetRecordedData();
		if (Bytes.Num() == sizeof(T))
		{
			FMemory::Memcpy(&Result, Bytes.GetData(), sizeof(T));
		}
	}

	return Result;
}
template<typename T>
void SetPropertyValueImpl(TWeakPtr<IDatasmithBasePropertyCaptureElement>& Element, const T& InValue)
{
	TSharedPtr<IDatasmithPropertyCaptureElement> PinnedElement = StaticCastSharedPtr<IDatasmithPropertyCaptureElement>(Element.Pin());
	if (PinnedElement.IsValid())
	{
		PinnedElement->SetRecordedData((uint8*)&InValue, sizeof(T));
	}
}

bool UDatasmithPropertyCaptureElement::GetValueBool()
{
	return GetValueImpl<bool>(DatasmithElement);
}

void UDatasmithPropertyCaptureElement::SetValueBool(bool InValue)
{
	SetPropertyValueImpl(DatasmithElement, InValue);
}

int32 UDatasmithPropertyCaptureElement::GetValueInt()
{
	return GetValueImpl<int32>(DatasmithElement);
}

void UDatasmithPropertyCaptureElement::SetValueInt(int32 InValue)
{
	SetPropertyValueImpl(DatasmithElement, InValue);
}

float UDatasmithPropertyCaptureElement::GetValueFloat()
{
	return GetValueImpl<float>(DatasmithElement);
}

void UDatasmithPropertyCaptureElement::SetValueFloat(float InValue)
{
	SetPropertyValueImpl(DatasmithElement, InValue);
}

FString UDatasmithPropertyCaptureElement::GetValueString()
{
	return GetValueImpl<FString>(DatasmithElement);
}

void UDatasmithPropertyCaptureElement::SetValueString(const FString& InValue)
{
	SetPropertyValueImpl(DatasmithElement, InValue);
}

FRotator UDatasmithPropertyCaptureElement::GetValueRotator()
{
	return GetValueImpl<FRotator>(DatasmithElement);
}

void UDatasmithPropertyCaptureElement::SetValueRotator(FRotator InValue)
{
	SetPropertyValueImpl(DatasmithElement, InValue);
}

FColor UDatasmithPropertyCaptureElement::GetValueColor()
{
	return GetValueImpl<FColor>(DatasmithElement);
}

void UDatasmithPropertyCaptureElement::SetValueColor(FColor InValue)
{
	SetPropertyValueImpl(DatasmithElement, InValue);
}

FLinearColor UDatasmithPropertyCaptureElement::GetValueLinearColor()
{
	return GetValueImpl<FLinearColor>(DatasmithElement);
}

void UDatasmithPropertyCaptureElement::SetValueLinearColor(FLinearColor InValue)
{
	SetPropertyValueImpl(DatasmithElement, InValue);
}

FVector UDatasmithPropertyCaptureElement::GetValueVector()
{
	return GetValueImpl<FVector>(DatasmithElement);
}

void UDatasmithPropertyCaptureElement::SetValueVector(FVector InValue)
{
	SetPropertyValueImpl(DatasmithElement, InValue);
}

FQuat UDatasmithPropertyCaptureElement::GetValueQuat()
{
	return GetValueImpl<FQuat>(DatasmithElement);
}

void UDatasmithPropertyCaptureElement::SetValueQuat(FQuat InValue)
{
	SetPropertyValueImpl(DatasmithElement, InValue);
}

FVector4 UDatasmithPropertyCaptureElement::GetValueVector4()
{
	return GetValueImpl<FVector4>(DatasmithElement);
}

void UDatasmithPropertyCaptureElement::SetValueVector4(FVector4 InValue)
{
	SetPropertyValueImpl(DatasmithElement, InValue);
}

FVector2D UDatasmithPropertyCaptureElement::GetValueVector2D()
{
	return GetValueImpl<FVector2D>(DatasmithElement);
}

void UDatasmithPropertyCaptureElement::SetValueVector2D(FVector2D InValue)
{
	SetPropertyValueImpl(DatasmithElement, InValue);
}

FIntPoint UDatasmithPropertyCaptureElement::GetValueIntPoint()
{
	return GetValueImpl<FIntPoint>(DatasmithElement);
}

void UDatasmithPropertyCaptureElement::SetValueIntPoint(FIntPoint InValue)
{
	SetPropertyValueImpl(DatasmithElement, InValue);
}

/*
 * UDatasmithObjectPropertyCaptureElement
 */
void UDatasmithObjectPropertyCaptureElement::SetRecordedObject(UDatasmithObjectElement* Object)
{
	if (Object)
	{
		DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithBasePropertyCaptureElement, DatasmithElement, );
		TSharedPtr<IDatasmithObjectPropertyCaptureElement> ObjectPropertyElement = StaticCastSharedPtr<IDatasmithObjectPropertyCaptureElement>(Element);
		ObjectPropertyElement->SetRecordedObject(Object->GetIDatasmithElement());
	}
}

UDatasmithObjectElement* UDatasmithObjectPropertyCaptureElement::GetRecordedObject() const
{
	if (UDatasmithSceneElementBase* Scene = GetTypedOuter<UDatasmithSceneElementBase>())
	{
		DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithBasePropertyCaptureElement, DatasmithElement, nullptr);
		TSharedPtr<IDatasmithObjectPropertyCaptureElement> ObjectPropertyElement = StaticCastSharedPtr<IDatasmithObjectPropertyCaptureElement>(Element);
		TSharedPtr<IDatasmithElement> ObjectElement = ObjectPropertyElement->GetRecordedObject().Pin();
		if (ObjectElement.IsValid())
		{
			return Scene->FindOrAddElement(ObjectElement);
		}
	}
	return nullptr;
}

/*
 * UDatasmithActorBindingElement
 */
void UDatasmithActorBindingElement::SetActor(UDatasmithActorElement* Actor)
{
	if (Actor)
	{
		TSharedPtr<IDatasmithActorElement> DatasmithActorElement = Actor->GetIDatasmithActorElement().Pin();
		if (DatasmithActorElement.IsValid())
		{
			DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithActorBindingElement, DatasmithElement, );
			Element->SetActor(DatasmithActorElement);
		}
	}
}

UDatasmithActorElement* UDatasmithActorBindingElement::GetActor() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithActorBindingElement, DatasmithElement, nullptr);
	if (UDatasmithSceneElementBase* Scene = GetTypedOuter<UDatasmithSceneElementBase>())
	{
		TSharedPtr<IDatasmithActorElement> ObjectElement = Element->GetActor();
		if (ObjectElement.IsValid())
		{
			return Scene->FindOrAddActorElement(ObjectElement);
		}
	}
	return nullptr;
}

UDatasmithPropertyCaptureElement* UDatasmithActorBindingElement::CreatePropertyCapture()
{
	TSharedPtr<IDatasmithActorBindingElement> ParentElement = GetActorBindingElement().Pin();
	if (!ParentElement.IsValid())
	{
		return nullptr;
	}

	if (UDatasmithSceneElementBase* Scene = GetTypedOuter<UDatasmithSceneElementBase>())
	{
		TSharedPtr<IDatasmithPropertyCaptureElement> Element = FDatasmithSceneFactory::CreatePropertyCapture();
		ParentElement->AddPropertyCapture(Element.ToSharedRef());

		return Scene->FindOrAddElement(Element);
	}
	return nullptr;
}

UDatasmithObjectPropertyCaptureElement* UDatasmithActorBindingElement::CreateObjectPropertyCapture()
{
	TSharedPtr<IDatasmithActorBindingElement> ParentElement = GetActorBindingElement().Pin();
	if (!ParentElement.IsValid())
	{
		return nullptr;
	}

	if (UDatasmithSceneElementBase* Scene = GetTypedOuter<UDatasmithSceneElementBase>())
	{
		TSharedPtr<IDatasmithObjectPropertyCaptureElement> Element = FDatasmithSceneFactory::CreateObjectPropertyCapture();
		ParentElement->AddPropertyCapture(Element.ToSharedRef());

		return Scene->FindOrAddElement(Element);
	}
	return nullptr;
}

void UDatasmithActorBindingElement::AddPropertyCapture(const UDatasmithBasePropertyCaptureElement* Property)
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithActorBindingElement, DatasmithElement, );
	if (Property)
	{
		TSharedPtr<IDatasmithBasePropertyCaptureElement> PropElement = Property->GetBasePropertyCaptureElement().Pin();
		if (PropElement.IsValid())
		{
			Element->AddPropertyCapture(PropElement.ToSharedRef());
		}
	}
}

int32 UDatasmithActorBindingElement::GetPropertyCapturesCount() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithActorBindingElement, DatasmithElement, INDEX_NONE);
	return Element->GetPropertyCapturesCount();
}

UDatasmithBasePropertyCaptureElement* UDatasmithActorBindingElement::GetPropertyCapture(int32 Index)
{
	if (UDatasmithSceneElementBase* Scene = GetTypedOuter<UDatasmithSceneElementBase>())
	{
		DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithActorBindingElement, DatasmithElement, nullptr);
		TSharedPtr<IDatasmithBasePropertyCaptureElement> PropertyElement = Element->GetPropertyCapture(Index);

		if (PropertyElement->IsSubType(EDatasmithElementVariantSubType::PropertyCapture))
		{
			return Scene->FindOrAddElement(StaticCastSharedPtr<IDatasmithPropertyCaptureElement>(PropertyElement));
		}
		else if (PropertyElement->IsSubType(EDatasmithElementVariantSubType::ObjectPropertyCapture))
		{
			return Scene->FindOrAddElement(StaticCastSharedPtr<IDatasmithObjectPropertyCaptureElement>(PropertyElement));
		}
	}
	return nullptr;
}

void UDatasmithActorBindingElement::RemovePropertyCapture(const UDatasmithBasePropertyCaptureElement* Property)
{
	if (Property)
	{
		TSharedPtr<IDatasmithBasePropertyCaptureElement> PropertyElement = Property->GetBasePropertyCaptureElement().Pin();
		if (PropertyElement.IsValid())
		{
			DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithActorBindingElement, DatasmithElement, );
			Element->RemovePropertyCapture(PropertyElement.ToSharedRef());
		}
	}
}

bool UDatasmithActorBindingElement::IsElementValid() const
{
	DATASMITHOBJECTELEMENT_ISELEMENTVALID(DatasmithElement);
	return true;
}

UDatasmithActorBindingElement* UDatasmithVariantElement::CreateActorBinding()
{
	TSharedPtr<IDatasmithVariantElement> ParentElement = GetVariantElement().Pin();
	if (!ParentElement.IsValid())
	{
		return nullptr;
	}

	if (UDatasmithSceneElementBase* Scene = GetTypedOuter<UDatasmithSceneElementBase>())
	{
		TSharedPtr<IDatasmithActorBindingElement> Element = FDatasmithSceneFactory::CreateActorBinding();
		ParentElement->AddActorBinding(Element.ToSharedRef());

		return Scene->FindOrAddElement(Element);
	}
	return nullptr;
}

/*
 * UDatasmithVariantElement
 */
void UDatasmithVariantElement::AddActorBinding(const UDatasmithActorBindingElement* Binding)
{
	if (Binding)
	{
		TSharedPtr<IDatasmithActorBindingElement> BindingElement = Binding->GetActorBindingElement().Pin();
		if (BindingElement.IsValid())
		{
			DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithVariantElement, DatasmithElement, );
			Element->AddActorBinding(BindingElement.ToSharedRef());
		}
	}
}

int32 UDatasmithVariantElement::GetActorBindingsCount() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithVariantElement, DatasmithElement, INDEX_NONE);
	return Element->GetActorBindingsCount();
}

UDatasmithActorBindingElement* UDatasmithVariantElement::GetActorBinding(int32 Index)
{
	if (UDatasmithSceneElementBase* Scene = GetTypedOuter<UDatasmithSceneElementBase>())
	{
		DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithVariantElement, DatasmithElement, nullptr);
		TSharedPtr<IDatasmithActorBindingElement> BindingElement = Element->GetActorBinding(Index);
		return Scene->FindOrAddElement(BindingElement);
	}
	return nullptr;
}

void UDatasmithVariantElement::RemoveActorBinding(const UDatasmithActorBindingElement* Binding)
{
	if (Binding)
	{
		TSharedPtr<IDatasmithActorBindingElement> BindingElement = Binding->GetActorBindingElement().Pin();
		if (BindingElement.IsValid())
		{
			DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithVariantElement, DatasmithElement, );
			Element->RemoveActorBinding(BindingElement.ToSharedRef());
		}
	}
}

bool UDatasmithVariantElement::IsElementValid() const
{
	DATASMITHOBJECTELEMENT_ISELEMENTVALID(DatasmithElement);
	return true;
}

UDatasmithVariantElement* UDatasmithVariantSetElement::CreateVariant(FName InElementName)
{
	// Get parent element
	TSharedPtr<IDatasmithVariantSetElement> ParentElement = GetVariantSetElement().Pin();
	if (!ParentElement.IsValid())
	{
		return nullptr;
	}

	if (UDatasmithSceneElementBase* Scene = GetTypedOuter<UDatasmithSceneElementBase>())
	{
		// Use default name
		if (InElementName == NAME_None)
		{
			InElementName = FName(TEXT("Variant"));
		}

		// Find unique Element Name
		FDatasmithUniqueNameProvider NameProvider;
		NameProvider.Reserve(ParentElement->GetVariantsCount());
		FDatasmithUElementsUtils::ForVariantElement<IDatasmithVariantElement>(ParentElement, [&](TSharedPtr<IDatasmithVariantElement> Variant)
		{
			NameProvider.AddExistingName(FString(Variant->GetName()));
			return true;
		});
		FString UniqueElementName = NameProvider.GenerateUniqueName(InElementName.ToString());

		// Create the Element with the unique name
		TSharedPtr<IDatasmithVariantElement> Element = FDatasmithSceneFactory::CreateVariant(*UniqueElementName);
		ParentElement->AddVariant(Element.ToSharedRef());

		return Scene->FindOrAddElement(Element);
	}
	return nullptr;
}

/*
 * UDatasmithVariantSetElement
 */
void UDatasmithVariantSetElement::AddVariant(const UDatasmithVariantElement* Variant)
{
	if (Variant)
	{
		TSharedPtr<IDatasmithVariantElement> VariantElement = Variant->GetVariantElement().Pin();
		if (VariantElement.IsValid())
		{
			DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithVariantSetElement, DatasmithElement, );
			Element->AddVariant(VariantElement.ToSharedRef());
		}
	}
}

int32 UDatasmithVariantSetElement::GetVariantsCount() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithVariantSetElement, DatasmithElement, INDEX_NONE);
	return Element->GetVariantsCount();
}

UDatasmithVariantElement* UDatasmithVariantSetElement::GetVariant(int32 Index)
{
	if (UDatasmithSceneElementBase* Scene = GetTypedOuter<UDatasmithSceneElementBase>())
	{
		DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithVariantSetElement, DatasmithElement, nullptr);
		TSharedPtr<IDatasmithVariantElement> VariantElement = Element->GetVariant(Index);
		return Scene->FindOrAddElement(VariantElement);
	}
	return nullptr;
}

void UDatasmithVariantSetElement::RemoveVariant(const UDatasmithVariantElement* Variant)
{
	if (Variant)
	{
		TSharedPtr<IDatasmithVariantElement> VariantElement = Variant->GetVariantElement().Pin();
		if (VariantElement.IsValid())
		{
			DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithVariantSetElement, DatasmithElement, );
			Element->RemoveVariant(VariantElement.ToSharedRef());
		}
	}
}

bool UDatasmithVariantSetElement::IsElementValid() const
{
	DATASMITHOBJECTELEMENT_ISELEMENTVALID(DatasmithElement);
	return true;
}

UDatasmithVariantSetElement* UDatasmithLevelVariantSetsElement::CreateVariantSet(FName InElementName)
{
	// Get parent element
	TSharedPtr<IDatasmithLevelVariantSetsElement> ParentElement = GetLevelVariantSetsElement().Pin();
	if (!ParentElement.IsValid())
	{
		return nullptr;
	}

	if (UDatasmithSceneElementBase* Scene = GetTypedOuter<UDatasmithSceneElementBase>())
	{
		// Use default name
		if (InElementName == NAME_None)
		{
			InElementName = FName(TEXT("VariantSet"));
		}

		// Find unique Element Name
		FDatasmithUniqueNameProvider NameProvider;
		NameProvider.Reserve(ParentElement->GetVariantSetsCount());
		FDatasmithUElementsUtils::ForVariantElement<IDatasmithVariantSetElement>(ParentElement, [&](TSharedPtr<IDatasmithVariantSetElement> VariantSet)
		{
			NameProvider.AddExistingName(FString(VariantSet->GetName()));
			return true;
		});
		FString UniqueElementName = NameProvider.GenerateUniqueName(InElementName.ToString());

		// Create the Element with the unique name
		TSharedPtr<IDatasmithVariantSetElement> Element = FDatasmithSceneFactory::CreateVariantSet(*UniqueElementName);
		ParentElement->AddVariantSet(Element.ToSharedRef());

		return Scene->FindOrAddElement(Element);
	}
	return nullptr;
}

/*
 * UDatasmithLevelVariantSetsElement
 */
void UDatasmithLevelVariantSetsElement::AddVariantSet(const UDatasmithVariantSetElement* VariantSet)
{
	if (VariantSet)
	{
		TSharedPtr<IDatasmithVariantSetElement> VariantSetElement = VariantSet->GetVariantSetElement().Pin();
		if (VariantSetElement.IsValid())
		{
			DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithLevelVariantSetsElement, DatasmithElement, );
			Element->AddVariantSet(VariantSetElement.ToSharedRef());
		}
	}
}

int32 UDatasmithLevelVariantSetsElement::GetVariantSetsCount() const
{
	DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithLevelVariantSetsElement, DatasmithElement, INDEX_NONE);
	return Element->GetVariantSetsCount();
}

UDatasmithVariantSetElement* UDatasmithLevelVariantSetsElement::GetVariantSet(int32 Index)
{
	if (UDatasmithSceneElementBase* Scene = GetTypedOuter<UDatasmithSceneElementBase>())
	{
		DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithLevelVariantSetsElement, DatasmithElement, nullptr);
		TSharedPtr<IDatasmithVariantSetElement> VariantSetElement = Element->GetVariantSet(Index);
		return Scene->FindOrAddElement(VariantSetElement);
	}
	return nullptr;
}

void UDatasmithLevelVariantSetsElement::RemoveVariantSet(const UDatasmithVariantSetElement* VariantSet)
{
	if (VariantSet)
	{
		TSharedPtr<IDatasmithVariantSetElement> VariantSetElement = VariantSet->GetVariantSetElement().Pin();
		if (VariantSetElement.IsValid())
		{
			DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN(IDatasmithLevelVariantSetsElement, DatasmithElement, );
			Element->RemoveVariantSet(VariantSetElement.ToSharedRef());
		}
	}
}

bool UDatasmithLevelVariantSetsElement::IsElementValid() const
{
	DATASMITHOBJECTELEMENT_ISELEMENTVALID(DatasmithElement);
	return true;
}

#undef DATASMITHOBJECTELEMENT_GETSHARED_AND_EARLYRETURN
#undef DATASMITHOBJECTELEMENT_GETARGUMENT_AND_EARLYREATURN
#undef DATASMITHOBJECTELEMENT_ISELEMENTVALID
