// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectElements/DatasmithUSceneElement.h"

#include "IDatasmithSceneElements.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithUElementsUtils.h"
#include "DatasmithUtils.h"
#include "Utility/DatasmithImporterUtils.h"

#include "Misc/Paths.h"

#define DATASMITHSCENE_TESTARGUMENT(ItemName, NewVariableElementName, IElementType, InElement, GetFunctionName) \
	if (!InElement) \
	{ \
		UE_LOGF(LogDatasmithImport, Error, "%ls is invalid", ItemName); \
		return; \
	} \
	if (!InElement->IsElementValid()) \
	{ \
		UE_LOGF(LogDatasmithImport, Error, "The %ls is not from this Scene", ItemName); \
		return; \
	} \
	TSharedPtr<IElementType> NewVariableElementName = InElement->GetFunctionName().Pin(); \
	if (!NewVariableElementName.IsValid()) \
	{ \
		return; \
	}

#define DATASMITHSCENE_GETALL(UElementType, IElementType, EElementType, CountFunctionName, GetElementFuntionName) \
	TArray<UElementType*> Result; \
	int32 ElementsCount = CountFunctionName(); \
	if (ElementsCount > 0) \
	{ \
		Result.Reserve(ElementsCount); \
		for (int32 Index = 0; Index < ElementsCount; ++Index) \
		{ \
			TSharedPtr<IDatasmithElement> Element = SceneElement->GetElementFuntionName(Index); \
			if ( Element->IsA( EElementType ) ) \
			{ \
				Result.Add(FindOrAddElement(StaticCastSharedPtr<IElementType>(Element))); \
			} \
		} \
	} \
	return Result;

#define DATASMITHSCENE_GETALL_SUBTYPE(UElementType, IElementType, EElementType, EElementSubType, CountFunctionName, GetElementFuntionName) \
	TArray<UElementType*> Result; \
	int32 ElementsCount = CountFunctionName(); \
	if (ElementsCount > 0) \
	{ \
		Result.Reserve(ElementsCount); \
		for (int32 Index = 0; Index < ElementsCount; ++Index) \
		{ \
			TSharedPtr<IDatasmithElement> Element = SceneElement->GetElementFuntionName(Index); \
			if ( Element->IsA( EElementType ) ) \
			{ \
				TSharedPtr<IElementType> CastedElement = StaticCastSharedPtr<IElementType>(Element); \
				if ( CastedElement->IsSubType( EElementSubType ) ) \
				{ \
					Result.Add(FindOrAddElement(StaticCastSharedPtr<IElementType>(Element))); \
				} \
			} \
		} \
	} \
	return Result;

#define DATASMITHSCENE_EARLYRETURN_REMOVE(ItemName, IElementType, GetFunctionName, RemoveFunctionName) \
	DATASMITHSCENE_TESTARGUMENT(ItemName, Element, IElementType, InElement, GetFunctionName) \
	SceneElement->RemoveFunctionName(Element); \

#define DATASMITHSCENE_EARLYRETURN_REMOVECHILD(ItemName, IElementType, GetFunctionName, RemoveFunctionName, RemoveRule) \
	DATASMITHSCENE_TESTARGUMENT(ItemName, Element, IElementType, InElement, GetFunctionName) \
	SceneElement->RemoveFunctionName(Element, RemoveRule);

#define DATASMITHSCENE_ISELEMENTVALID(GetCountFunctionName, GetElementFunctionName) \
	const int32 ElementCount = GetCountFunctionName(); \
	if (ElementCount > 0) \
	{ \
		for (int32 Index = 0; Index < ElementCount; ++Index) \
		{ \
			if (Element == SceneElement->GetElementFunctionName(Index)) \
			{ \
				return true; \
			} \
		} \
	} \
	return false;

#define DATASMITHSCENE_GETALLACTORS(UElementType, IElementType, FunctionName) \
		TArray<UElementType*> Result; \
		TArray<TSharedPtr<IElementType>> AllActors = FDatasmithSceneUtils::FunctionName(SceneElement); \
		if (AllActors.Num() > 0) \
		{ \
			Result.Reserve(AllActors.Num()); \
			for (TSharedPtr<IElementType>& ActorElement : AllActors) \
			{ \
				Result.Add(FindOrAddElement(ActorElement)); \
			} \
		} \
		return Result; \

#define DATASMITHSCENE_CREATEELEMENTWITHUNIQUENAME_VIALOOP(IElementType, DefaultFName, InElementFName, ElementCountFuncName, GetElementFuncName, FactoryCreateFuncName, AddToSceneFuncName) \
	/* Use default name */ \
	if (InElementFName == NAME_None) \
	{ \
		InElementFName = DefaultFName; \
	} \
	/* Find unique Element Name*/ \
	const int32 ElementCount = ElementCountFuncName(); \
	FDatasmithUniqueNameProvider NameProvider; \
	NameProvider.Reserve(ElementCount); \
	for (int32 Index = 0; Index < ElementCount; ++Index) \
	{ \
		NameProvider.AddExistingName(FString(SceneElement->GetElementFuncName(Index)->GetName())); \
	} \
	FString UniqueElementName = NameProvider.GenerateUniqueName(InElementFName.ToString()); \
	/* Create the Element with the unique name */ \
	TSharedPtr<IElementType> Element = FDatasmithSceneFactory::FactoryCreateFuncName(*UniqueElementName); \
	SceneElement->AddToSceneFuncName(Element); \
	return FindOrAddElement(Element);

#define DATASMITHSCENE_CREATEELEMENTWITHUNIQUENAME_VIALIST(IElementType, DefaultFName, InElementFName, SceneUtilGetAllElementFuncName, FactoryCreateFuncName, AddToSceneFuncName) \
	/* Use default name */ \
	if (InElementFName == NAME_None) \
	{ \
		InElementFName = DefaultFName; \
	} \
	/* Find unique Element Name*/ \
	TArray<TSharedPtr<IElementType>> AllElements = FDatasmithSceneUtils::SceneUtilGetAllElementFuncName(SceneElement); \
	FDatasmithUniqueNameProvider NameProvider; \
	NameProvider.Reserve(AllElements.Num()); \
	for (const TSharedPtr<IElementType>& Element : AllElements) \
	{ \
		NameProvider.AddExistingName(FString(Element->GetName())); \
	} \
	FString UniqueElementName = NameProvider.GenerateUniqueName(InElementFName.ToString()); \
	/* Create the Element with the unique name */ \
	TSharedPtr<IElementType> Element = FDatasmithSceneFactory::FactoryCreateFuncName(*UniqueElementName); \
	SceneElement->AddToSceneFuncName(Element); \
	return FindOrAddElement(Element);

/**
 * FDatasmithSceneCollector
 */
UDatasmithSceneElementBase::FDatasmithSceneCollector::FDatasmithSceneCollector()
	: DatasmithSceneElement(nullptr)
{
}

void UDatasmithSceneElementBase::FDatasmithSceneCollector::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (IsValid(DatasmithSceneElement))
	{
		DatasmithSceneElement->ExternalAddReferencedObjects(Collector);
	}
}

namespace DatasmithSceneImpl
{
	template<typename IElement, typename UElement>
	void ExternalAddReferencedObjects(FReferenceCollector& Collector, TMap<TWeakPtr<IElement>, TObjectPtr<UElement>>& Map)
	{
		// Remove all nullptr from the maps
		Map.Remove(TWeakPtr<IElement>());
		for (auto& Itt : Map)
		{
			Collector.AddReferencedObject(Itt.Value);
		}
	}
}

void UDatasmithSceneElementBase::ExternalAddReferencedObjects(FReferenceCollector& Collector)
{
	DatasmithSceneImpl::ExternalAddReferencedObjects(Collector, Materials);
	DatasmithSceneImpl::ExternalAddReferencedObjects(Collector, MaterialIDs);
	DatasmithSceneImpl::ExternalAddReferencedObjects(Collector, Meshes);
	DatasmithSceneImpl::ExternalAddReferencedObjects(Collector, MeshActors);
	DatasmithSceneImpl::ExternalAddReferencedObjects(Collector, LightActors);
	DatasmithSceneImpl::ExternalAddReferencedObjects(Collector, CameraActors);
	DatasmithSceneImpl::ExternalAddReferencedObjects(Collector, PostProcesses);
	DatasmithSceneImpl::ExternalAddReferencedObjects(Collector, Textures);
	DatasmithSceneImpl::ExternalAddReferencedObjects(Collector, PropertyCaptures);
	DatasmithSceneImpl::ExternalAddReferencedObjects(Collector, ObjectPropertyCaptures);
	DatasmithSceneImpl::ExternalAddReferencedObjects(Collector, ActorBindings);
	DatasmithSceneImpl::ExternalAddReferencedObjects(Collector, Variants);
	DatasmithSceneImpl::ExternalAddReferencedObjects(Collector, VariantSets);
	DatasmithSceneImpl::ExternalAddReferencedObjects(Collector, LevelVariantSets);
}

/**
 * UDatasmithSceneElementBase
 */
UDatasmithSceneElementBase::UDatasmithSceneElementBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DatasmithSceneCollector.DatasmithSceneElement = this;
}

FString UDatasmithSceneElementBase::GetHost() const
{
	return FString(SceneElement->GetHost());
}

FString UDatasmithSceneElementBase::GetExporterVersion() const
{
	return FString(SceneElement->GetExporterVersion());
}

FString UDatasmithSceneElementBase::GetVendor() const
{
	return FString(SceneElement->GetVendor());
}

FString UDatasmithSceneElementBase::GetProductName() const
{
	return FString(SceneElement->GetProductName());
}

FString UDatasmithSceneElementBase::GetProductVersion() const
{
	return FString(SceneElement->GetProductVersion());
}

FString UDatasmithSceneElementBase::GetUserID() const
{
	return FString(SceneElement->GetUserID());
}

FString UDatasmithSceneElementBase::GetUserOS() const
{
	return FString(SceneElement->GetUserOS());
}

int32 UDatasmithSceneElementBase::GetExportDuration() const
{
	return SceneElement->GetExportDuration();
}

bool UDatasmithSceneElementBase::GetUsePhysicalSky() const
{
	return SceneElement->GetUsePhysicalSky();
}

/*
 * UDatasmithSceneElementBase Mesh
 */
UDatasmithMeshElement* UDatasmithSceneElementBase::CreateMesh(FName InElementName)
{
	static FName NAME_Mesh = FName(TEXT("Mesh"));
	DATASMITHSCENE_CREATEELEMENTWITHUNIQUENAME_VIALOOP(IDatasmithMeshElement, NAME_Mesh, InElementName, SceneElement->GetMeshesCount, GetMesh, CreateMesh, AddMesh)
}

TArray<UDatasmithMeshElement*> UDatasmithSceneElementBase::GetMeshes()
{
	DATASMITHSCENE_GETALL(UDatasmithMeshElement, IDatasmithMeshElement, EDatasmithElementType::StaticMesh, SceneElement->GetMeshesCount, GetMesh)
}

UDatasmithMeshElement* UDatasmithSceneElementBase::GetMeshByPathName(const FString& MeshPathName)
{
	UDatasmithMeshElement* Result = nullptr;

	if (MeshPathName.IsEmpty())
	{
		UE_LOGF(LogDatasmithImport, Warning, "The MeshPathName is empty.");
		return nullptr;
	}

	if (!FPaths::IsRelative(MeshPathName))
	{
		UE_LOGF(LogDatasmithImport, Log, "The MeshPathName '%ls' refer an Unreal path.", *MeshPathName);
		return nullptr;
	}

	const int32 MeshCount = SceneElement->GetMeshesCount();
	for (int32 Index = 0; Index < MeshCount; ++Index)
	{
		TSharedPtr<IDatasmithMeshElement> MeshElement = SceneElement->GetMesh(Index);
		if (MeshElement->GetName() == MeshPathName)
		{
			return FindOrAddElement(MeshElement);
		}
	}

	UE_LOGF(LogDatasmithImport, Log, "The MeshPathName '%ls' couldn't be found.", *MeshPathName);
	return nullptr;
}

void UDatasmithSceneElementBase::RemoveMesh(UDatasmithMeshElement* InElement)
{
	DATASMITHSCENE_EARLYRETURN_REMOVE(TEXT("Mesh"), IDatasmithMeshElement, GetDatasmithMeshElement, RemoveMesh);
}

/*
 * UDatasmithSceneElementBase Mesh Actor
 */
UDatasmithMeshActorElement* UDatasmithSceneElementBase::CreateMeshActor(FName InElementName)
{
	static FName NAME_MeshActor = FName(TEXT("MeshActor"));
	DATASMITHSCENE_CREATEELEMENTWITHUNIQUENAME_VIALIST(IDatasmithMeshActorElement, NAME_MeshActor, InElementName, GetAllMeshActorsFromScene, CreateMeshActor, AddActor)
}

TArray<UDatasmithMeshActorElement*> UDatasmithSceneElementBase::GetMeshActors()
{
	DATASMITHSCENE_GETALL(UDatasmithMeshActorElement, IDatasmithMeshActorElement, EDatasmithElementType::StaticMeshActor, SceneElement->GetActorsCount, GetActor)
}

TArray<UDatasmithMeshActorElement*> UDatasmithSceneElementBase::GetAllMeshActors()
{
	DATASMITHSCENE_GETALLACTORS(UDatasmithMeshActorElement, IDatasmithMeshActorElement, GetAllMeshActorsFromScene)
}

void UDatasmithSceneElementBase::RemoveMeshActor(UDatasmithMeshActorElement* InElement, EDatasmithActorRemovalRule RemoveRule)
{
	DATASMITHSCENE_EARLYRETURN_REMOVECHILD(TEXT("MeshActor"), IDatasmithMeshActorElement, GetDatasmithMeshActorElement, RemoveActor, RemoveRule)
}

/*
 * UDatasmithSceneElementBase Light Actor
 */
TArray<UDatasmithLightActorElement*> UDatasmithSceneElementBase::GetLightActors()
{
	DATASMITHSCENE_GETALL(UDatasmithLightActorElement, IDatasmithLightActorElement, EDatasmithElementType::Light, SceneElement->GetActorsCount, GetActor)
}

TArray<UDatasmithLightActorElement*> UDatasmithSceneElementBase::GetAllLightActors()
{
	DATASMITHSCENE_GETALLACTORS(UDatasmithLightActorElement, IDatasmithLightActorElement, GetAllLightActorsFromScene)
}

void UDatasmithSceneElementBase::RemoveLightActor(UDatasmithLightActorElement* InElement, EDatasmithActorRemovalRule RemoveRule)
{
	DATASMITHSCENE_EARLYRETURN_REMOVECHILD(TEXT("LightActor"), IDatasmithLightActorElement, GetDatasmithLightActorElement, RemoveActor, RemoveRule)
}

/**
 * UDatasmithSceneElementBase Camera Actor
 */
UDatasmithCameraActorElement* UDatasmithSceneElementBase::CreateCameraActor(FName InElementName)
{
	static FName NAME_CameraActor = FName(TEXT("CameraActor"));
	DATASMITHSCENE_CREATEELEMENTWITHUNIQUENAME_VIALIST(IDatasmithCameraActorElement, NAME_CameraActor, InElementName, GetAllCameraActorsFromScene, CreateCameraActor, AddActor)
}

TArray<UDatasmithCameraActorElement*> UDatasmithSceneElementBase::GetCameraActors()
{
	DATASMITHSCENE_GETALL(UDatasmithCameraActorElement, IDatasmithCameraActorElement, EDatasmithElementType::Camera, SceneElement->GetActorsCount, GetActor)
}

TArray<UDatasmithCameraActorElement*> UDatasmithSceneElementBase::GetAllCameraActors()
{
	DATASMITHSCENE_GETALLACTORS(UDatasmithCameraActorElement, IDatasmithCameraActorElement, GetAllCameraActorsFromScene)
}

void UDatasmithSceneElementBase::RemoveCameraActor(UDatasmithCameraActorElement* InElement, EDatasmithActorRemovalRule RemoveRule)
{
	DATASMITHSCENE_EARLYRETURN_REMOVECHILD(TEXT("CameraActor"), IDatasmithCameraActorElement, GetDatasmithCameraActorElement, RemoveActor, RemoveRule)
}

/**
 * UDatasmithSceneElementBase Custom Actor
 */
TArray<UDatasmithCustomActorElement*> UDatasmithSceneElementBase::GetCustomActors()
{
	DATASMITHSCENE_GETALL(UDatasmithCustomActorElement, IDatasmithCustomActorElement, EDatasmithElementType::CustomActor, SceneElement->GetActorsCount, GetActor)
}

TArray<UDatasmithCustomActorElement*> UDatasmithSceneElementBase::GetAllCustomActors()
{
	DATASMITHSCENE_GETALLACTORS(UDatasmithCustomActorElement, IDatasmithCustomActorElement, GetAllCustomActorsFromScene)
}

void UDatasmithSceneElementBase::RemoveCustomActor(UDatasmithCustomActorElement* InElement, EDatasmithActorRemovalRule RemoveRule)
{
	DATASMITHSCENE_EARLYRETURN_REMOVECHILD(TEXT("CustomActor"), IDatasmithCustomActorElement, GetDatasmithCustomActorElement, RemoveActor, RemoveRule)
}

/**
 * UDatasmithSceneElementBase Texture
 */
UDatasmithTextureElement* UDatasmithSceneElementBase::CreateTexture(FName InElementName)
{
	static FName NAME_Texture = FName(TEXT("Texture"));
	DATASMITHSCENE_CREATEELEMENTWITHUNIQUENAME_VIALOOP(IDatasmithTextureElement, NAME_Texture, InElementName, SceneElement->GetTexturesCount, GetTexture, CreateTexture, AddTexture)
}

TArray<UDatasmithTextureElement*> UDatasmithSceneElementBase::GetTextures()
{
	DATASMITHSCENE_GETALL(UDatasmithTextureElement, IDatasmithTextureElement, EDatasmithElementType::Texture, SceneElement->GetTexturesCount, GetTexture)
}

void UDatasmithSceneElementBase::RemoveTexture(UDatasmithTextureElement* InElement)
{
	DATASMITHSCENE_EARLYRETURN_REMOVE(TEXT("Texture"), IDatasmithTextureElement, GetDatasmithTextureElement, RemoveTexture)
}

UDatasmithPostProcessElement* UDatasmithSceneElementBase::GetPostProcess()
{
	TSharedPtr<IDatasmithPostProcessElement> PostProcessElement = SceneElement->GetPostProcess();
	if (PostProcessElement.IsValid())
	{
		return FindOrAddElement(PostProcessElement);
	}
	return nullptr;
}

/**
 * UDatasmithSceneElementBase Material
 */
TArray<UDatasmithBaseMaterialElement*> UDatasmithSceneElementBase::GetAllMaterials()
{
	DATASMITHSCENE_GETALL(UDatasmithBaseMaterialElement, IDatasmithBaseMaterialElement, EDatasmithElementType::BaseMaterial, SceneElement->GetMaterialsCount, GetMaterial)
}

void UDatasmithSceneElementBase::RemoveMaterial(UDatasmithBaseMaterialElement* InElement)
{
	DATASMITHSCENE_EARLYRETURN_REMOVE(TEXT("Material"), IDatasmithBaseMaterialElement, GetDatasmithBaseMaterialElement, RemoveMaterial)
}

/**
 * UDatasmithSceneElementBase Meta Data
 */
TArray<UDatasmithMetaDataElement*> UDatasmithSceneElementBase::GetMetaData()
{
	DATASMITHSCENE_GETALL(UDatasmithMetaDataElement, IDatasmithMetaDataElement, EDatasmithElementType::MetaData, SceneElement->GetMetaDataCount, GetMetaData)
}

UDatasmithMetaDataElement* UDatasmithSceneElementBase::GetMetaDataForObject(UDatasmithObjectElement* Object)
{
	if (Object == nullptr)
	{
		return nullptr;
	}

	TArray<UDatasmithMetaDataElement*> AllMetaData = GetMetaData();

	for (UDatasmithMetaDataElement* CurrentMetaData : AllMetaData)
	{
		if (CurrentMetaData->GetAssociatedElement() == Object)
		{
			return CurrentMetaData;
		}
	}

	return nullptr;
}

FString UDatasmithSceneElementBase::GetMetaDataValueForKey(UDatasmithObjectElement* Object, const FString& Key)
{
	if (Object)
	{
		if (UDatasmithMetaDataElement* MetaDataElement = GetMetaDataForObject(Object))
		{
			if (UDatasmithKeyValueProperty* Property = MetaDataElement->GetPropertyByName(Key))
			{
				return Property->GetValue();
			}
		}
	}
	return FString();
}

void UDatasmithSceneElementBase::GetMetaDataKeysAndValuesForValue(UDatasmithObjectElement* Object, const FString& StringToMatch, TArray<FString>& OutKeys, TArray<FString>& OutValues)
{
	OutKeys.Reset();
	OutValues.Reset();

	if (Object)
	{
		if (UDatasmithMetaDataElement* MetaDataElement = GetMetaDataForObject(Object))
		{
			TArray<FString> Keys;
			TArray<FString> Values;
			MetaDataElement->GetProperties(Keys, Values);
			for (int32 i = 0; i < Keys.Num(); ++i)
			{
				if (Values[i].Contains(StringToMatch))
				{
					OutKeys.Add(Keys[i]);
					OutValues.Add(Values[i]);
				}
			}
		}
	}
}

void UDatasmithSceneElementBase::GetAllMetaData(TSubclassOf<UDatasmithObjectElement> ObjectClass, TArray<UDatasmithMetaDataElement*>& OutMetaData)
{
	OutMetaData.Reset();

	TArray<UDatasmithMetaDataElement*> MetaDataElements = GetMetaData();
	for (UDatasmithMetaDataElement* MetaDataElement : MetaDataElements)
	{
		UDatasmithObjectElement* Object = MetaDataElement->GetAssociatedElement();
		if (Object && (ObjectClass == nullptr || Object->IsA(ObjectClass.Get())))
		{
			OutMetaData.Add(MetaDataElement);
		}
	}
}

void UDatasmithSceneElementBase::GetAllObjectsAndValuesForKey(const FString& Key, TSubclassOf<UDatasmithObjectElement> ObjectClass, TArray<UDatasmithObjectElement*>& OutObjects, TArray<FString>& OutValues)
{
	OutObjects.Reset();
	OutValues.Reset();

	if (Key.IsEmpty())
	{
		return;
	}

	TArray<UDatasmithMetaDataElement*> MetaDataElements;
	GetAllMetaData(ObjectClass, MetaDataElements);

	for (UDatasmithMetaDataElement* MetaDataElement : MetaDataElements)
	{
		if (UDatasmithKeyValueProperty* Property = MetaDataElement->GetPropertyByName(Key))
		{
			if (MetaDataElement->GetAssociatedElement())
			{
				OutObjects.Add(MetaDataElement->GetAssociatedElement());
				OutValues.Add(Property->GetValue());
			}
		}
	}
}

UDatasmithLevelVariantSetsElement* UDatasmithSceneElementBase::CreateLevelVariantSets(FName InElementName)
{
	static FName NAME_LevelVariantSets = FName(TEXT("LevelVariantSets"));
	DATASMITHSCENE_CREATEELEMENTWITHUNIQUENAME_VIALOOP(IDatasmithLevelVariantSetsElement, NAME_LevelVariantSets, InElementName, SceneElement->GetLevelVariantSetsCount, GetLevelVariantSets, CreateLevelVariantSets, AddLevelVariantSets)
}

TArray<UDatasmithLevelVariantSetsElement*> UDatasmithSceneElementBase::GetAllLevelVariantSets()
{
	DATASMITHSCENE_GETALL_SUBTYPE(UDatasmithLevelVariantSetsElement, IDatasmithLevelVariantSetsElement, EDatasmithElementType::Variant, EDatasmithElementVariantSubType::LevelVariantSets, SceneElement->GetLevelVariantSetsCount, GetLevelVariantSets);
}

void UDatasmithSceneElementBase::RemoveLevelVariantSets(UDatasmithLevelVariantSetsElement* InElement)
{
	DATASMITHSCENE_EARLYRETURN_REMOVE(TEXT("LevelVariantSets"), IDatasmithLevelVariantSetsElement, GetLevelVariantSetsElement, RemoveLevelVariantSets)
}

/*
 * UDatasmithSceneElementBase Attach
 */
void UDatasmithSceneElementBase::AttachActor(UDatasmithActorElement* NewParent, UDatasmithActorElement* Child, EDatasmithActorAttachmentRule AttachmentRule)
{
	DATASMITHSCENE_TESTARGUMENT(TEXT("Actor"), ParentElement, IDatasmithActorElement, NewParent, GetIDatasmithActorElement)
	DATASMITHSCENE_TESTARGUMENT(TEXT("Actor"), ChildElement, IDatasmithActorElement, Child, GetIDatasmithActorElement)
	SceneElement->AttachActor(ParentElement, ChildElement, AttachmentRule);
}

void UDatasmithSceneElementBase::AttachActorToSceneRoot(UDatasmithActorElement* Child, EDatasmithActorAttachmentRule AttachmentRule)
{
	DATASMITHSCENE_TESTARGUMENT(TEXT("Actor"), ChildElement, IDatasmithActorElement, Child, GetIDatasmithActorElement)
	SceneElement->AttachActorToSceneRoot(ChildElement, AttachmentRule);
}

void UDatasmithSceneElementBase::SetDatasmithSceneElement(TSharedPtr<IDatasmithScene> InElement)
{
	SceneElement = InElement;

	Reset();
}

/*
 * UDatasmithSceneElementBase Is Valid
 */
bool UDatasmithSceneElementBase::IsElementValid(const TWeakPtr<IDatasmithMaterialIDElement>& Element) const
{
	const TSharedPtr<IDatasmithMaterialIDElement> MaterialElement = Element.Pin();
	if (MaterialElement.IsValid())
	{
		return FDatasmithSceneUtils::IsMaterialIDUsedInScene(SceneElement, MaterialElement);
	}
	return false;
}

bool UDatasmithSceneElementBase::IsElementValid(const TWeakPtr<IDatasmithBaseMaterialElement>& Element) const
{
	DATASMITHSCENE_ISELEMENTVALID(SceneElement->GetMaterialsCount, GetMaterial);
}

bool UDatasmithSceneElementBase::IsElementValid(const TWeakPtr<IDatasmithMeshElement>& Element) const
{
	DATASMITHSCENE_ISELEMENTVALID(SceneElement->GetMeshesCount, GetMesh);
}

namespace DatasmithSceneImpl
{
	template<class T>
	bool IsActorElementValid(const TSharedPtr<class IDatasmithScene>& SceneElement, const TWeakPtr<T>& Element)
	{
		TSharedPtr<IDatasmithActorElement> SharedElement = Element.Pin();
		bool bFound = false;
		if (SharedElement.IsValid() && SceneElement.IsValid())
		{
			FDatasmithSceneUtils::TActorHierarchy Hierarchy;
			bFound = FDatasmithSceneUtils::FindActorHierarchy(SceneElement.Get(), SharedElement, Hierarchy);
		}
		return bFound;
	}
}

bool UDatasmithSceneElementBase::IsElementValid(const TWeakPtr<IDatasmithMeshActorElement>& Element) const
{
	return DatasmithSceneImpl::IsActorElementValid(SceneElement, Element);
}

bool UDatasmithSceneElementBase::IsElementValid(const TWeakPtr<IDatasmithLightActorElement>& Element) const
{
	return DatasmithSceneImpl::IsActorElementValid(SceneElement, Element);
}

bool UDatasmithSceneElementBase::IsElementValid(const TWeakPtr<IDatasmithCameraActorElement>& Element) const
{
	return DatasmithSceneImpl::IsActorElementValid(SceneElement, Element);
}

bool UDatasmithSceneElementBase::IsElementValid(const TWeakPtr<IDatasmithTextureElement>& Element) const
{
	DATASMITHSCENE_ISELEMENTVALID(SceneElement->GetTexturesCount, GetTexture);
}

bool UDatasmithSceneElementBase::IsElementValid(const TWeakPtr<IDatasmithPostProcessElement>& Element) const
{
	const TSharedPtr<IDatasmithPostProcessElement> PostProcessElement = Element.Pin();
	if (PostProcessElement.IsValid())
	{
		return FDatasmithSceneUtils::IsPostProcessUsedInScene(SceneElement, PostProcessElement);
	}
	return false;
}

bool UDatasmithSceneElementBase::IsElementValid(const TWeakPtr<IDatasmithMetaDataElement>& Element) const
{
	DATASMITHSCENE_ISELEMENTVALID(SceneElement->GetMetaDataCount, GetMetaData);
}

bool UDatasmithSceneElementBase::IsElementValid(const TWeakPtr<IDatasmithCustomActorElement>& Element) const
{
	return DatasmithSceneImpl::IsActorElementValid(SceneElement, Element);
}

bool UDatasmithSceneElementBase::IsElementValid(const TWeakPtr<IDatasmithBasePropertyCaptureElement>& Element) const
{
	TSharedPtr<IDatasmithBasePropertyCaptureElement> PinnedElement = Element.Pin();
	if (!PinnedElement.IsValid())
	{
		return false;
	}

	bool bIsValid = false;
	FDatasmithUElementsUtils::ForVariantElement<IDatasmithBasePropertyCaptureElement>(SceneElement, [&](TSharedPtr<IDatasmithBasePropertyCaptureElement> CurrentElement)
	{
		if (CurrentElement == PinnedElement)
		{
			bIsValid = true;

			// Stop iterating
			return false;
		}
		return true;
	});

	return bIsValid;
}

bool UDatasmithSceneElementBase::IsElementValid(const TWeakPtr<IDatasmithActorBindingElement>& Element) const
{
	TSharedPtr<IDatasmithActorBindingElement> PinnedElement = Element.Pin();
	if (!PinnedElement.IsValid())
	{
		return false;
	}

	bool bIsValid = false;
	FDatasmithUElementsUtils::ForVariantElement<IDatasmithActorBindingElement>(SceneElement, [&](TSharedPtr<IDatasmithActorBindingElement> CurrentElement)
	{
		if (CurrentElement == PinnedElement)
		{
			bIsValid = true;

			// Stop iterating
			return false;
		}
		return true;
	});

	return bIsValid;
}

bool UDatasmithSceneElementBase::IsElementValid(const TWeakPtr<IDatasmithVariantElement>& Element) const
{
	TSharedPtr<IDatasmithVariantElement> PinnedElement = Element.Pin();
	if (!PinnedElement.IsValid())
	{
		return false;
	}

	bool bIsValid = false;
	FDatasmithUElementsUtils::ForVariantElement<IDatasmithVariantElement>(SceneElement, [&](TSharedPtr<IDatasmithVariantElement> CurrentElement)
	{
		if (CurrentElement == PinnedElement)
		{
			bIsValid = true;

			// Stop iterating
			return false;
		}
		return true;
	});

	return bIsValid;
}

bool UDatasmithSceneElementBase::IsElementValid(const TWeakPtr<IDatasmithVariantSetElement>& Element) const
{
	TSharedPtr<IDatasmithVariantSetElement> PinnedElement = Element.Pin();
	if (!PinnedElement.IsValid())
	{
		return false;
	}

	bool bIsValid = false;
	FDatasmithUElementsUtils::ForVariantElement<IDatasmithVariantSetElement>(SceneElement, [&](const TSharedPtr<IDatasmithVariantSetElement>& CurrentElement)
	{
		if (CurrentElement == PinnedElement)
		{
			bIsValid = true;

			// Stop iterating
			return false;
		}
		return true;
	});

	return bIsValid;
}

bool UDatasmithSceneElementBase::IsElementValid(const TWeakPtr<IDatasmithLevelVariantSetsElement>& Element) const
{
	DATASMITHSCENE_ISELEMENTVALID(SceneElement->GetLevelVariantSetsCount, GetLevelVariantSets);
}

/**
 * UDatasmithSceneElementBase Find Or Add
 */
UDatasmithObjectElement* UDatasmithSceneElementBase::FindOrAddElement(const TSharedPtr<IDatasmithElement>& InElement)
{
	if (!InElement.IsValid())
	{
		return nullptr;
	}

	UDatasmithObjectElement* Object = nullptr;

	if (InElement->IsA(EDatasmithElementType::Actor))
	{
		Object = FindOrAddActorElement(StaticCastSharedPtr<IDatasmithActorElement>(InElement));
	}
	else if (InElement->IsA(EDatasmithElementType::Texture))
	{
		Object = FindOrAddElement(StaticCastSharedPtr<IDatasmithTextureElement>(InElement));
	}
	else if (InElement->IsA(EDatasmithElementType::BaseMaterial))
	{
		Object = FindOrAddElement(StaticCastSharedPtr<IDatasmithBaseMaterialElement>(InElement));
	}
	else if (InElement->IsA(EDatasmithElementType::MaterialId))
	{
		Object = FindOrAddElement(StaticCastSharedPtr<IDatasmithMaterialIDElement>(InElement));
	}
	else if (InElement->IsA(EDatasmithElementType::StaticMesh))
	{
		Object = FindOrAddElement(StaticCastSharedPtr<IDatasmithMeshElement>(InElement));
	}
	else if (InElement->IsA(EDatasmithElementType::PostProcess))
	{
		Object = FindOrAddElement(StaticCastSharedPtr<IDatasmithPostProcessElement>(InElement));
	}
	else if (InElement->IsA(EDatasmithElementType::MetaData))
	{
		Object = FindOrAddElement(StaticCastSharedPtr<IDatasmithMetaDataElement>(InElement));
	}
	else if (InElement->IsA(EDatasmithElementType::Variant))
	{
		TSharedPtr<IDatasmithVariantElement> VariantElement = StaticCastSharedPtr<IDatasmithVariantElement>(InElement);

		if (VariantElement->IsSubType(EDatasmithElementVariantSubType::PropertyCapture))
		{
			Object = FindOrAddElement(StaticCastSharedPtr<IDatasmithPropertyCaptureElement>(InElement));
		}
		else if (VariantElement->IsSubType(EDatasmithElementVariantSubType::ObjectPropertyCapture))
		{
			Object = FindOrAddElement(StaticCastSharedPtr<IDatasmithObjectPropertyCaptureElement>(InElement));
		}
		else if (VariantElement->IsSubType(EDatasmithElementVariantSubType::ActorBinding))
		{
			Object = FindOrAddElement(StaticCastSharedPtr<IDatasmithActorBindingElement>(InElement));
		}
		else if (VariantElement->IsSubType(EDatasmithElementVariantSubType::Variant))
		{
			Object = FindOrAddElement(StaticCastSharedPtr<IDatasmithVariantElement>(InElement));
		}
		else if (VariantElement->IsSubType(EDatasmithElementVariantSubType::VariantSet))
		{
			Object = FindOrAddElement(StaticCastSharedPtr<IDatasmithVariantSetElement>(InElement));
		}
		else if (VariantElement->IsSubType(EDatasmithElementVariantSubType::LevelVariantSets))
		{
			Object = FindOrAddElement(StaticCastSharedPtr<IDatasmithLevelVariantSetsElement>(InElement));
		}
	}

	return Object;
}

UDatasmithActorElement* UDatasmithSceneElementBase::FindOrAddActorElement(const TSharedPtr<IDatasmithActorElement>& InElement)
{
	if (InElement->IsA(EDatasmithElementType::StaticMeshActor))
	{
		TSharedPtr<IDatasmithMeshActorElement> MeshActorElement = StaticCastSharedPtr<IDatasmithMeshActorElement>(InElement);
		return FindOrAddElement(MeshActorElement);
	}
	else if (InElement->IsA(EDatasmithElementType::Light))
	{
		TSharedPtr<IDatasmithLightActorElement> LightActorElement = StaticCastSharedPtr<IDatasmithLightActorElement>(InElement);
		return FindOrAddElement(LightActorElement);
	}
	else if (InElement->IsA(EDatasmithElementType::Camera))
	{
		TSharedPtr<IDatasmithCameraActorElement> CameraActorElement = StaticCastSharedPtr<IDatasmithCameraActorElement>(InElement);
		return FindOrAddElement(CameraActorElement);
	}
	else if (InElement->IsA(EDatasmithElementType::CustomActor))
	{
		TSharedPtr<IDatasmithCustomActorElement> CustomActorElement = StaticCastSharedPtr<IDatasmithCustomActorElement>(InElement);
		return FindOrAddElement(CustomActorElement);
	}
	return nullptr;
}

UDatasmithBaseMaterialElement* UDatasmithSceneElementBase::FindOrAddElement(const TSharedPtr<IDatasmithBaseMaterialElement>& InElement)
{
	return FDatasmithUElementsUtils::FindOrAddElement(this, Materials, InElement, [&](UDatasmithBaseMaterialElement* Element)
	{
		Element->SetDatasmithBaseMaterialElement(InElement);
	});
}

UDatasmithMaterialIDElement* UDatasmithSceneElementBase::FindOrAddElement(const TSharedPtr<IDatasmithMaterialIDElement>& InElement)
{
	return FDatasmithUElementsUtils::FindOrAddElement(this, MaterialIDs, InElement, [&](UDatasmithMaterialIDElement* Element)
		{
			Element->SetDatasmithMaterialIDElement(InElement);
		});
}

UDatasmithMeshElement* UDatasmithSceneElementBase::FindOrAddElement(const TSharedPtr<IDatasmithMeshElement>& InElement)
{
	return FDatasmithUElementsUtils::FindOrAddElement(this, Meshes, InElement, [&](UDatasmithMeshElement* Element)
		{
			Element->SetDatasmithMeshElement(InElement);
		});
}

UDatasmithMeshActorElement* UDatasmithSceneElementBase::FindOrAddElement(const TSharedPtr<IDatasmithMeshActorElement>& InElement)
{
	return FDatasmithUElementsUtils::FindOrAddElement(this, MeshActors, InElement, [&](UDatasmithMeshActorElement* Element)
		{
			Element->SetDatasmithMeshActorElement(InElement);
		});
}

UDatasmithLightActorElement* UDatasmithSceneElementBase::FindOrAddElement(const TSharedPtr<IDatasmithLightActorElement>& InElement)
{
	return FDatasmithUElementsUtils::FindOrAddElement(this, LightActors, InElement, [&](UDatasmithLightActorElement* Element)
	{
		Element->SetDatasmithLightActorElement(InElement);
	});
}

UDatasmithCameraActorElement* UDatasmithSceneElementBase::FindOrAddElement(const TSharedPtr<IDatasmithCameraActorElement>& InElement)
{
	return FDatasmithUElementsUtils::FindOrAddElement(this, CameraActors, InElement, [&](UDatasmithCameraActorElement* Element)
		{
			Element->SetDatasmithCameraActorElement(InElement);
		});
}

UDatasmithPostProcessElement* UDatasmithSceneElementBase::FindOrAddElement(const TSharedPtr<IDatasmithPostProcessElement>& InElement)
{
	return FDatasmithUElementsUtils::FindOrAddElement(this, PostProcesses, InElement, [&](UDatasmithPostProcessElement* Element)
		{
			Element->SetDatasmithPostProcessElement(InElement);
		});
}

UDatasmithTextureElement* UDatasmithSceneElementBase::FindOrAddElement(const TSharedPtr<IDatasmithTextureElement>& InElement)
{
	return FDatasmithUElementsUtils::FindOrAddElement(this, Textures, InElement, [&](UDatasmithTextureElement* Element)
		{
			Element->SetDatasmithTextureElement(InElement);
		});
}

UDatasmithMetaDataElement* UDatasmithSceneElementBase::FindOrAddElement(const TSharedPtr<IDatasmithMetaDataElement>& InElement)
{
	return FDatasmithUElementsUtils::FindOrAddElement(this, MetaData, InElement, [&](UDatasmithMetaDataElement* Element)
	{
		Element->SetDatasmithMetaDataElement(InElement);
	});
}

UDatasmithCustomActorElement* UDatasmithSceneElementBase::FindOrAddElement(const TSharedPtr<IDatasmithCustomActorElement>& InElement)
{
	return FDatasmithUElementsUtils::FindOrAddElement(this, CustomActors, InElement, [&](UDatasmithCustomActorElement* Element)
	{
		Element->SetDatasmithCustomActorElement(InElement);
	});
}

UDatasmithPropertyCaptureElement* UDatasmithSceneElementBase::FindOrAddElement(const TSharedPtr<IDatasmithPropertyCaptureElement>& InElement)
{
	return FDatasmithUElementsUtils::FindOrAddElement(this, PropertyCaptures, InElement, [&](UDatasmithPropertyCaptureElement* Element)
	{
		Element->SetPropertyCaptureElement(InElement);
	});
}

UDatasmithObjectPropertyCaptureElement* UDatasmithSceneElementBase::FindOrAddElement(const TSharedPtr<IDatasmithObjectPropertyCaptureElement>& InElement)
{
	return FDatasmithUElementsUtils::FindOrAddElement(this, ObjectPropertyCaptures, InElement, [&](UDatasmithObjectPropertyCaptureElement* Element)
	{
		Element->SetObjectPropertyCaptureElement(InElement);
	});
}

UDatasmithActorBindingElement* UDatasmithSceneElementBase::FindOrAddElement(const TSharedPtr<IDatasmithActorBindingElement>& InElement)
{
	return FDatasmithUElementsUtils::FindOrAddElement(this, ActorBindings, InElement, [&](UDatasmithActorBindingElement* Element)
	{
		Element->SetActorBindingElement(InElement);
	});
}

UDatasmithVariantElement* UDatasmithSceneElementBase::FindOrAddElement(const TSharedPtr<IDatasmithVariantElement>& InElement)
{
	return FDatasmithUElementsUtils::FindOrAddElement(this, Variants, InElement, [&](UDatasmithVariantElement* Element)
	{
		Element->SetVariantElement(InElement);
	});
}

UDatasmithVariantSetElement* UDatasmithSceneElementBase::FindOrAddElement(const TSharedPtr<IDatasmithVariantSetElement>& InElement)
{
	return FDatasmithUElementsUtils::FindOrAddElement(this, VariantSets, InElement, [&](UDatasmithVariantSetElement* Element)
	{
		Element->SetVariantSetElement(InElement);
	});
}

UDatasmithLevelVariantSetsElement* UDatasmithSceneElementBase::FindOrAddElement(const TSharedPtr<IDatasmithLevelVariantSetsElement>& InElement)
{
	return FDatasmithUElementsUtils::FindOrAddElement(this, LevelVariantSets, InElement, [&](UDatasmithLevelVariantSetsElement* Element)
	{
		Element->SetLevelVariantSetsElement(InElement);
	});
}

namespace DatasmithSceneImpl
{
	template<typename T>
	void Reset(T& Elements)
	{
		for (auto& Itt : Elements)
		{
			auto& Element = Itt.Value;
			if (Element)
			{
				Element->MarkAsGarbage();
			}
		}
		Elements.Reset();
	}
}

void UDatasmithSceneElementBase::Reset()
{
	DatasmithSceneImpl::Reset(Materials);
	DatasmithSceneImpl::Reset(MaterialIDs);
	DatasmithSceneImpl::Reset(Meshes);
	DatasmithSceneImpl::Reset(MeshActors);
	DatasmithSceneImpl::Reset(LightActors);
	DatasmithSceneImpl::Reset(CameraActors);
	DatasmithSceneImpl::Reset(PostProcesses);
	DatasmithSceneImpl::Reset(Textures);
	DatasmithSceneImpl::Reset(PropertyCaptures);
	DatasmithSceneImpl::Reset(ObjectPropertyCaptures);
	DatasmithSceneImpl::Reset(ActorBindings);
	DatasmithSceneImpl::Reset(Variants);
	DatasmithSceneImpl::Reset(VariantSets);
	DatasmithSceneImpl::Reset(LevelVariantSets);
}

#undef DATASMITHSCENE_GETALL
#undef DATASMITHSCENE_GETALL_SUBTYPE
#undef DATASMITHSCENE_EARLYRETURN_REMOVE
#undef DATASMITHSCENE_EARLYRETURN_REMOVECHILD
#undef DATASMITHSCENE_ISELEMENTVALID
#undef DATASMITHSCENE_GETALLACTORS
#undef DATASMITHSCENE_CREATEELEMENTWITHUNIQUENAME_VIALOOP
#undef DATASMITHSCENE_CREATEELEMENTWITHUNIQUENAME_VIALIST
