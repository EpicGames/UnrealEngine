// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/DatasmithConsumerDetails.h"

#include "DatasmithConsumer.h"
#include "Utility/DatasmithImporterUtils.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Dialogs/DlgPickPath.h"
#include "EditorLevelUtils.h"
#include "Engine/Level.h"
#include "Engine/LevelStreamingAlwaysLoaded.h"
#include "Engine/World.h"
#include "Factories/LevelFactory.h"
#include "Misc/FileHelper.h"
#include "Misc/MessageDialog.h"
#include "Misc/PackageName.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "DatasmithConsumer"

namespace DatasmithConsumerDetailsUtil
{
	// Inspired from ContentBrowserUtils::IsValidObjectPathForCreate
	bool VerifyObjectName(const FString& ObjectName, FText& OutErrorMessage)
	{
		if ( !FFileHelper::IsFilenameValidForSaving(ObjectName, OutErrorMessage) )
		{
			// Return false to indicate that the user should enter a new name
			return false;
		}

		// Make sure the new name only contains valid characters
		if ( !FName::IsValidXName( ObjectName, INVALID_OBJECTNAME_CHARACTERS INVALID_LONGPACKAGE_CHARACTERS, &OutErrorMessage ) )
		{
			// Return false to indicate that the user should enter a new name
			return false;
		}

		return true;
	}

	/** Helper class to force a widget to fill in a space. Copied from SDetailSingleItemRow.cpp */
	class SConstrainedBox : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SConstrainedBox) {}
		SLATE_DEFAULT_SLOT(FArguments, Content)
			SLATE_END_ARGS()

			void Construct(const FArguments& InArgs)
		{
			ChildSlot
				[
					InArgs._Content.Widget
				];
		}

		virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override
		{
			// Voluntarily ridiculously large value to force the child widget to fill up the available space
			const float MinWidthVal = 2000;
			const FVector2D ChildSize = ChildSlot.GetWidget()->GetDesiredSize();
			return FVector2D(FMath::Max(MinWidthVal, ChildSize.X), ChildSize.Y);
		}
	};

	class SFolderProperty : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SFolderProperty) {}
		SLATE_END_ARGS()

	public:
		void Construct(const FArguments& InArgs, UDatasmithConsumer* InConsumer)
		{
			ConsumerPtr = InConsumer;
			bCommitting = false;
			bTextValid = true;

			FSlateFontInfo FontInfo = IDetailLayoutBuilder::GetDetailFont();

			TSharedRef<SWidget> BrowseButton = PropertyCustomizationHelpers::MakeBrowseButton( FSimpleDelegate::CreateSP( this, &SFolderProperty::OnBrowseContentFolder ) );

			ChildSlot
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Padding(5.0f, 2.5f, 2.0f, 2.5f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(9.0f)
					.HAlign(EHorizontalAlignment::HAlign_Fill)
					[
						SAssignNew(ContentFolderTextBox, SEditableTextBox)
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.HintText(LOCTEXT("DataprepSlateHelper_ContentFolderHintText", "Set the content folder to save in"))
						.IsReadOnly(false)
						.OnTextCommitted(FOnTextCommitted::CreateSP(this, &SFolderProperty::OnTextCommitted))
						.OnTextChanged(FOnTextChanged::CreateSP(this, &SFolderProperty::OnTextChanged))
					]
					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.HAlign(EHorizontalAlignment::HAlign_Right)
					[
						BrowseButton
					]
				]

			];

			UpdateContentFolderText();
		}

	private:
		void OnBrowseContentFolder()
		{
							   //Ask the user for the root path where they want any content to be placed
			if( UDataprepContentConsumer* DataprepConsumer = ConsumerPtr.Get() )
			{
				FString Path = DataprepConsumer->GetTargetContentFolder();
				if( Path.IsEmpty() )
				{
					Path = FPaths::GetPath( DataprepConsumer->GetOutermost()->GetPathName() );
				}
				Path += TEXT("/"); // Trailing '/' is needed to set the default path

				TSharedRef<SDlgPickPath> PickContentPathDlg =
					SNew(SDlgPickPath)
					.Title(LOCTEXT("DataprepSlateHelper_ChooseImportRootContentPath", "Choose Location for importing the Datasmith content"))
					.DefaultPath(FText::FromString(Path));

				if ( PickContentPathDlg->ShowModal() == EAppReturnType::Ok )
				{
					FScopedTransaction Transaction( LOCTEXT("Consumer_SetTargetContentFolder", "Set Target Content Folder") );

					FText ErrorReason;
					if( DataprepConsumer->SetTargetContentFolder( PickContentPathDlg->GetPath().ToString(), ErrorReason ) )
					{
						UpdateContentFolderText();
					}
					else
					{
						Transaction.Cancel();
						UE_LOGF( LogDatasmithImport, Error, "%ls", *ErrorReason.ToString() );
					}
				}
			}
		}

		// Inspired from ContentBrowserUtils::IsValidFolderPathForCreate
		void OnTextChanged(const FText& InText)
		{
			FText OutErrorMessage = FText::GetEmpty();
			FString FolderPath = InText.ToString();

			// Check length of the folder name
			if ( FolderPath.Len() > 0 )
			{
				if(FolderPath.StartsWith( TEXT("..")))
				{
					OutErrorMessage = LOCTEXT( "InvalidFolderName_RelativePath", "Relative path is not accepted." );
				}
				else
				{
					if( FolderPath.StartsWith( TEXT("/Content") ) )
					{
						FolderPath = FolderPath.Replace( TEXT( "/Content" ), TEXT( "/Game" ) );
					}

					if(FolderPath.Len() <= FPlatformMisc::GetMaxPathLength())
					{
						if(FolderPath[FolderPath.Len() - 1] == L'/')
						{
							FolderPath.LeftInline(FolderPath.Len() - 1, EAllowShrinking::No);
						}

						if(FolderPath != TEXT( "/Game" ))
						{
							FString FolderName = FPaths::GetBaseFilename(FolderPath);

							if(VerifyObjectName(FolderName, OutErrorMessage))
							{
								const FString InvalidChars = INVALID_LONGPACKAGE_CHARACTERS TEXT("/[]"); // Slash and Square brackets are invalid characters for a folder name

																										 // See if the name contains invalid characters.
								FString Char;
								for( int32 CharIdx = 0; CharIdx < FolderName.Len(); ++CharIdx )
								{
									Char = FolderName.Mid(CharIdx, 1);

									if ( InvalidChars.Contains(*Char) )
									{
										FString ReadableInvalidChars = InvalidChars;
										ReadableInvalidChars.ReplaceInline(TEXT("\r"), TEXT(""));
										ReadableInvalidChars.ReplaceInline(TEXT("\n"), TEXT(""));
										ReadableInvalidChars.ReplaceInline(TEXT("\t"), TEXT(""));

										OutErrorMessage = FText::Format(LOCTEXT("InvalidFolderName_InvalidCharacters", "A folder name may not contain any of the following characters: {0}"), FText::FromString(ReadableInvalidChars));
										break;
									}
								}

								if(OutErrorMessage.IsEmpty())
								{
									if(FFileHelper::IsFilenameValidForSaving(FolderPath, OutErrorMessage))
									{
										FString PathOnDisk;
										if(FPackageName::TryConvertLongPackageNameToFilename(FolderPath, PathOnDisk))
										{
											// Make sure we are not creating a folder path that is too long
											if (PathOnDisk.Len() > FPlatformMisc::GetMaxPathLength() - 32/*MAX_CLASS_NAME_LENGTH*/)
											{
												// The full path for the folder is too long
												OutErrorMessage = FText::Format(LOCTEXT("RenameFolderPathTooLong", "The full path for the folder is too deep, the maximum is '{0}'. Please choose a shorter name for the folder or create it in a shallower folder structure."), FText::AsNumber(FPlatformMisc::GetMaxPathLength() - 32/*MAX_CLASS_NAME_LENGTH*/));
											}
										}
										else
										{
											OutErrorMessage = FText::Format(LOCTEXT("RenameFolderFailedDiskPath", "Folder path could not be converted to disk path: '{0}'"), FText::FromString(FolderPath));
										}
									}
								}
							}
						}
					}
					else
					{
						OutErrorMessage = FText::Format( LOCTEXT("InvalidFolderName_TooLongForCooking", "Filename is too long ({0} characters); this may interfere with cooking for consoles. Unreal filenames should be no longer than {1} characters. Filename value: {2}" ),
							FText::AsNumber(FolderPath.Len()), FText::AsNumber(FPlatformMisc::GetMaxPathLength()), FText::FromString(FolderPath) );
					}
				}
			}
			else
			{
				OutErrorMessage = LOCTEXT( "InvalidFolderName_IsTooShort", "Please provide a name for this folder." );
			}

			bTextValid = OutErrorMessage.IsEmpty();

			ContentFolderTextBox->SetError(OutErrorMessage);
		}

		void UpdateContentFolderText()
		{
			ContentFolderTextBox->SetError(FText::GetEmpty());

			if(UDataprepContentConsumer* Consumer = ConsumerPtr.Get())
			{
				FString TargetContentFolder( Consumer->GetTargetContentFolder() );

				if( TargetContentFolder.IsEmpty() )
				{
					TargetContentFolder = TEXT("/Content");
				}
				else if( TargetContentFolder.StartsWith( TEXT( "/Game" ) ) )
				{
					TargetContentFolder = TargetContentFolder.Replace( TEXT( "/Game" ), TEXT( "/Content" ) );
				}

				ContentFolderTextBox->SetText( FText::FromString( TargetContentFolder + TEXT( "/" ) ) );
			}
			else
			{
				ContentFolderTextBox->SetText( TAttribute<FText>() );
			}
		}

		void OnTextCommitted( const FText& NewText, ETextCommit::Type CommitType)
		{
			if(bCommitting)
			{
				return;
			}

			if(!bTextValid)
			{
				UpdateContentFolderText();
				bTextValid = true;
				return;
			}

			if( UDataprepContentConsumer* DataprepConsumer = ConsumerPtr.Get() )
			{
				bCommitting = true;

				FString NewContentFolder( NewText.ToString() );

				// Replace /Content/ with /Game/ since /Content is only used for display 
				if( NewContentFolder.StartsWith( TEXT("/Content") ) )
				{
					NewContentFolder = NewContentFolder.Replace( TEXT( "/Content" ), TEXT( "/Game" ) );
				}

				// Remove ending '/' if applicable
				if( !NewContentFolder.IsEmpty() && NewContentFolder[ NewContentFolder.Len()-1 ] == TEXT('/') )
				{
					NewContentFolder[ NewContentFolder.Len()-1 ] = 0;
					NewContentFolder = NewContentFolder.LeftChop(1);
				}

				if(NewContentFolder != DataprepConsumer->GetTargetContentFolder())
				{
					FScopedTransaction Transaction( LOCTEXT("Consumer_SetTargetContentFolder", "Set Target Content Folder") );

					FText ErrorReason;
					if( !DataprepConsumer->SetTargetContentFolder( NewContentFolder, ErrorReason ) )
					{
						Transaction.Cancel();
						UE_LOGF( LogDatasmithImport, Error, "%ls", *ErrorReason.ToString() );
					}

					UpdateContentFolderText();
				}

				bCommitting = false;
			}
		}

	private:
		TWeakObjectPtr< UDatasmithConsumer > ConsumerPtr;
		TSharedPtr< SEditableTextBox > ContentFolderTextBox;
		// Indicates committing is in progress
		bool bCommitting;
		// Indicates if text is valid or not
		bool bTextValid;
	};

	class SLevelProperty : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SLevelProperty) {}
		SLATE_END_ARGS()

	public:
		void Construct(const FArguments& InArgs, UDatasmithConsumer* InConsumer)
		{
			ConsumerPtr = InConsumer;
			bCommitting = false;
			bTextValid = true;

			FSlateFontInfo FontInfo = IDetailLayoutBuilder::GetDetailFont();

			ChildSlot
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.HAlign(EHorizontalAlignment::HAlign_Fill)
				.Padding(5.0f, 2.5f, 2.0f, 2.5f)
				[
					SAssignNew(LevelTextBox, SEditableTextBox)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.HintText(LOCTEXT("DataprepLevelProperty_HintText", "Set the name of the level to save in"))
					.IsReadOnly(false)
					.OnTextCommitted(FOnTextCommitted::CreateSP(this, &SLevelProperty::OnTextCommitted))
					.OnTextChanged(FOnTextChanged::CreateSP(this, &SLevelProperty::OnTextChanged))
				]
			];

			UpdateLevelText();
		}

	private:
		void UpdateLevelText()
		{
			LevelTextBox->SetError(FText::GetEmpty());

			if(UDatasmithConsumer* Consumer = ConsumerPtr.Get())
			{
				LevelTextBox->SetText( FText::FromString( Consumer->GetLevelName() ) );
			}
			else
			{
				LevelTextBox->SetText( TAttribute<FText>() );
			}
		}

		void OnTextChanged(const FText& InText)
		{
			const FString NewLevelName = InText.ToString();

			FText OutErrorMessage = FText::GetEmpty();

			int32 Index;
			if(NewLevelName.FindChar(L'/', Index))
			{
				OutErrorMessage = LOCTEXT( "InvalidLevelName_RelativePath", "Path or relative path is not accepted." );
			}
			else
			{
				VerifyObjectName(NewLevelName, OutErrorMessage);
			}

			bTextValid = OutErrorMessage.IsEmpty();

			LevelTextBox->SetError(OutErrorMessage);
		}

		void OnTextCommitted( const FText& NewText, ETextCommit::Type CommitType)
		{
			if(bCommitting)
			{
				return;
			}

			if(!bTextValid)
			{
				UpdateLevelText();
				bTextValid = true;
				return;
			}

			if( UDatasmithConsumer* DataprepConsumer = ConsumerPtr.Get() )
			{
				bCommitting = true;

				const FString NewLevelName = NewText.ToString();

				if(NewLevelName != DataprepConsumer->GetLevelName())
				{
					FScopedTransaction Transaction( LOCTEXT("Consumer_SetLevelName", "Set Output Level") );

					FText OutReason;
					if( !DataprepConsumer->SetLevelName( NewLevelName, OutReason ) )
					{
						Transaction.Cancel();

						UE_LOGF( LogDatasmithImport, Error, "Cannot create a level named %ls - %ls", *NewLevelName, *OutReason.ToString() );
					}

					UpdateLevelText();
				}

				bCommitting = false;
			}
		}

	private:
		TWeakObjectPtr< UDatasmithConsumer > ConsumerPtr;
		TSharedPtr< SEditableTextBox > LevelTextBox;
		// Indicates committing is in progress. Used to avoid re-entering UI event processing
		bool bCommitting;
		// Indicates if text in widget is valid or not
		bool bTextValid;
	};
}

void FDatasmithConsumerDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray< TWeakObjectPtr< UObject > > Objects;
	DetailBuilder.GetObjectsBeingCustomized( Objects );
	check( Objects.Num() > 0 );

	UDatasmithConsumer* Consumer = Cast< UDatasmithConsumer >(Objects[0].Get());
	check( Consumer );

	//FName CategoryName( TEXT("DatasmithFileProducerCustom") );
	IDetailCategoryBuilder& ImportSettingsCategoryBuilder = DetailBuilder.EditCategory( NAME_None, FText::GetEmpty(), ECategoryPriority::Important );

	TSharedRef<IPropertyHandle> TargetFolderHandle = DetailBuilder.GetProperty( TEXT("TargetContentFolder"), UDataprepContentConsumer::StaticClass() );
	TargetFolderHandle->MarkHiddenByCustomization();

	TSharedRef<IPropertyHandle> LevelNameHandle = DetailBuilder.GetProperty( TEXT("LevelName"), UDataprepContentConsumer::StaticClass() );
	LevelNameHandle->MarkHiddenByCustomization();

	TSharedRef<IPropertyHandle> LevelPathHandle = DetailBuilder.GetProperty( TEXT("OutputLevelPath"), UDatasmithConsumer::StaticClass() );
	LevelPathHandle->MarkHiddenByCustomization();

	{
		using namespace DatasmithConsumerDetailsUtil;

		FText PropertyName = FText::FromString( TEXT( "Folder" ) );
		FDetailWidgetRow& CustomRow = ImportSettingsCategoryBuilder.AddCustomRow( PropertyName );

		CustomRow.NameContent()
		[
			SNew( STextBlock )
			.Text( PropertyName )
			.Font( IDetailLayoutBuilder::GetDetailFont() )
		];

		CustomRow.ValueContent()
		.MinDesiredWidth( 2000.0f )
		[
			SNew( SFolderProperty, Consumer )
		];
	}

	{
		using namespace DatasmithConsumerDetailsUtil;

		FText PropertyName = FText::FromString( TEXT( "Level Name" ) );
		FDetailWidgetRow& CustomRow = ImportSettingsCategoryBuilder.AddCustomRow( PropertyName );

		CustomRow.NameContent()
		[
			SNew( STextBlock )
			.Text( PropertyName )
			.Font( IDetailLayoutBuilder::GetDetailFont() )
		];

		CustomRow.ValueContent()
		.MinDesiredWidth( 2000.0f )
		[
			SNew( SLevelProperty, Consumer )
		];
	}
}

#undef LOCTEXT_NAMESPACE