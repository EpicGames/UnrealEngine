// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "IDetailCustomization.h"

class IDetailLayoutBuilder;

// Customization of the details of the Datasmith Consumer for the data prep editor.
class DATASMITHIMPORTER_API FDatasmithConsumerDetails : public IDetailCustomization
{
public:
	static TSharedRef< IDetailCustomization > MakeDetails() { return MakeShared<FDatasmithConsumerDetails>(); };

	/** Called when details should be customized */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;
};

