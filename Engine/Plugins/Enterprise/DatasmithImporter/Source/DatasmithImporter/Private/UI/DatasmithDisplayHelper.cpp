// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/DatasmithDisplayHelper.h"

#include "Interfaces/IMainFrameModule.h"
#include "Internationalization/Internationalization.h"
#include "UI/DatasmithImportOptionsWindow.h"
#include "Widgets/SWindow.h"


#define LOCTEXT_NAMESPACE "DatasmithDisplayHelper"

namespace Datasmith
{

	FDisplayResult DisplayOptions(const TArray<TStrongObjectPtr<UObject>>& Options, const FDisplayParameters& Parameters)
	{
		TArray<UObject*> ImportOptions;
		for (const auto& OptionObject : Options)
		{
			ImportOptions.Add(OptionObject.Get());
		}

		return DisplayOptions(ImportOptions, Parameters);
	}

	FDisplayResult DisplayOptions(const TArray<UObject*>& Options, const FDisplayParameters& Parameters)
	{
		TSharedPtr<SWindow> ParentWindow;

		if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
		{
			IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
			ParentWindow = MainFrame.GetParentWindow();
		}

		const FText& Title = !Parameters.WindowTitle.IsEmpty() ? Parameters.WindowTitle : LOCTEXT("DatasmithOptionWindow_DefaultTitle", "Datasmith Options");
		TSharedRef<SWindow> Window = SNew(SWindow)
			.Title(Title)
			.SizingRule(ESizingRule::Autosized);

		TSharedPtr<SDatasmithOptionsWindow> OptionsWindow;
		Window->SetContent
		(
			SAssignNew(OptionsWindow, SDatasmithOptionsWindow)
			.ImportOptions(Options)
			.WidgetWindow(Window)
			.FileNameText(Parameters.FileLabel)
			.FilePathText(Parameters.FileTooltip)
			.PackagePathText(Parameters.PackageLabel)
			.ProceedButtonLabel(Parameters.ProceedButtonLabel)
			.ProceedButtonTooltip(Parameters.ProceedButtonTooltip)
			.CancelButtonLabel(Parameters.CancelButtonLabel)
			.CancelButtonTooltip(Parameters.CancelButtonTooltip)
			.bAskForSameOption(Parameters.bAskForSameOption)
			.MinDetailHeight(Parameters.MinDetailHeight)
			.MinDetailWidth(Parameters.MinDetailWidth)
		);

		FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);

		FDisplayResult Result;
		Result.bValidated = OptionsWindow->ShouldImport();
		Result.bUseSameOption = Parameters.bAskForSameOption && OptionsWindow->UseSameOptions();
		return Result;
	}

	FDisplayResult DisplayOptions(const TStrongObjectPtr<UObject>& Options, const FDisplayParameters& Parameters)
	{
		return DisplayOptions(TArray<TStrongObjectPtr<UObject>>{Options}, Parameters);
	}

} // ns Datasmith

#undef LOCTEXT_NAMESPACE
