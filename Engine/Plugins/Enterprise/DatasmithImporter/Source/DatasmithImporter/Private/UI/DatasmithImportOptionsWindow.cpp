// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/DatasmithImportOptionsWindow.h"

#include "DatasmithUtils.h"

#include "Styling/AppStyle.h"
#include "HAL/PlatformProcess.h"
#include "HttpModule.h"
#include "IDetailRootObjectCustomization.h"
#include "Interfaces/IHttpResponse.h"
#include "Interfaces/IHttpRequest.h"
#include "Misc/AutomationTest.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Templates/Function.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "DatasmithOptionsWindow"

FString SDatasmithOptionsWindow::DocumentationURL = TEXT("https://docs.unrealengine.com/en-US/WorkingWithContent/Importing/Datasmith/");

void SDatasmithOptionsWindow::Construct(const FArguments& InArgs)
{
	ImportOptions = InArgs._ImportOptions;
	Window = InArgs._WidgetWindow;
	bShouldProceed = false;
	bUseSameOptions = true;

	TSharedPtr<SBox> DetailsViewBox;
	FText VersionText(FText::Format(LOCTEXT("DatasmithOptionWindow_Version", " Version   {0}"), FText::FromString(FDatasmithUtils::GetEnterpriseVersionAsString())));

	TSharedRef< SCompoundWidget > WarningWidget = ConstructWarningWidget( InArgs._FileFormatVersion, InArgs._FileSDKVersion );

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(0, 10)
		.AutoHeight()
		[
			SNew(SInlineEditableTextBlock)
			.IsReadOnly(true)
			.Text(InArgs._FileNameText)
			.ToolTipText(InArgs._FilePathText)
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SInlineEditableTextBlock)
			.IsReadOnly(true)
			.Visibility(InArgs._PackagePathText.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible)
			.Text(InArgs._PackagePathText)
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2)
		[
			SAssignNew(DetailsViewBox, SBox)
			.MinDesiredHeight(InArgs._MinDetailHeight)
			.MinDesiredWidth(InArgs._MinDetailWidth)
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2)
		.HAlign(HAlign_Fill)
		[
			WarningWidget
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2)
		[
			SNew( SCheckBox )
			.Visibility(InArgs._bAskForSameOption ? EVisibility::Visible : EVisibility::Collapsed)
			.IsChecked(bUseSameOptions ? ECheckBoxState::Checked : ECheckBoxState::Unchecked)
			.OnCheckStateChanged(this, &SDatasmithOptionsWindow::OnSameOptionToggle)
			.Padding(5)
			[
				SNew(SInlineEditableTextBlock)
				.IsReadOnly(true)
				.Text(LOCTEXT("DatasmithOptionWindow_SameOption", "Use those options for all elements"))
			]
		]
		+ SVerticalBox::Slot()
		.Padding(0, 3)
		.AutoHeight()
		[
			SNew(SUniformGridPanel)
			.SlotPadding(0)
			+ SUniformGridPanel::Slot(0, 0)
			.HAlign(HAlign_Left)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(5)
				+ SUniformGridPanel::Slot(0, 0)
				.HAlign(HAlign_Left)
				[
					SNew(SInlineEditableTextBlock)
					.IsReadOnly(true)
					.Text(VersionText)
				]
				+ SUniformGridPanel::Slot(1, 0)
				.HAlign(HAlign_Left)
				[
					SNew(SBox)
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.WidthOverride(16)
					.HeightOverride(16)
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FAppStyle::GetBrush("Icons.Help"))
						.OnMouseButtonDown(this, &SDatasmithOptionsWindow::OnHelp)
					]
				]
			]
			+ SUniformGridPanel::Slot(1, 0)
			.HAlign(HAlign_Right)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(5, 0)
				.AutoWidth()
				.HAlign(HAlign_Right)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Text(InArgs._ProceedButtonLabel)
					.ToolTipText(InArgs._ProceedButtonTooltip)
					.OnClicked(this, &SDatasmithOptionsWindow::OnProceed)
				]
				+ SHorizontalBox::Slot()
				.Padding(5, 0)
				.AutoWidth()
				.HAlign(HAlign_Right)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Text(InArgs._CancelButtonLabel)
					.ToolTipText(InArgs._CancelButtonTooltip)
					.OnClicked(this, &SDatasmithOptionsWindow::OnCancel)
				]
			]
		]
	];

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bAllowMultipleTopLevelObjects = true;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	TSharedPtr<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	DetailsViewBox->SetContent(DetailsView.ToSharedRef());
	// @todo: Find a way to aggregate properties of all UObjects into one set
	DetailsView->SetObjects(ImportOptions);
}

void SDatasmithOptionsWindow::OnSameOptionToggle(ECheckBoxState InCheckState)
{
	bUseSameOptions = InCheckState == ECheckBoxState::Checked;
}

FReply SDatasmithOptionsWindow::OnHelp(const FGeometry& SenderGeometry, const FPointerEvent& MouseEvent)
{
	FPlatformProcess::LaunchURL( *GetDocumentationURL(), nullptr, nullptr );
	return FReply::Handled();
}

TSharedRef< SCompoundWidget > SDatasmithOptionsWindow::ConstructWarningWidget( float FileVersion, FText FileSDKVersion )
{
	const bool bIsOlder = FileVersion > 0.f && FileVersion < FDatasmithUtils::GetDatasmithFormatVersionAsFloat();
	const bool bIsNewer = FileVersion > 0.f && FileVersion > FDatasmithUtils::GetDatasmithFormatVersionAsFloat();
	const bool bIsIncompatible = FileVersion > 0.f && FMath::FloorToInt( FileVersion ) != FMath::FloorToInt( FDatasmithUtils::GetDatasmithFormatVersionAsFloat() );

	const bool bIsError = bIsNewer || bIsIncompatible;
	const bool bIsWarning = !bIsError && bIsOlder;

	FText UpdateText;

	if ( bIsNewer )
	{
		UpdateText = FText::Format( LOCTEXT( "UpdateVersion_UnrealStudio", "For best results, install Unreal Studio version {0} or later." ), FileSDKVersion );
	}
	else
	{
		UpdateText = LOCTEXT( "UpdateVersion_Datasmith", "For best results, update your Datasmith exporter plugin if possible." );
	}

	FText WarningText(
		FText::Format( LOCTEXT( "VersionWarning", "The selected file was created with a version of a Datasmith exporter ({0})\nthat doesn't match your Unreal Engine version ({1}).\n{2}" ),
		FileSDKVersion,
		FText::FromString( FDatasmithUtils::GetEnterpriseVersionAsString() ),
		UpdateText ) );

	FLinearColor MessageColor = bIsError ? FLinearColor::Red : FLinearColor::Yellow;
	FString IconName = TEXT("NotificationList.DefaultMessage");

	TSharedRef< SCompoundWidget > WarningWidget =
		SNew(SBorder)
		.BorderBackgroundColor( MessageColor )
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SGridPanel)
			+ SGridPanel::Slot(0, 0)
			.Padding(5)
			.HAlign(HAlign_Left)
			[
				SNew(SBox)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Top)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::GetBrush(*IconName))
				]
			]
			+ SGridPanel::Slot(1, 0)
			.Padding(5)
			.HAlign(HAlign_Left)
			[
				SNew(SInlineEditableTextBlock)
				.IsReadOnly(true)
				.Text(WarningText)
			]
		];

	WarningWidget->SetVisibility( ( bIsError || bIsWarning ) ? EVisibility::Visible : EVisibility::Collapsed );

	return WarningWidget;
}

class FDatasmithImportOptionHelpURLTest;

/**
 * Helper class used by FDatasmithImportOptionHelpURLTest to some context alive while we're waiting for the HttpRequest to complete.
 */
class FDatasmithImportOptionHelpURLTestHelper
{
public:
	FDatasmithImportOptionHelpURLTestHelper(FDatasmithImportOptionHelpURLTest* InOptionHelpURLTest, const FString& InURL)
		: OptionHelpURLTest(InOptionHelpURLTest)
		, URLToValidate(InURL)
	{}

	/**
	 * Send an asynchronous http request to the URLToValidate and validate that the http response is Ok(200)
	 */
	bool SendHttpRequest();

	/**
	 * Evaluate if the HttpRequest has completed at regular interval, until it completes or reach MaxTimeoutDelay.
	 */
	static void EvaluateHttpRequestResult(const TSharedPtr<FDatasmithImportOptionHelpURLTestHelper, ESPMode::ThreadSafe>& HelperPtr);

	FDatasmithImportOptionHelpURLTest* OptionHelpURLTest;
	FString URLToValidate;
	bool bHttpRequestCompleted = false;
	bool bHttpRequestHasValidURL = false;
	const float CommandDelay = 1.0f;
	float MaxTimeoutDelay = 0;
	int NumberOfTries = 0;
};

/**
 * Quick automated test to validate that the Datasmith Documentation url is valid. This is to avoid having it silently go bad.
 */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FDatasmithImportOptionHelpURLTest, "Editor.Import.Datasmith.Validate Documentation URL", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FDatasmithImportOptionHelpURLTest::RunTest(const FString& Parameters)
{
	TSharedPtr<FDatasmithImportOptionHelpURLTestHelper, ESPMode::ThreadSafe> URLTestHelper = MakeShared<FDatasmithImportOptionHelpURLTestHelper, ESPMode::ThreadSafe>(this, SDatasmithOptionsWindow::GetDocumentationURL());
	URLTestHelper->OptionHelpURLTest = this;

	if (!URLTestHelper->SendHttpRequest())
	{
		AddError(FString::Printf(TEXT("Could not send Http request to validate datasmith documentation URL: %s"), *SDatasmithOptionsWindow::GetDocumentationURL()));
		return false;
	}
	else
	{
		FDatasmithImportOptionHelpURLTestHelper::EvaluateHttpRequestResult(URLTestHelper);
		return true;
	}
}

bool FDatasmithImportOptionHelpURLTestHelper::SendHttpRequest()
{
	FHttpModule& HttpModule = FHttpModule::Get();
	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = HttpModule.CreateRequest();
	HttpRequest->SetVerb(TEXT("GET"));
	HttpRequest->SetURL(URLToValidate);
	HttpRequest->SetHeader(TEXT("Accept"), TEXT("*/*"));
	HttpRequest->OnProcessRequestComplete().BindLambda([this](FHttpRequestPtr Request, FHttpResponsePtr Response, bool bConnectedSuccessfully)
	{
		if (bConnectedSuccessfully)
		{
			bHttpRequestHasValidURL = Response->GetResponseCode() == EHttpResponseCodes::Ok;
		}
		bHttpRequestCompleted = true;
	});

	TOptional<float> RequestTimeout = HttpRequest->GetTimeout();
	MaxTimeoutDelay = RequestTimeout.IsSet() ? RequestTimeout.GetValue() : HttpModule.GetHttpActivityTimeout();

	return HttpRequest->ProcessRequest();
}

void FDatasmithImportOptionHelpURLTestHelper::EvaluateHttpRequestResult(const TSharedPtr<FDatasmithImportOptionHelpURLTestHelper, ESPMode::ThreadSafe>& HelperPtr)
{
	const float TotalWaitTime = HelperPtr->NumberOfTries * HelperPtr->CommandDelay;
	if (!HelperPtr->bHttpRequestCompleted && TotalWaitTime < HelperPtr->MaxTimeoutDelay)
	{
		// As long as the request is not complete and we have not waited for the timeout duration, add a delayed command to reevaluate later.
		// This is to avoid having a single big delayed command which could take up to >30 sec, even when the request might complete under a second.
		HelperPtr->OptionHelpURLTest->AddCommand(new FDelayedFunctionLatentCommand([HelperPtr]()
		{
			FDatasmithImportOptionHelpURLTestHelper::EvaluateHttpRequestResult(HelperPtr);
		}, HelperPtr->CommandDelay));
	}
	else if (!(HelperPtr->bHttpRequestCompleted && HelperPtr->bHttpRequestHasValidURL))
	{
		HelperPtr->OptionHelpURLTest->AddError(FString::Printf(TEXT("Http request could not reach datasmith documentation URL, link might be outdated: %s"), *HelperPtr->URLToValidate));
	}

	HelperPtr->NumberOfTries++;
};

#undef LOCTEXT_NAMESPACE
