// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/DatasmithStyle.h"

#include "Framework/Application/SlateApplication.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"

TUniquePtr< FSlateStyleSet > FDatasmithStyle::DatasmithStyleInstance = NULL;

void FDatasmithStyle::Initialize()
{
	if (!DatasmithStyleInstance.IsValid())
	{
		DatasmithStyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*DatasmithStyleInstance);
	}
}

void FDatasmithStyle::Shutdown()
{
	if (DatasmithStyleInstance.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*DatasmithStyleInstance);
		DatasmithStyleInstance.Reset();
	}
}

FName FDatasmithStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("DatasmithStyle"));
	return StyleSetName;
}

FName FDatasmithStyle::GetContextName()
{
	static FName ContextName(TEXT("Datasmith"));
	return ContextName;
}

#define IMAGE_BRUSH( RelativePath, ... ) FSlateImageBrush( Style->RootToContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )

const FVector2D Icon20x20(20.0f, 20.0f);
const FVector2D Icon40x40(40.0f, 40.0f);

TUniquePtr< FSlateStyleSet > FDatasmithStyle::Create()
{
	TUniquePtr< FSlateStyleSet > Style = MakeUnique<FSlateStyleSet>(GetStyleSetName());
	Style->SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Enterprise"));
	return Style;
}

void FDatasmithStyle::SetIcon(const FString& StyleName, const FString& ResourcePath)
{
	FSlateStyleSet* Style = DatasmithStyleInstance.Get();

	FString Name(GetContextName().ToString());
	Name = Name + "." + StyleName;
	Style->Set(*Name, new IMAGE_BRUSH(ResourcePath, Icon40x40));

	Name += ".Small";
	Style->Set(*Name, new IMAGE_BRUSH(ResourcePath, Icon20x20));

	FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
}

#undef IMAGE_BRUSH

const ISlateStyle& FDatasmithStyle::Get()
{
	check(DatasmithStyleInstance);
	return *DatasmithStyleInstance;
}
