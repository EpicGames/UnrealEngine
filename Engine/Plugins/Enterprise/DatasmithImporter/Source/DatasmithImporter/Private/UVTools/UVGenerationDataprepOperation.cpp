// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVGenerationDataprepOperation.h"

#include "UVTools/UVGenerationFlattenMapping.h"
#include "UVTools/UVGenerationUtils.h"

#include "Async/ParallelFor.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshActor.h"
#include "MeshDescription.h"
#include "OverlappingCorners.h"
#include "Templates/Greater.h"

void UUVGenerationFlattenMappingOperation::OnExecution_Implementation(const FDataprepContext& InContext)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UUVGenerationFlattenMappingOperation::OnExecution_Implementation)

	TSet<UStaticMesh*> StaticMeshes;

	for (UObject* Object : InContext.Objects)
	{
		if (AActor* Actor = Cast<AActor>(Object))
		{
			if (!IsValidChecked(Actor) || Actor->IsUnreachable())
			{
				continue;
			}

			TInlineComponentArray<UStaticMeshComponent*> ComponentArray;
			Actor->GetComponents(ComponentArray);
			for (UStaticMeshComponent* MeshComponent : ComponentArray)
			{
				if (UStaticMesh* StaticMesh = MeshComponent->GetStaticMesh())
				{
					StaticMeshes.Add(StaticMesh);
				}
			}
		}
		else if (UStaticMeshComponent* MeshComponent = Cast<UStaticMeshComponent>(Object))
		{
			if (UStaticMesh* StaticMesh = MeshComponent->GetStaticMesh())
			{
				StaticMeshes.Add(StaticMesh);
			}
		}
		else if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Object))
		{
			StaticMeshes.Add(StaticMesh);
		}
	}

	// The actual work
	auto Compute = [&](UStaticMesh* StaticMesh)
	{
		for (int32 LodIndex = 0; LodIndex < StaticMesh->GetNumSourceModels(); ++LodIndex)
		{
			if (!StaticMesh->IsMeshDescriptionValid(LodIndex))
			{
				continue;
			}

			int32 TargetChannel = UVChannel;
			if (ChannelSelection == EUnwrappedUVDatasmithOperationChannelSelection::FirstEmptyChannel)
			{
				TargetChannel = UVGenerationUtils::GetNextOpenUVChannel(StaticMesh, LodIndex);
				if (TargetChannel < 0)
				{
					continue;
				}
			}

			FMeshBuildSettings& BuildSettings = StaticMesh->GetSourceModel(LodIndex).BuildSettings;
			FMeshDescription& MeshDescription = *StaticMesh->GetMeshDescription(LodIndex);

			UUVGenerationFlattenMapping::GenerateUVs(MeshDescription, TargetChannel, BuildSettings.bRemoveDegenerates, AngleThreshold);
		}
	};

	// Start with the biggest mesh first to help balancing tasks on threads
	TArray<UStaticMesh*> StaticMeshArray(StaticMeshes.Array());
	Algo::SortBy(
		StaticMeshArray,
		[](const UStaticMesh* Mesh) { return Mesh->IsMeshDescriptionValid(0) ? Mesh->GetMeshDescription(0)->Vertices().Num() : 0; },
		TGreater<>()
	);

	ParallelFor(StaticMeshArray.Num(),
		[&](int32 Index)
		{
			Compute(StaticMeshArray[Index]);
		},
		EParallelForFlags::Unbalanced
	);
}