// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DataprepOperation.h"

#include "UVGenerationDataprepOperation.generated.h"

UENUM()
enum class EUnwrappedUVDatasmithOperationChannelSelection : uint8
{
	FirstEmptyChannel UMETA(Tooltip = "Generate the unwrapped UV in the first UV channel that is empty."),
	SpecifyChannel UMETA(Tooltip = "Manually select the target UV channel for the unwrapped UV generation."),
};

UCLASS(Experimental, Category = MeshOperation, Meta = (DisplayName = "Generate Unwrapped UVs", ToolTip = "For each static mesh to process, generate an unwrapped UV map in the specified channel"))
class DATASMITHIMPORTER_API UUVGenerationFlattenMappingOperation : public UDataprepOperation
{
	GENERATED_BODY()

	UUVGenerationFlattenMappingOperation()
		: ChannelSelection(EUnwrappedUVDatasmithOperationChannelSelection::FirstEmptyChannel),
		UVChannel(0),
		AngleThreshold(66.f)
	{
	}

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "UV Generation Settings")
	EUnwrappedUVDatasmithOperationChannelSelection ChannelSelection;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "UV Generation Settings", meta = (DisplayName = "UV Channel", ToolTip = "The UV channel where to generate the flatten mapping", ClampMin = "0", ClampMax = "7", EditCondition = "ChannelSelection == EUnwrappedUVDatasmithOperationChannelSelection::SpecifyChannel")) //Clampmax is from MAX_MESH_TEXTURE_COORDS_MD - 1
	int UVChannel;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "UV Generation Settings", meta = (ClampMin = "1", ClampMax = "90"))
	float AngleThreshold;

	//~ Begin UDataprepOperation Interface
public:
	virtual FText GetCategory_Implementation() const override
	{
		return FDataprepOperationCategories::MeshOperation;
	}

protected:
	virtual void OnExecution_Implementation(const FDataprepContext& InContext) override;
	//~ End UDataprepOperation Interface
};