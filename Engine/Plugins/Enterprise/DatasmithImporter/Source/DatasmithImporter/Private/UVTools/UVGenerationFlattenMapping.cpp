// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVTools/UVGenerationFlattenMapping.h"

#include "Algo/Sort.h"
#include "Async/ParallelFor.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Engine/EngineTypes.h"
#include "Engine/StaticMesh.h"
#include "Math/Box2D.h"
#include "Math/ConvexHull2d.h"
#include "Math/Quat.h"
#include "Math/RotationMatrix.h"
#include "Math/UnrealMathUtility.h"
#include "MeshAttributes.h"
#include "MeshDescription.h"
#include "Misc/ScopedSlowTask.h"
#include "OverlappingCorners.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"
#include "Templates/TypeHash.h"

#define FLATTEN_AREA_WEIGHT 0.7f

class FUVGenerationFlattenMappingInternal
{
public:
	struct FaceStruct
	{
		double Area;
		int32 V0;
		int32 V1;
		int32 V2;

		int32 Edges[3];

		FVector Normal;
		FVector VertexPosition[3];
		FVector2D Uvs[3];
		int32 UvIndex[3];

		int32 Group;
		int32 PolygonID;
	};

	static TArray<FaceStruct> GetFacesFrom(FMeshDescription& InMesh, const TArray<int32>& OverlappingCorners={});
	static void CalculateUVs(TArray<FaceStruct>& FlattenFaces, int32 VertexNum, float AngleThreshold, float AreaWeight);

private:
	static void GetUVIslands(const TArray< TArray< FaceStruct* > >& FaceGroups, const TArray< int32 >& EdgeInUse, TArray< TArray< FaceStruct* > >& IslandsOfFaces);
	static void BoundsOfIsland(TArray< FaceStruct* >& Faces, double& MinX, double& MinY, double& MaxX, double& MaxY);
	static void GetIslandSizes(TArray< TArray< FaceStruct* > >& IslandsOfFaces, double& MaxWidth, double& MaxHeight, TArray< FVector4 >& IslandsBoundingBoxes);

	/** Rotate each island so that its UVs are mostly axis aligned */
	static void AlignIslandsOnAxes(TArray< TArray< FaceStruct* > >& IslandsOfFaces);
	static void ExpandIslands(TArray< TArray< FaceStruct* > >& IslandsOfFaces);

	static void AlignIslandOnAxes(TArray< FaceStruct* >& IslandOfFaces);
	static void ExpandIsland(int32 IslandIndex, TArray< TArray<FaceStruct*> >& IslandsOfFaces, const TArray< FVector4 >& IslandsBoundingBoxes, double MaxSize, double ScaleFactor);

	static FVector GetBestCandidateProjection(const TSet< FaceStruct* >& TempMeshFaces, FVector CandidatesProjectVec[19]);
	static void CreateGroupUVIslands(TSet< FaceStruct* > TempMeshFaces, TArray< TArray< FaceStruct* > >& IslandsOfFaces, const TArray< int32 >& EdgeInUse, float AngleThreshold, float AreaWeight);

	struct FVertexPair
	{
		int32 A;
		int32 B;
		FVertexPair() : A(0), B(0) {}
		FVertexPair(int32 A, int32 B) : A(A), B(B) {}
	};

	static void AddFaceVertexPair(FaceStruct* Face, TArray< TArray< FVertexPair > >& VertexPair, TArray<int32>& EdgeInUse);

	static int32 SplitInGeomGroups(TArray<FaceStruct>& Faces);

	static FVector2D Rotate2D(FVector2D Point, float Angle);
	static TArray< FVector2D > ComputeOrientedBox2D(const TArray< FVector2D >& ConvexHull);
};

void FUVGenerationFlattenMappingInternal::AddFaceVertexPair(FaceStruct* Face, TArray< TArray< FVertexPair > >& VertexPair, TArray<int32>& EdgeInUse)
{
	ensure( VertexPair.IsValidIndex( Face->V0 ) && VertexPair.IsValidIndex( Face->V1 ) && VertexPair.IsValidIndex( Face->V2 ) );

	if ( !VertexPair.IsValidIndex( Face->V0 ) || !VertexPair.IsValidIndex( Face->V1 ) || !VertexPair.IsValidIndex( Face->V2 ) )
	{
		return;
	}

	//if the pair of vertex have not been used initialize it

	int32 Index = -1;
	for (int32 i = 0; i < VertexPair[Face->V0].Num(); i++)
	{
		if (VertexPair[Face->V0][i].A == Face->V1)
		{
			Index = (int32)VertexPair[Face->V0][i].B;
		}
	}

	if (Index == -1)
	{
		FVertexPair t1(Face->V1, EdgeInUse.Num());
		VertexPair[Face->V0].Add(t1);

		FVertexPair t2(Face->V0, EdgeInUse.Num());
		VertexPair[Face->V1].Add(t2);
		Face->Edges[0] = EdgeInUse.Num();
		EdgeInUse.Add(1);
	}
	else //increment its usage
	{
		EdgeInUse[Index]++;
		Face->Edges[0] = Index;
	}

	Index = -1;
	for (int32 i = 0; i < VertexPair[Face->V1].Num(); i++)
	{
		if (VertexPair[Face->V1][i].A == Face->V2)
		{
			Index = (int32)VertexPair[Face->V1][i].B;
		}
	}

	if (Index == -1)
	{
		FVertexPair t1(Face->V2, EdgeInUse.Num());
		VertexPair[Face->V1].Add(t1);

		FVertexPair t2(Face->V1, EdgeInUse.Num());
		VertexPair[Face->V2].Add(t2);
		Face->Edges[1] = EdgeInUse.Num();
		EdgeInUse.Add(1);
	}
	else //increment its usage
	{
		EdgeInUse[Index]++;
		Face->Edges[1] = Index;
	}

	Index = -1;
	for (int32 i = 0; i < VertexPair[Face->V2].Num(); i++)
	{
		if (VertexPair[Face->V2][i].A == Face->V0)
		{
			Index = (int32)VertexPair[Face->V2][i].B;
		}
	}

	if (Index == -1)
	{
		FVertexPair t1(Face->V0, EdgeInUse.Num());
		VertexPair[Face->V2].Add(t1);

		FVertexPair t2(Face->V2, EdgeInUse.Num());
		VertexPair[Face->V0].Add(t2);
		Face->Edges[2] = EdgeInUse.Num();
		EdgeInUse.Add(1);
	}
	else //increment its usage
	{
		EdgeInUse[Index]++;
		Face->Edges[2] = Index;
	}
}

int32 FUVGenerationFlattenMappingInternal::SplitInGeomGroups(TArray<FaceStruct>& Faces)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithFlattenMappingInternal::SplitInGeomGroups)

	TMap< int32, TArray<FaceStruct*, TInlineAllocator<8> > > HashVertexFaces;

	TArray<FaceStruct*> FriendFaces;
	FriendFaces.AddDefaulted(Faces.Num());

	for (int32 Face = 0; Face < Faces.Num(); Face++)
	{
		FriendFaces[Face] = &Faces[Face];
	}

	int32 maxVert = 0;

	for (int32 Face = 0; Face < FriendFaces.Num(); Face++)
	{
		FaceStruct* FriendFace = FriendFaces[ Face ];

		maxVert = FMath::Max(maxVert, FMath::Max(FriendFace->V0, FMath::Max(FriendFace->V1, FriendFace->V2)));

		if (!HashVertexFaces.Contains(FriendFace->V0))
		{
			TArray<FaceStruct*, TInlineAllocator<8> > Vec;
			Vec.Add(FriendFace);
			HashVertexFaces.Emplace(FriendFace->V0, Vec);
		}
		else
		{
			HashVertexFaces.Find(FriendFace->V0)->Add(FriendFace);
		}

		if (!HashVertexFaces.Contains(FriendFace->V1))
		{
			TArray<FaceStruct*, TInlineAllocator<8> > Vec;
			Vec.Add(FriendFace);
			HashVertexFaces.Emplace(FriendFace->V1, Vec);
		}
		else
		{
			HashVertexFaces.Find(FriendFace->V1)->Add(FriendFace);
		}

		if (!HashVertexFaces.Contains(FriendFace->V2))
		{
			TArray<FaceStruct*, TInlineAllocator<8> > Vec;
			Vec.Add(FriendFace);
			HashVertexFaces.Emplace(FriendFace->V2, Vec);
		}
		else
		{
			HashVertexFaces.Find(FriendFace->V2)->Add(FriendFace);
		}
	}

	int32 CurrentGroup = 0;
	TSet<int32> AlreadyParsedVertex;
	TSet<int32> InUseVertex;

	// Since we're going to remove elements until the TMap is empty,
	// it is preferable to avoid using iterators as it is incredibly
	// slow to scan bitarrays to find used bits when the map is
	// really sparse.
	TArray<int32> Keys;
	HashVertexFaces.GenerateKeyArray(Keys);

	for (int32 Key : Keys)
	{
		TArray<FaceStruct*, TInlineAllocator<8>>* Values = HashVertexFaces.Find(Key);
		if (Values == nullptr)
		{
			continue;
		}

		InUseVertex.Reset();

		for (FaceStruct* Face : *Values)
		{
			Face->Group = CurrentGroup;
			InUseVertex.Emplace(Face->V0);
			InUseVertex.Emplace(Face->V1);
			InUseVertex.Emplace(Face->V2);
		}

		HashVertexFaces.Remove(Key);

		while (InUseVertex.Num() > 0)
		{
			auto VerticesIt = InUseVertex.CreateIterator();

			int32 currVertex = *VerticesIt;
			if (HashVertexFaces.Contains(currVertex))
			{
				const TArray<FaceStruct*, TInlineAllocator<8>> & fv = *HashVertexFaces.Find(currVertex);
				for (int32 i = 0; i < fv.Num(); i++)
				{
					fv[i]->Group = CurrentGroup;
					if (!AlreadyParsedVertex.Contains(fv[i]->V0))
					{
						InUseVertex.Emplace(fv[i]->V0);
					}

					if (!AlreadyParsedVertex.Contains(fv[i]->V1))
					{
						InUseVertex.Emplace(fv[i]->V1);
					}

					if (!AlreadyParsedVertex.Contains(fv[i]->V2))
					{
						InUseVertex.Emplace(fv[i]->V2);
					}
				}
				HashVertexFaces.Remove(currVertex);
			}
			AlreadyParsedVertex.Emplace(currVertex);
			VerticesIt.RemoveCurrent();
		}
		CurrentGroup++;
	}
	return CurrentGroup;
}


void FUVGenerationFlattenMappingInternal::GetUVIslands(const TArray< TArray<FaceStruct*> >& FaceGroups, const TArray<int32>& EdgeInUse, TArray< TArray<FaceStruct*> >& IslandsOfFaces)
{
	int32 FaceGroupIndex = FaceGroups.Num();

	while (FaceGroupIndex > 0)
	{
		FaceGroupIndex--;
		const TArray<FaceStruct*>& FacesAtThisProjection = FaceGroups[FaceGroupIndex];

		if (FacesAtThisProjection.Num() <= 0)
		{
			continue;
		}

		// Build edge dict
		TArray< TArray<int32> > EdgeFaces;
		EdgeFaces.AddDefaulted( EdgeInUse.Num() );

		for (int32 f = 0; f < FacesAtThisProjection.Num(); f++)
		{
			for (int32 k = 0; k < 3; k++)
			{
				int32 EdgeIndex = FacesAtThisProjection[f]->Edges[k];
				if (EdgeInUse[EdgeIndex] > 1)//not seam
				{
					EdgeFaces[EdgeIndex].Add(f);
				}
			}
		}

		TArray<int32> FacesToProcess;
		FacesToProcess.AddUninitialized( FacesAtThisProjection.Num() );

		for ( int32 i = 0; i < FacesAtThisProjection.Num(); ++i )
		{
			FacesToProcess[i] = i;
		}

		TArray<bool> ProcessedFaces;
		ProcessedFaces.AddZeroed( FacesAtThisProjection.Num() );

		TArray<int32> CurrentFacesToSearch;
		CurrentFacesToSearch.Reserve( FacesAtThisProjection.Num() );

		for ( int32 i = 0; i < FacesToProcess.Num(); ++i )
		{
			int32 FaceIndex = FacesToProcess[i];

			if ( ProcessedFaces[FaceIndex] )
			{
				continue;
			}

			TArray<FaceStruct*> NewIslandOfFaces;
			NewIslandOfFaces.Add( FacesAtThisProjection[ FaceIndex ] );

			CurrentFacesToSearch.Empty( FacesAtThisProjection.Num() );
			CurrentFacesToSearch.Add( FaceIndex );

			while ( CurrentFacesToSearch.Num() > 0 )
			{
				int32 CurrentFaceIndex = CurrentFacesToSearch.Pop( EAllowShrinking::No );

				for (int32 Edge = 0; Edge < 3; Edge++)
				{
					int32 EdgeIndex = FacesAtThisProjection[CurrentFaceIndex]->Edges[Edge];
					for (int32 FaceAtEdge = 0; FaceAtEdge < EdgeFaces[EdgeIndex].Num(); FaceAtEdge++)
					{
						int32 OtherFaceIndex = EdgeFaces[EdgeIndex][FaceAtEdge];
						if (CurrentFaceIndex != OtherFaceIndex && !ProcessedFaces[OtherFaceIndex])
						{
							CurrentFacesToSearch.Add( OtherFaceIndex ); // Mark it for further search on this same Island
							NewIslandOfFaces.Add( FacesAtThisProjection[OtherFaceIndex] );
							ProcessedFaces[OtherFaceIndex] = true;
						}
					}
				}

				ProcessedFaces[CurrentFaceIndex] = true;
			}

			IslandsOfFaces.Add( NewIslandOfFaces );
		}
	}
}

void FUVGenerationFlattenMappingInternal::BoundsOfIsland(TArray<FaceStruct*>& Faces, double& MinX, double& MinY, double& MaxX, double& MaxY)
{
	MinX = Faces[0]->Uvs[0][0];
	MaxX = MinX;
	MinY = Faces[0]->Uvs[0][1];
	MaxY = MinY;

	double X, Y;
	for (int32 i = 0; i < Faces.Num(); i++)
	{
		for (int32 j = 0; j < 3;j++)
		{
			X = Faces[i]->Uvs[j][0];
			Y = Faces[i]->Uvs[j][1];

			MinX = FMath::Min(X, MinX);
			MinY = FMath::Min(Y, MinY);

			MaxX = FMath::Max(X, MaxX);
			MaxY = FMath::Max(Y, MaxY);
		}
	}
}

void FUVGenerationFlattenMappingInternal::GetIslandSizes(TArray< TArray<FaceStruct*> >& IslandsOfFaces, double& MaxWidth, double& MaxHeight, TArray<FVector4>& IslandsBoundingBoxes)
{
	double MinX, MinY, MaxX, MaxY;
	MinX = MinY = MaxX = MaxY = 0.0f;

	MaxWidth = 0;
	MaxHeight = 0;

	for (int32 IslandId = 0; IslandId < IslandsOfFaces.Num(); IslandId++)
	{
		BoundsOfIsland(IslandsOfFaces[IslandId], MinX, MinY, MaxX, MaxY);
		IslandsBoundingBoxes.Add(FVector4(MinX, MinY, MaxX, MaxY));

		double Width = MaxX - MinX;
		double Height = MaxY - MinY;

		MaxWidth = FMath::Max( Width, MaxWidth );
		MaxHeight = FMath::Max( Height, MaxHeight );
	}
}

void FUVGenerationFlattenMappingInternal::AlignIslandsOnAxes(TArray< TArray< FaceStruct* > >& IslandsOfFaces)
{
	ParallelFor( IslandsOfFaces.Num(), [ &IslandsOfFaces ]( int32 Index )
	{
		AlignIslandOnAxes( IslandsOfFaces[ Index ] );
	} );
}

void FUVGenerationFlattenMappingInternal::AlignIslandOnAxes(TArray< FaceStruct* >& Island)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithFlattenMappingInternal::AlignIslandOnAxes)

	TSet< FVector2D > Vertices;
	Vertices.Reserve( Island.Num() * 3 );

	for ( const FaceStruct* Face : Island )
	{
		Vertices.Add( Face->Uvs[0] );
		Vertices.Add( Face->Uvs[1] );
		Vertices.Add( Face->Uvs[2] );
	}

	TArray< FVector2D > VerticesArray = Vertices.Array();

	TArray< int32 > ConvexHullIndices;
	ConvexHull2D::ComputeConvexHull( VerticesArray, ConvexHullIndices );

	TArray< FVector2D > ConvexHull;
	ConvexHull.Reserve( ConvexHullIndices.Num() );

	for ( int32 Index : ConvexHullIndices )
	{
		ConvexHull.Emplace( VerticesArray[ Index ] );
	}

	FVector IslandTangent(ForceInit);

	TArray< FVector2D > OBB = ComputeOrientedBox2D( ConvexHull );

	if ( OBB.Num() >= 3 )
	{
		FVector LongestOOBBSide( ForceInit );

		for ( int32 i = 0; i < 3; ++i ) // Test the first two sides
		{
			FVector OOBBSide = FVector( OBB[ i + 1 ] - OBB[ i ], 0.f );

			if ( OOBBSide.Size() > LongestOOBBSide.Size() )
			{
				LongestOOBBSide = OOBBSide;
			}
		}

		IslandTangent = LongestOOBBSide;
	}

	FQuat IslandRotation( ForceInit );

	if ( !IslandTangent.IsNearlyZero() )
	{
		IslandTangent.Normalize();

		IslandRotation = FQuat::FindBetweenNormals( IslandTangent, FVector::ForwardVector );
	}

	for ( int32 Face = 0; Face < Island.Num(); Face++ )
	{
		for ( int32 UVIndex = 0; UVIndex < 3; UVIndex++ )
		{
			FVector RotatedUVs = IslandRotation.RotateVector( FVector( Island[Face]->Uvs[UVIndex], 0.f ) );
			Island[Face]->Uvs[UVIndex] = FVector2D( RotatedUVs.X, RotatedUVs.Y );
		}
	}
}

void FUVGenerationFlattenMappingInternal::ExpandIslands(TArray< TArray<FaceStruct*> >& IslandsOfFaces)
{
	TArray<FVector4> IslandsBoundingBoxes;
	double MaxWidth = 0;
	double MaxHeight = 0;
	GetIslandSizes(IslandsOfFaces, MaxWidth, MaxHeight, IslandsBoundingBoxes);

	double MaxSize = FMath::Max(MaxWidth, MaxHeight);
	double ScaleFactor = 1.0f / MaxSize;

	ParallelFor( IslandsOfFaces.Num(), [ &IslandsOfFaces, &IslandsBoundingBoxes, MaxSize, ScaleFactor ]( int32 Index )
	{
		ExpandIsland( Index, IslandsOfFaces, IslandsBoundingBoxes, MaxSize, ScaleFactor );
	} );
}

void FUVGenerationFlattenMappingInternal::ExpandIsland(int32 IslandIndex, TArray< TArray<FaceStruct*> >& IslandsOfFaces, const TArray< FVector4 >& IslandsBoundingBoxes, double MaxSize, double ScaleFactor)
{
	double CurrentWidth = IslandsBoundingBoxes[IslandIndex][2] - IslandsBoundingBoxes[IslandIndex][0];
	double CurrentHeight = IslandsBoundingBoxes[IslandIndex][3] - IslandsBoundingBoxes[IslandIndex][1];

	double OffsetX = (1.0 - (CurrentWidth / MaxSize)) / 2.0;
	double OffsetY = (1.0 - (CurrentHeight / MaxSize)) / 2.0;

	TArray< FaceStruct* >& Island = IslandsOfFaces[ IslandIndex ];

	for ( FaceStruct* Face : Island )
	{
		for (int32 UVIndex = 0; UVIndex < 3; UVIndex++)
		{
			Face->Uvs[UVIndex][0] = OffsetX + (Face->Uvs[UVIndex][0] - IslandsBoundingBoxes[IslandIndex][0]) * ScaleFactor;
			Face->Uvs[UVIndex][1] = OffsetY + (Face->Uvs[UVIndex][1] - IslandsBoundingBoxes[IslandIndex][1]) * ScaleFactor;
		}
	}
}


#define NUM_CANDIDATES 7
FVector FUVGenerationFlattenMappingInternal::GetBestCandidateProjection(const TSet<FaceStruct*>& TempMeshFaces, FVector CandidatesProjectVec[19])
{
#if 0
	double candidatesArea[19] = { 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 };

	for (int32 c = 0; c<NUM_CANDIDATES; c++)
	{
		//this projection has not been used!
		if (CandidatesProjectVec[c][3]>0 || c == 0)
		{
			for (auto itr = TempMeshFaces.begin(); itr != TempMeshFaces.end(); ++itr)
			{
				double dotproduct = CandidatesProjectVec[c].DotProduct((*itr)->Normal);
				if (dotproduct> USER_PROJECTION_LIMIT_CONVERTED)
				{
					candidatesArea[c] += (*itr)->Area;
				}
			}
		}
	}
	//candidatesArea[0] *= 1.5;
	for (int32 c = 7; c < NUM_CANDIDATES; c++)
	{
		candidatesArea[c] *= 0.75;
	}
	int32 bestCandidate = 0;
	double bestCandidateArea = candidatesArea[0];

	for (int32 c = 1; c < NUM_CANDIDATES; c++)
	{
		if (candidatesArea[c] > bestCandidateArea)
		{
			bestCandidateArea = candidatesArea[c];
			bestCandidate = c;
		}
	}

	//this projection has been used!
	if (bestCandidate != 0)
		CandidatesProjectVec[bestCandidate][3] = -1;

	return CandidatesProjectVec[bestCandidate];
#else
	return CandidatesProjectVec[0];
#endif
}

void FUVGenerationFlattenMappingInternal::CreateGroupUVIslands(TSet<FaceStruct*> TempMeshFaces, TArray< TArray<FaceStruct*> >& IslandsOfFaces, const TArray<int32>& EdgeInUse, float AngleThreshold, float AreaWeight)
{
	const float SlopeProjectionLimit = FMath::Cos(FMath::DegreesToRadians(AngleThreshold));
	const float SlopeProjectionLimitHalf = FMath::Cos(FMath::DegreesToRadians(AngleThreshold / 2.f));

	//Initially it comes from the biggest Face
	double MostDifferentAngle = -1.0f;
	int32 MostDifferentIndex = 0;

	double BiggestArea = -1;

	FVector CandidatesProjectVec[19];
	CandidatesProjectVec[0] = FVector(0.f, 0.f, 1.f);
	CandidatesProjectVec[1] = FVector(0.f, 0.f, 1.f);
	CandidatesProjectVec[2] = FVector(0.f, 0.f, -1.f);
	CandidatesProjectVec[3] = FVector(0.f, 1.f, 0.f);
	CandidatesProjectVec[4] = FVector(0.f, -1.f, 0.f);
	CandidatesProjectVec[5] = FVector(1.f, 0.f, 0.f);
	CandidatesProjectVec[6] = FVector(-1.f, 0.f, 0.f);

	const double Sqrt2 = FMath::Sqrt(2.f);

	CandidatesProjectVec[7] = FVector(0, Sqrt2, Sqrt2);
	CandidatesProjectVec[8] = FVector(0, Sqrt2, -Sqrt2);
	CandidatesProjectVec[9] = FVector(0, -Sqrt2, Sqrt2);
	CandidatesProjectVec[10] = FVector(0, -Sqrt2, -Sqrt2);

	CandidatesProjectVec[11] = FVector(Sqrt2, Sqrt2, 0.f);
	CandidatesProjectVec[12] = FVector(Sqrt2, -Sqrt2, 0.f);
	CandidatesProjectVec[13] = FVector(-Sqrt2, Sqrt2, 0.f);
	CandidatesProjectVec[14] = FVector(-Sqrt2, -Sqrt2, 0.f);

	CandidatesProjectVec[15] = FVector(Sqrt2, 0.f, Sqrt2);
	CandidatesProjectVec[16] = FVector(Sqrt2, 0.f, -Sqrt2);
	CandidatesProjectVec[17] = FVector(-Sqrt2, 0.f, Sqrt2);
	CandidatesProjectVec[18] = FVector(-Sqrt2, 0.f, -Sqrt2);


	//Faces on the new projection
	TArray<FaceStruct*> NextProjectionFaces;
	//projection Normal vectors
	TArray<FVector> ProjectVecs;

	TArray<FaceStruct*> ValidFaces;
	int32 MaxEdgeIndex = 0;
	for (auto itr = TempMeshFaces.CreateIterator(); itr; ++itr)
	{
		if ((*itr)->Area > BiggestArea )
		{
			BiggestArea = (*itr)->Area;

			CandidatesProjectVec[0] = (*itr)->Normal;
		}
		MaxEdgeIndex = FMath::Max(MaxEdgeIndex, (*itr)->V0);
		MaxEdgeIndex = FMath::Max(MaxEdgeIndex, (*itr)->V1);
		MaxEdgeIndex = FMath::Max(MaxEdgeIndex, (*itr)->V2);
		ValidFaces.Add((*itr));
	}

	if (BiggestArea <= 0)
	{
		return;
	}

	FVector NewProjectVec = GetBestCandidateProjection(TempMeshFaces, CandidatesProjectVec);

	bool bExit = false;

	while (bExit == false)
	{
		// Search for the biggest Face and the Faces that share its Normal
		for (auto itr = TempMeshFaces.CreateIterator(); itr; ++itr)
		{
			// we use half the angle limit to find the Faces that shares the projection
			if (FVector::DotProduct( NewProjectVec, (*itr)->Normal ) > SlopeProjectionLimitHalf)
			{
				NextProjectionFaces.Add((*itr));
				itr.RemoveCurrent();
			}
		}

		FVector AverageProjectionVector = FVector(0, 0, 0);
		for (int32 Face = 0; Face < (int32)NextProjectionFaces.Num(); Face++)
		{
			AverageProjectionVector += NextProjectionFaces[Face]->Normal * ((NextProjectionFaces[Face]->Area * AreaWeight) + (1.0 - AreaWeight));
		}
		AverageProjectionVector.Normalize();

		if (AverageProjectionVector[0] != 0 || AverageProjectionVector[1] != 0 || AverageProjectionVector[2] != 0) //Avoid NAN
		{
			ProjectVecs.Add(AverageProjectionVector);
		}

		//Get the opposite angle for the next loop
		MostDifferentAngle = 1.0;
		MostDifferentIndex = -1;

		FaceStruct* MostUniqueFace = NULL;
		for (auto itr = TempMeshFaces.CreateIterator(); itr; ++itr)
		{
			float AngleDifference = -1.f;

			//Get the closest Vec angle we are to.
			for (int32 p = 0; p < ProjectVecs.Num(); p++)
			{
				AngleDifference = FMath::Max( AngleDifference, FVector::DotProduct( ProjectVecs[p], (*itr)->Normal ) );
			}

			if (AngleDifference < MostDifferentAngle)
			{
				MostUniqueFace = (*itr);
				MostDifferentAngle = AngleDifference;
			}
		}

		if (MostUniqueFace && MostDifferentAngle < SlopeProjectionLimit)
		{
			NewProjectVec = MostUniqueFace->Normal;
			NextProjectionFaces.Empty();
			NextProjectionFaces.Add(MostUniqueFace);
			TempMeshFaces.Remove(MostUniqueFace);
			NextProjectionFaces.Empty();
		}
		else
		{
			if (ProjectVecs.Num() >= 1 || TempMeshFaces.Num()==0)
			{
				bExit = true;
			}
		}
	}

	if (ProjectVecs.Num() < 1)
	{
		return;
	}

	//Test for an alternative Group to avoid Unreal Lightmass issue with peninsular Faces
	TArray< TArray<FaceStruct*> > FaceProjectionGroupList;
	FaceProjectionGroupList.AddDefaulted(ProjectVecs.Num());

	// For big meshes, it is preferable to use TMap instead
	// of TArray here to reduce the initialization cost and
	// and memory usage when vertex count is high. The TMap
	// usage is going to be really sparse anyway.
	TArray< TMap<int32, int32> > EdgeProjectionGroupList;
	EdgeProjectionGroupList.SetNum(ProjectVecs.Num());

	//best angle and second best for each Face
	TArray<FVector2D> BestAnglesForFace;
	BestAnglesForFace.AddDefaulted(ValidFaces.Num());

	for (int32 FaceIndex = 0; FaceIndex < ValidFaces.Num(); FaceIndex++)
	{
		FVector Normal = ValidFaces[FaceIndex]->Normal;

		// Initialize first
		double BestAngle = -1;
		double SecondBestAngle = -1;
		int32 BestAngleIdx = -1;
		int32 SecondBestAngleIdx = -1;

		// Cycle through the remaining, first already done
		for (int32 p = 0; p < ProjectVecs.Num(); p++)
		{
			float NewAngle = FVector::DotProduct( Normal, ProjectVecs[p] );

			if (NewAngle > SecondBestAngle)
			{
				if (NewAngle >= BestAngle)
				{
					SecondBestAngle = BestAngle;
					BestAngle = NewAngle;
					SecondBestAngleIdx = BestAngleIdx;
					BestAngleIdx = p;
				}
				else
				{
					SecondBestAngle = NewAngle;
					SecondBestAngleIdx = p;
				}
			}

		}
		BestAnglesForFace[FaceIndex][0] = FMath::Max(0, BestAngleIdx);
		BestAnglesForFace[FaceIndex][1] = SecondBestAngleIdx;

		if (SecondBestAngle > SlopeProjectionLimit)
		{
			BestAnglesForFace[FaceIndex][1] = (int32)SecondBestAngleIdx;
		}
		else
		{
			BestAnglesForFace[FaceIndex][1] = (int32)BestAnglesForFace[FaceIndex][0];
		}
	}


	//how many times is used that vertex on that Group
	for (int32 FaceIndex = (int32)ValidFaces.Num() - 1; FaceIndex >= 0; FaceIndex--)
	{
		int32 PrefAngleIdx = (int32)BestAnglesForFace[FaceIndex][0];
		const FaceStruct* Face = ValidFaces[FaceIndex];

		TMap<int32, int32>& EdgeProjectionGroup = EdgeProjectionGroupList[PrefAngleIdx];
		EdgeProjectionGroup.FindOrAdd(Face->V0)++;
		EdgeProjectionGroup.FindOrAdd(Face->V1)++;
		EdgeProjectionGroup.FindOrAdd(Face->V2)++;
	}

	TMap< int32, TSet< int32 > > AnglesIndexToFaces;

	for (int32 FaceIndex = 0; FaceIndex < ValidFaces.Num(); FaceIndex++)
	{
		int32 PrefAngleIdx = (int32)BestAnglesForFace[FaceIndex][0];
		int32 AltAngleIdx = (int32)BestAnglesForFace[FaceIndex][1];

		if (AltAngleIdx >= 0 && AltAngleIdx != PrefAngleIdx)
		{
			AnglesIndexToFaces.FindOrAdd( AltAngleIdx ).Add( FaceIndex );
		}
	}

	TArray< int32 > AnglesIndicesToParse;
	for ( auto It = AnglesIndexToFaces.CreateConstIterator(); It; ++It )
	{
		AnglesIndicesToParse.Add( It->Key );
	}

	//move isolated triangles to alternative Group if possible
	while( AnglesIndicesToParse.Num() )
	{
		int32 AnglesIndexToParse = AnglesIndicesToParse.Pop();

		for( auto It = AnglesIndexToFaces[ AnglesIndexToParse ].CreateIterator(); It; ++It )
		{
			int32 FaceIndex = *It;

			const FaceStruct* Face = ValidFaces[FaceIndex];

			int32 PrefAngleIdx = (int32)BestAnglesForFace[FaceIndex][0];
			int32 AltAngleIdx = (int32)BestAnglesForFace[FaceIndex][1];

			//if there is an alternative we'll check which one shares more edges
			if (AltAngleIdx >= 0 && AltAngleIdx != PrefAngleIdx)
			{
				TMap<int32, int32>& EdgeProjectionGroup = EdgeProjectionGroupList[PrefAngleIdx];

				int32 UnconnectedVert = 0;

				const int32* Value = EdgeProjectionGroup.Find(Face->V0);
				if (Value == nullptr || *Value < 2)
				{
					UnconnectedVert++;
				}

				Value = EdgeProjectionGroup.Find(Face->V1);
				if (Value == nullptr || *Value < 2)
				{
					UnconnectedVert++;
				}

				Value = EdgeProjectionGroup.Find(Face->V2);
				if (Value == nullptr || *Value < 2)
				{
					UnconnectedVert++;
				}

				TMap<int32, int32>& AltEdgeProjectionGroup = EdgeProjectionGroupList[AltAngleIdx];

				int32 AltUnconnectedVert = 0;
				Value = AltEdgeProjectionGroup.Find(Face->V0);
				if (Value == nullptr || *Value < 1)
				{
					AltUnconnectedVert++;
				}
				Value = AltEdgeProjectionGroup.Find(Face->V1);
				if (Value == nullptr || *Value < 1)
				{
					AltUnconnectedVert++;
				}
				Value = AltEdgeProjectionGroup.Find(Face->V2);
				if (Value == nullptr || *Value < 1)
				{
					AltUnconnectedVert++;
				}

				if (UnconnectedVert > AltUnconnectedVert)
				{
					EdgeProjectionGroup.FindChecked(Face->V0)--;
					EdgeProjectionGroup.FindChecked(Face->V1)--;
					EdgeProjectionGroup.FindChecked(Face->V2)--;

					AltEdgeProjectionGroup.FindOrAdd(Face->V0)++;
					AltEdgeProjectionGroup.FindOrAdd(Face->V1)++;
					AltEdgeProjectionGroup.FindOrAdd(Face->V2)++;

					BestAnglesForFace[FaceIndex][0] = (int32)BestAnglesForFace[FaceIndex][1];
					BestAnglesForFace[FaceIndex][1] = -1;

					It.RemoveCurrent();

					if ( AnglesIndexToFaces.Contains( PrefAngleIdx ) )
					{
						AnglesIndicesToParse.AddUnique( PrefAngleIdx );
					}

					if ( AnglesIndexToFaces.Contains( AltAngleIdx ) )
					{
						AnglesIndicesToParse.AddUnique( AltAngleIdx );
					}
				}
			}
			else
			{
				It.RemoveCurrent();
			}
		}
	}

	//Fill the FacesPerProjection where each angle id has the Faces that use it
	for (int32 FaceIndex = (int32)ValidFaces.Num() - 1; FaceIndex >= 0; FaceIndex--)
	{
		int32 PrefAngleIdx = (int32)BestAnglesForFace[FaceIndex][0];
		FaceProjectionGroupList[PrefAngleIdx].Add(ValidFaces[FaceIndex]);
	}


	//FaceProjectionGroupList contains the Faces that match its vector
	for (int32 i = 0; i < ProjectVecs.Num(); i++)
	{
		// Account for ProjectVecs having no Faces.
		if (FaceProjectionGroupList[i].Num() < 1)
		{
			continue;
		}

		FQuat ProjectionToUp = FQuat::FindBetweenNormals( ProjectVecs[i], FVector::UpVector );

		//Get the UVs for Projection using its vertex
		for (int32 f = 0; f < FaceProjectionGroupList[i].Num(); f++)
		{
			FaceStruct* Face = FaceProjectionGroupList[i][f];

			for (int32 v = 0; v < 3; v++)
			{
				FVector Temp = ProjectionToUp.RotateVector(Face->VertexPosition[v]);
				Face->Uvs[v] = FVector2D(Temp[0], Temp[1]);
			}
		}
	}

	GetUVIslands(FaceProjectionGroupList, EdgeInUse, IslandsOfFaces);
}

FVector2D FUVGenerationFlattenMappingInternal::Rotate2D( FVector2D Point, float Angle )
{
	float RotatedX = Point.X * FMath::Cos( Angle ) - Point.Y * FMath::Sin( Angle );
	float RotatedY = Point.X * FMath::Sin( Angle ) + Point.Y * FMath::Cos( Angle );

	return FVector2D( RotatedX, RotatedY );
}

TArray< FVector2D > FUVGenerationFlattenMappingInternal::ComputeOrientedBox2D( const TArray< FVector2D >& ConvexHull )
{
	if ( ConvexHull.Num() <= 1 )
	{
		return TArray< FVector2D >();
	}

	FBox2D SmallestAABBox( ForceInit );
	float MinAngle = 0.f;

	for ( int32 i = 0; i < ConvexHull.Num(); ++i )
	{
		int32 NextIndex = i + 1;

		FVector2D Current( ConvexHull[ i ] );
		FVector2D Next( ConvexHull[ NextIndex % ConvexHull.Num() ] );

		FVector2D Segment = Current - Next;

		FVector2D MinPoint( TNumericLimits<float>::Max(), TNumericLimits<float>::Max() );
		FVector2D MaxPoint( TNumericLimits<float>::Lowest(), TNumericLimits<float>::Lowest() );

		// Angle of segment to X axis
		float Angle = FMath::Acos( Segment.GetSafeNormal().X );
		if ( Angle > PI * 0.5f )
		{
			Angle = -1.f * ( PI - Angle );
		}

		// Rotate every point and get min and max values for each direction
		for ( FVector2D Point : ConvexHull )
		{
			FVector2D RotatedPoint = Rotate2D( Point, -Angle );

			MinPoint.X = FMath::Min( MinPoint.X, RotatedPoint.X );
			MaxPoint.X = FMath::Max( MaxPoint.X, RotatedPoint.X );

			MaxPoint.Y = FMath::Max( MaxPoint.Y, RotatedPoint.Y );
			MinPoint.Y = FMath::Min( MinPoint.Y, RotatedPoint.Y );
		}

		FBox2D AABBox( MinPoint, MaxPoint );

		if ( !SmallestAABBox.bIsValid || SmallestAABBox.GetArea() > AABBox.GetArea() )
		{
			SmallestAABBox = AABBox;
			MinAngle = Angle;
		}
	}

	TArray< FVector2D > Result;

	if ( SmallestAABBox.bIsValid )
	{
		FVector2D BoxCenter;
		FVector2D BoxExtents;
		SmallestAABBox.GetCenterAndExtents( BoxCenter, BoxExtents );

		FVector2D BoxMin = BoxCenter - BoxExtents;
		FVector2D BoxMax = BoxCenter + BoxExtents;

		FVector2D BottomLeft = BoxCenter + FVector2D( BoxMin.X, BoxMin.Y );
		FVector2D TopLeft = BoxCenter + FVector2D( BoxMin.X, BoxMax.Y );
		FVector2D TopRight = BoxCenter + FVector2D( BoxMax.X, BoxMax.Y );
		FVector2D BottomRight = BoxCenter + FVector2D( BoxMax.X, BoxMin.Y );

		// Rotate SmallestAABBox back to create the OBB
		Result.Add( Rotate2D( BottomLeft, MinAngle ) );
		Result.Add( Rotate2D( TopLeft, MinAngle ) );
		Result.Add( Rotate2D( TopRight, MinAngle ) );
		Result.Add( Rotate2D( BottomRight, MinAngle ) );
	}

	return Result;
}

void FUVGenerationFlattenMappingInternal::CalculateUVs(TArray<FaceStruct>& FlattenFaces, int32 VertexNum, float AngleThreshold, float AreaWeight)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithFlattenMappingInternal::CalculateUVs)

	int32 NumGeomGroups = FUVGenerationFlattenMappingInternal::SplitInGeomGroups(FlattenFaces);

	//temporal array of Faces with surface Area bigger than 0 that can be deleted freely
	TArray< TSet< FUVGenerationFlattenMappingInternal::FaceStruct* > > TempMeshFaces;

	TempMeshFaces.AddDefaulted(NumGeomGroups);

	//too many Groups could cause a major slowdown
	if (NumGeomGroups > 256)
	{
		int32 PerGroup = (NumGeomGroups) / 256;
		int32 MaxGroup = 0;
		for (int32 Face = 0; Face < FlattenFaces.Num(); Face++)
		{
			if (FlattenFaces[Face].Area > 0)
			{
				int32 CurrentGroup = FlattenFaces[Face].Group % PerGroup;
				if (CurrentGroup > MaxGroup)
				{
					MaxGroup = CurrentGroup;
				}
				TempMeshFaces[CurrentGroup].Add(&FlattenFaces[Face]);
			}
		}
	}
	else
	{
		for (int32 Face = 0; Face < FlattenFaces.Num(); Face++)
		{
			if (FlattenFaces[Face].Area > 0)
			{
				TempMeshFaces[FlattenFaces[Face].Group].Add(&FlattenFaces[Face]);
			}
		}
	}

	TempMeshFaces.RemoveAllSwap([](const TSet< FUVGenerationFlattenMappingInternal::FaceStruct* >& Element)
	{
		return Element.Num() <= 0;
	}, EAllowShrinking::No);

	const int32 NumGeomGroupsNotEmpty = TempMeshFaces.Num();

	//Accessing IslandsOfFaces is not thread safe, well use a list per Group
	//and we'll add all them
	TArray< TArray< FUVGenerationFlattenMappingInternal::FaceStruct* > > IslandsOfFaces;
	TArray< TArray< TArray< FUVGenerationFlattenMappingInternal::FaceStruct* > > > IslandsOfFacesbyGroup;

	IslandsOfFacesbyGroup.AddDefaulted(NumGeomGroupsNotEmpty);

	ParallelFor(NumGeomGroupsNotEmpty, [&TempMeshFaces, &IslandsOfFacesbyGroup, VertexNum, AngleThreshold, AreaWeight](int32 Index)
	{
		if (TempMeshFaces[Index].Num() > 0)
		{
			// cannot create a edge usage vector like
			// TArray<int32> EdgeInUse(InMesh.GetMeshEdgeCount(), 0);
			// since fbx sdk is superslow here!
			// we'll create a pair of vertex connection
			TArray<int32> EdgeInUse;

			TArray< TArray< FVertexPair > > VertexPair;
			VertexPair.AddDefaulted(VertexNum);

			for (auto itr = TempMeshFaces[Index].CreateIterator(); itr; ++itr)
			{
				FUVGenerationFlattenMappingInternal::AddFaceVertexPair((*itr), VertexPair, EdgeInUse);
			}
			FUVGenerationFlattenMappingInternal::CreateGroupUVIslands(TempMeshFaces[Index], IslandsOfFacesbyGroup[Index], EdgeInUse, AngleThreshold, AreaWeight);
		}
	});

	for (int32 i = 0; i < IslandsOfFacesbyGroup.Num(); i++)
	{
		IslandsOfFaces.Append(IslandsOfFacesbyGroup[i]);
	}

	FUVGenerationFlattenMappingInternal::AlignIslandsOnAxes(IslandsOfFaces); // Need to align before expanding
	FUVGenerationFlattenMappingInternal::ExpandIslands(IslandsOfFaces); //overlap each Island

	// Save island index in Group member of FaceStruct. Used later on to detect shared vertex instances across island
	int32 IslandIndex = 0;
	for (TArray< FUVGenerationFlattenMappingInternal::FaceStruct* >& Faces : IslandsOfFaces)
	{
		for(FUVGenerationFlattenMappingInternal::FaceStruct* Face : Faces)
		{
			Face->Group = IslandIndex;
		}

		++IslandIndex;
	}

}

TArray<FUVGenerationFlattenMappingInternal::FaceStruct> FUVGenerationFlattenMappingInternal::GetFacesFrom(FMeshDescription& InMesh, const TArray<int32>& InstanceIDMapping)
{
	int32 NumTris = InMesh.Triangles().Num();

	TArray< FUVGenerationFlattenMappingInternal::FaceStruct > FlattenFaces;
	FlattenFaces.Reserve(NumTris);
	const TVertexAttributesConstRef<FVector3f> VertexPositions = InMesh.VertexAttributes().GetAttributesRef<FVector3f>(MeshAttribute::Vertex::Position);
	const TVertexInstanceAttributesConstRef<FVector3f> Normals = InMesh.VertexInstanceAttributes().GetAttributesRef<FVector3f>(MeshAttribute::VertexInstance::Normal);

	const FVertexInstanceArray& Instances = InMesh.VertexInstances();
	check(InstanceIDMapping.Num() == 0 || InstanceIDMapping.Num() == Instances.GetArraySize());

	for (const FPolygonID PolygonID : InMesh.Polygons().GetElementIDs())
	{
		TArrayView<const FTriangleID> TriangleIDs = InMesh.GetPolygonTriangles(PolygonID);
		for (const FTriangleID& TriangleID : TriangleIDs)
		{
			FaceStruct Face;
			Face.Group = -1;
			Face.PolygonID = PolygonID.GetValue();

			FVertexInstanceID InstanceIDs[3];
			FVertexID VertexIDs[3];
			for (int32 i = 0; i < 3; ++i)
			{
				FVertexInstanceID OriginalID = InMesh.GetTriangleVertexInstance(TriangleID, i);
				InstanceIDs[i] = OriginalID;
				VertexIDs[i] = InMesh.GetVertexInstanceVertex(InstanceIDs[i]);
				Face.UvIndex[i] = OriginalID.GetValue();
				Face.Uvs[i] = FVector2D::ZeroVector;
			}

			const FVector& P0 = (FVector)VertexPositions[VertexIDs[0]];
			const FVector& P1 = (FVector)VertexPositions[VertexIDs[1]];
			const FVector& P2 = (FVector)VertexPositions[VertexIDs[2]];

			// Compute triangle normal and area
			FVector RawNormal = (P1 - P2) ^ (P0 - P2);
			double RawNormalSize = RawNormal.Size();
			Face.Normal = RawNormal/RawNormalSize;
			Face.Area = 0.5 * RawNormalSize;

			// Filter degenerated faces
			if (Face.Area < SMALL_NUMBER)
			{
				// The face is degenerate, use the wedge normal mean
				Face.Normal = (FVector)(Normals[InstanceIDs[0]] + Normals[InstanceIDs[1]] + Normals[InstanceIDs[2]]) / 3.f;
				if (!Face.Normal.Normalize())
				{
					continue;
				}
			}

			Face.V0 = VertexIDs[0].GetValue();
			Face.V1 = VertexIDs[1].GetValue();
			Face.V2 = VertexIDs[2].GetValue();

			Face.VertexPosition[0] = 0.01 * P0;
			Face.VertexPosition[1] = 0.01 * P1;
			Face.VertexPosition[2] = 0.01 * P2;

			FlattenFaces.Add(Face);
		}
	}

	return FlattenFaces;
}

void UUVGenerationFlattenMapping::GenerateFlattenMappingUVs(UStaticMesh* InStaticMesh, int32 UVChannel, float AngleThreshold)
{
	if (InStaticMesh == nullptr)
	{
		return;
	}

	InStaticMesh->Modify();
	FScopedSlowTask SlowTask(InStaticMesh->GetNumSourceModels(), FText::FromString(InStaticMesh->GetName()));

	for (int32 LodIndex = 0; LodIndex < InStaticMesh->GetNumSourceModels(); ++LodIndex)
	{
		SlowTask.EnterProgressFrame(1);

		if (!InStaticMesh->IsMeshDescriptionValid(LodIndex))
		{
			continue;
		}

		FMeshBuildSettings& BuildSettings = InStaticMesh->GetSourceModel(LodIndex).BuildSettings;
		FMeshDescription& MeshDescription = *InStaticMesh->GetMeshDescription(LodIndex);
		UUVGenerationFlattenMapping::GenerateUVs(MeshDescription, UVChannel, BuildSettings.bRemoveDegenerates, AngleThreshold);

		UStaticMesh::FCommitMeshDescriptionParams CommitMeshDescriptionParam;
		CommitMeshDescriptionParam.bUseHashAsGuid = true;
		InStaticMesh->CommitMeshDescription(LodIndex, CommitMeshDescriptionParam);
	}

	InStaticMesh->PostEditChange();
}

void UUVGenerationFlattenMapping::GenerateUVs(FMeshDescription& InMesh, int32 UVChannel, bool bRemoveDegenerates, float AngleThreshold)
{
	if (!ensure(UVChannel >= 0 && UVChannel < MAX_MESH_TEXTURE_COORDS_MD))
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UUVGenerationFlattenMapping::GenerateUVs);

	const TArray<int32> OverlappingCorners = GetOverlappingCornersRemapping(InMesh, bRemoveDegenerates);
	const TVertexInstanceAttributesRef<FVector2f> UVChannels = InMesh.VertexInstanceAttributes().GetAttributesRef<FVector2f>(MeshAttribute::VertexInstance::TextureCoordinate);
	if (UVChannels.GetNumChannels() < UVChannel + 1)
	{
		UVChannels.SetNumChannels(UVChannel + 1);
	}

	// Get the Internal Face Array
	TArray< FUVGenerationFlattenMappingInternal::FaceStruct > FlattenFaces = FUVGenerationFlattenMappingInternal::GetFacesFrom(InMesh, OverlappingCorners);

	FUVGenerationFlattenMappingInternal::CalculateUVs(FlattenFaces, InMesh.Vertices().Num(), AngleThreshold, FLATTEN_AREA_WEIGHT);

	// Write unwrapped UVs back to mesh description
	// Vertex instances shared across islands will be split
	TMap<int32, int32> ProcessedVertexInstances;
	TSet<int32> PolygonsToRetriangulate;
	ProcessedVertexInstances.Reserve( InMesh.VertexInstances().Num() );

	TAttributesSet<FVertexInstanceID>& VertexInstanceAttributesSet = InMesh.VertexInstanceAttributes();

	for (FUVGenerationFlattenMappingInternal::FaceStruct& Face : FlattenFaces)
	{
		bool bFoundSharedInstance = false;
		for(int32 Corner = 0; Corner < 3; ++Corner)
		{
			int32* GroupIndexPtr = ProcessedVertexInstances.Find( Face.UvIndex[Corner] );

			// First occurrence of vertex instance
			if( GroupIndexPtr == nullptr )
			{
				UVChannels.Set( FVertexInstanceID( Face.UvIndex[Corner] ), UVChannel, FVector2f(Face.Uvs[Corner]) );
				ProcessedVertexInstances.Add( Face.UvIndex[Corner], Face.Group );
			}
			// Vertex instance is shared across islands. Split it.
			else if( *GroupIndexPtr != Face.Group)
			{
				// Create new vertex instance based on original vertex instance
				FVertexInstanceID OriginalID( Face.UvIndex[Corner] );
				FVertexInstanceID NewInstanceID = InMesh.CreateVertexInstance( InMesh.GetVertexInstanceVertex( OriginalID ) );

				// Copy attributes from original vertex instance to new one
				VertexInstanceAttributesSet.ForEach(
					[&](const FName AttributeName, auto AttributeArrayRef)
					{
						for(int32 Index = 0; Index < AttributeArrayRef.GetNumChannels(); ++Index)
						{
							AttributeArrayRef.Set( NewInstanceID, Index, AttributeArrayRef.Get(OriginalID, Index) );
						}
					}
				);

				// Update new vertex instance with flattened uvs
				UVChannels.Set( NewInstanceID, UVChannel, FVector2f(Face.Uvs[Corner]) );

				// Update contour of associated polygon with new vertex instance
				const FPolygonID PolygonID( Face.PolygonID );
				TArray<FVertexInstanceID, TInlineAllocator<8>> PolygonVerts = InMesh.GetPolygonVertexInstances<TInlineAllocator<8>>( PolygonID );
				const int32 FoundIndex = PolygonVerts.Find( OriginalID );
				check(FoundIndex != INDEX_NONE);
				PolygonVerts[FoundIndex] = NewInstanceID;
				InMesh.SetPolygonVertexInstances( PolygonID, PolygonVerts );

				// Mark polygon for re-triangulation
				PolygonsToRetriangulate.Add( Face.PolygonID );

				Face.UvIndex[Corner] = NewInstanceID.GetValue();
			}
		}
	}

	// Re-triangulate polygons which vertex instances have been split
	for (int32 PolygonIDValue : PolygonsToRetriangulate)
	{
		InMesh.ComputePolygonTriangulation( FPolygonID( PolygonIDValue ) );
	}
}

TArray<int32> UUVGenerationFlattenMapping::GetOverlappingCornersRemapping(const FMeshDescription& InMeshDescription, const bool bRemoveDegenerates)
{
	float ComparisonThreshold = bRemoveDegenerates ? THRESH_POINTS_ARE_SAME : 0.0f;
	FOverlappingCorners OverlappingCorners;
	FStaticMeshOperations::FindOverlappingCorners(OverlappingCorners, InMeshDescription, ComparisonThreshold);

	// Create a instanceID mapping that merge overlapping vertices.
	// In order to produce a better UV mapping
	const FVertexInstanceArray& Instances = InMeshDescription.VertexInstances();
	int32 InstanceCount = Instances.GetArraySize();

	// maps original FVertexInstanceID value to replacement FVertexInstanceID value
	TArray<int32> OverlappingVerticesRemapping;
	OverlappingVerticesRemapping.AddUninitialized(InstanceCount);

	TArray<FVertexInstanceID> UsedVertexInstancePerWedge;
	// Store used vertex instance ID per wedge.
	// This secondary mapping is necessary because FMeshDescriptionOperations::FindOverlappingCorners uses wedge indices instead of FVertexInstanceID's
	int WedgeCount = 3 * InMeshDescription.Triangles().Num();
	UsedVertexInstancePerWedge.AddDefaulted(WedgeCount);

	int32 CurrentWedge = 0;
	for (const FPolygonID PolygonID : InMeshDescription.Polygons().GetElementIDs())
	{
		for (const FTriangleID& TriangleIDs : InMeshDescription.GetPolygonTriangles(PolygonID))
		{
			for (int32 Corner = 0; Corner < 3; ++Corner)
			{
				const FVertexInstanceID TriangleVertexInstanceID = InMeshDescription.GetTriangleVertexInstance(TriangleIDs, Corner);
				FVertexInstanceID ReplacementVertexInstanceId = TriangleVertexInstanceID;

				// Note: FindIfOverlapping uses wedges index as inputs/output, see FMeshDescriptionOperations::FindOverlappingCorners
				const TArray<int32>& OverlappingWedges = OverlappingCorners.FindIfOverlapping(CurrentWedge);
				if (OverlappingWedges.Num())
				{
					// Array is sorted (see FOverlappingCorners::FinishAdding) we use the first wedge of the overlapping wedges pool
					int32 ReplacementWedge = OverlappingWedges[0];
					if (ReplacementWedge < CurrentWedge)
					{
						ReplacementVertexInstanceId = UsedVertexInstancePerWedge[ReplacementWedge];
					}
				}

				OverlappingVerticesRemapping[TriangleVertexInstanceID.GetValue()] = ReplacementVertexInstanceId.GetValue();
				UsedVertexInstancePerWedge[CurrentWedge] = ReplacementVertexInstanceId;
				++CurrentWedge;
			}
		}
	}

	return OverlappingVerticesRemapping;
}
