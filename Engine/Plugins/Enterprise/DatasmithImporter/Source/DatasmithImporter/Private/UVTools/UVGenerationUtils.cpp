// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVTools/UVGenerationUtils.h"

#include "Engine/StaticMesh.h"
#include "MeshDescription.h"
#include "MeshUtilitiesCommon.h"
#include "OverlappingCorners.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"


int32 UVGenerationUtils::GetNextOpenUVChannel(UStaticMesh* StaticMesh, int32 LODIndex)
{
	if (!StaticMesh->IsMeshDescriptionValid(LODIndex))
	{
		return -1;
	}

	FMeshDescription* Mesh = StaticMesh->GetMeshDescription(LODIndex);
	int32 NumberOfUVs = StaticMesh->GetNumUVChannels(LODIndex);
	int32 FirstEmptyUVs = 0;

	for (; FirstEmptyUVs < NumberOfUVs; ++FirstEmptyUVs)
	{
		const TVertexInstanceAttributesConstRef<FVector2f> UVChannels = Mesh->VertexInstanceAttributes().GetAttributesRef<FVector2f>(MeshAttribute::VertexInstance::TextureCoordinate);
		const FVector2f DefValue = UVChannels.GetDefaultValue();
		bool bHasNonDefaultValue = false;

		for (FVertexInstanceID InstanceID : Mesh->VertexInstances().GetElementIDs())
		{
			if (UVChannels.Get(InstanceID, FirstEmptyUVs) != DefValue)
			{
				bHasNonDefaultValue = true;
				break;
			}
		}

		if (!bHasNonDefaultValue)
		{
			//We found an "empty" channel.
			break;
		}
	}
	return FirstEmptyUVs < MAX_MESH_TEXTURE_COORDS_MD ? FirstEmptyUVs : -1;
};

void UVGenerationUtils::SetupGeneratedLightmapUVResolution(UStaticMesh* StaticMesh, int32 LODIndex)
{
	if (!StaticMesh->IsMeshDescriptionValid(LODIndex))
	{
		return;
	}

	FMeshBuildSettings& BuildSettings = StaticMesh->GetSourceModel(LODIndex).BuildSettings;
	FMeshDescription& Mesh = *StaticMesh->GetMeshDescription(LODIndex);

	// Determine the absolute minimum lightmap resolution that can be used for packing
	float ComparisonThreshold = BuildSettings.bRemoveDegenerates ? THRESH_POINTS_ARE_SAME : 0.0f;
	FOverlappingCorners OverlappingCorners;
	FStaticMeshOperations::FindOverlappingCorners(OverlappingCorners, Mesh, ComparisonThreshold);

	// Packing expects at least one texel per chart. This is the absolute minimum to generate valid UVs.
	int32 ChartCount = FStaticMeshOperations::GetUVChartCount(Mesh, BuildSettings.SrcLightmapIndex, ELightmapUVVersion::Latest, OverlappingCorners);
	const int32 AbsoluteMinResolution = 1 << FMath::CeilLogTwo(FMath::Sqrt(static_cast<float>(ChartCount)));
	const int32 LightmapResolution = FMath::Clamp(BuildSettings.MinLightmapResolution, AbsoluteMinResolution, 512);

	BuildSettings.MinLightmapResolution = LightmapResolution;
}
