// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithImportFactoryHelper.h"

#include "Components/PrimitiveComponent.h"
#include "Components/SceneComponent.h"
#include "ContentBrowserModule.h"
#include "CoreMinimal.h"
#include "Dom/JsonObject.h"
#include "Editor.h"
#include "HAL/FileManager.h"
#include "IContentBrowserSingleton.h"
#include "LevelEditorViewport.h"
#include "Math/Box.h"
#include "Modules/ModuleManager.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"

extern UNREALED_API class FLevelEditorViewportClient* GCurrentLevelEditingViewportClient;

namespace DatasmithImportFactoryHelper
{
	TSharedPtr<FJsonObject> LoadJsonFile(const FString& JsonFilename)
	{
		FArchive* FileAr = IFileManager::Get().CreateFileReader(*JsonFilename);
		if (FileAr)
		{
			TSharedPtr<FJsonObject> RootJsonObject = MakeShareable(new FJsonObject());

			using FJsonReader = TJsonReader<TCHAR>;
			using FJsonReaderFactory = TJsonReaderFactory<TCHAR>;

			TSharedRef<FJsonReader> Reader = FJsonReaderFactory::Create(FileAr);
			bool bJsonLoaded = FJsonSerializer::Deserialize(Reader, RootJsonObject);
			FileAr->Close();

			delete FileAr;

			if (!bJsonLoaded)
			{
				return TSharedPtr<FJsonObject>();
			}

			return RootJsonObject;
		}

		return TSharedPtr<FJsonObject>();
	}

	void ComputeBounds( USceneComponent* ActorComponent, FBox& Bounds )
	{
		if ( !ActorComponent )
		{
			return;
		}

		const UPrimitiveComponent* PrimComp = Cast<const UPrimitiveComponent>(ActorComponent);
		if (PrimComp)
		{
			if ( PrimComp->IsRegistered() && !PrimComp->bHiddenInGame )
			{
				Bounds += PrimComp->Bounds.GetBox();
			}
		}

		for ( USceneComponent* ChildComponent : ActorComponent->GetAttachChildren() )
		{
			ComputeBounds( ChildComponent, Bounds );
		}
	}

	void SetupSceneViewport(AActor* SceneActor, TArray<FAssetData>& AssetDataList)
	{
		if ( !GCurrentLevelEditingViewportClient || !GEditor->GetActiveViewport())
		{
			return;
		}

		bool bFoundAViewComponent = false;

		TOptional< TWeakObjectPtr< AActor > > OriginalActorLock;
		if ( FLevelEditorViewportClient::FindViewComponentForActor( SceneActor ) )
		{
			OriginalActorLock = GCurrentLevelEditingViewportClient->GetActiveActorLock();
			GCurrentLevelEditingViewportClient->SetActorLock( SceneActor );
			bFoundAViewComponent = true;
		}

		if ( !bFoundAViewComponent )
		{
			FBox Bounds( ForceInit );
			ComputeBounds( SceneActor->GetRootComponent(), Bounds );

			GCurrentLevelEditingViewportClient->FocusViewportOnBox( Bounds, true );
		}

		bool bIsInGameView = GCurrentLevelEditingViewportClient->IsInGameView();
		GCurrentLevelEditingViewportClient->SetGameView( true );
		GCurrentLevelEditingViewportClient->SetIsCameraCut();

		// Unlock the viewport if we locked it to capture the thumbnail
		if ( OriginalActorLock )
		{
			// Reset the settings
			GCurrentLevelEditingViewportClient->ViewFOV = GCurrentLevelEditingViewportClient->FOVAngle;

			GCurrentLevelEditingViewportClient->SetActorLock( OriginalActorLock->Get() );
			GCurrentLevelEditingViewportClient->UpdateViewForLockedActor();

			// remove roll and pitch from camera when unbinding from actors
			GEditor->RemovePerspectiveViewRotation( true, true, false );
		}

		GCurrentLevelEditingViewportClient->SetGameView( bIsInGameView );
	}

}
