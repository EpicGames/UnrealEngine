// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetRegistry/AssetData.h"

class USceneComponent;
class FJsonObject;
class FString;
class AActor;

namespace DatasmithImportFactoryHelper
{
	TSharedPtr<FJsonObject> LoadJsonFile(const FString& JsonFilename);

	void ComputeBounds( USceneComponent* ActorComponent, FBox& Bounds );

	void SetupSceneViewport(AActor* SceneActor, TArray<FAssetData>& AssetDataList);
}
