// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithImporterImpl.h"

#include "DatasmithActorImporter.h"
#include "DatasmithAdditionalData.h"
#include "DatasmithAnimationElements.h"
#include "DatasmithAssetImportData.h"
#include "DatasmithAssetUserData.h"
#include "DatasmithCameraImporter.h"
#include "DatasmithImportContext.h"
#include "DatasmithLevelSequenceImporter.h"
#include "DatasmithLevelVariantSetsImporter.h"
#include "DatasmithLightImporter.h"
#include "DatasmithMaterialImporter.h"
#include "DatasmithPayload.h"
#include "DatasmithPostProcessImporter.h"
#include "DatasmithScene.h"
#include "DatasmithSceneActor.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithStaticMeshImporter.h"
#include "DatasmithTextureImporter.h"
#include "DatasmithTranslator.h"
#include "Engine/Texture2D.h"
#include "IDatasmithSceneElements.h"
#include "ObjectTemplates/DatasmithObjectTemplate.h"
#include "Utility/DatasmithImporterUtils.h"
#include "Utility/DatasmithTextureResize.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "Async/Async.h"
#include "Async/AsyncWork.h"
#include "CineCameraComponent.h"
#include "ComponentReregisterContext.h"
#include "Components/HierarchicalInstancedStaticMeshComponent.h"
#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorLevelUtils.h"
#include "Engine/Engine.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshActor.h"
#include "HAL/FileManager.h"
#include "IAssetTools.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Landscape.h"
#include "Layers/LayersSubsystem.h"
#include "LevelSequence.h"
#include "LevelVariantSets.h"
#include "MaterialEditingLibrary.h"
#include "Materials/Material.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialInstanceConstant.h"
#include "MaterialShared.h"
#include "Misc/FeedbackContext.h"
#include "Misc/FileHelper.h"
#include "Misc/PackageName.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/UObjectToken.h"
#include "ObjectTools.h"
#include "PackageTools.h"
#include "Serialization/ArchiveReplaceObjectRef.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/ObjectReader.h"
#include "Serialization/ObjectWriter.h"
#include "Settings/EditorExperimentalSettings.h"
#include "SourceControlOperations.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "UnrealEdGlobals.h"

extern UNREALED_API UEditorEngine* GEditor;

#define LOCTEXT_NAMESPACE "DatasmithImporter"


namespace UE::DatasmithImporterImpl::Private
{
	class FDatasmithWriter final : public FObjectWriter
	{
	public:
		FDatasmithWriter(UObject& Object, TArray< uint8 >& Bytes)
			: FObjectWriter(Bytes)
		{
			SetIsLoading(false);
			SetIsSaving(true);
			SetIsPersistent(true);

			SetPortFlags(GetPortFlags() | PPF_Duplicate);

			Object.Serialize(*this);
		}
	};

	class FDatasmithReader final : public FObjectReader
	{
	public:
		FDatasmithReader(UObject& Object, const TArray< uint8 >& Bytes)
			: FObjectReader(Bytes)
		{
			SetIsLoading(true);
			SetIsSaving(false);
			SetIsPersistent(true);

			SetPortFlags(GetPortFlags() | PPF_Duplicate);

			Object.Serialize(*this);
		}
	};
} // namespace UE::DatasmithImporterImpl::Private

void FDatasmithImporterImpl::ReportProgress(FScopedSlowTask* SlowTask, const float ExpectedWorkThisFrame, const FText& Text)
{
	if (SlowTask)
	{
		SlowTask->EnterProgressFrame(ExpectedWorkThisFrame, Text);
	}
}

bool FDatasmithImporterImpl::HasUserCancelledTask(FFeedbackContext* FeedbackContext)
{
	if (FeedbackContext)
	{
		return FeedbackContext->ReceivedUserCancel();
	}

	return false;
}

UObject* FDatasmithImporterImpl::PublicizeAsset( UObject* SourceAsset, const TCHAR* DestinationPath, UObject* ExistingAsset )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithImporterImpl::PublicizeAsset);

	UPackage* DestinationPackage = nullptr;

	const FString AssetName = UPackageTools::SanitizePackageName(SourceAsset->GetName());

	if ( !ExistingAsset )
	{
		FString SanitizedDestinationPath = UPackageTools::SanitizePackageName( DestinationPath );
		bool bPathIsComplete = AssetName == FPaths::GetBaseFilename( SanitizedDestinationPath );
		FString DestinationPackagePath = bPathIsComplete ? SanitizedDestinationPath : FPaths::Combine( SanitizedDestinationPath, AssetName );
		FString DestinationAssetPath = DestinationPackagePath + TEXT(".") + AssetName;
		ExistingAsset = FDatasmithImporterUtils::FindObject<UObject>( nullptr, DestinationAssetPath );

		DestinationPackage = ExistingAsset ? ExistingAsset->GetOutermost() : CreatePackage( *DestinationPackagePath );
	}
	else
	{
		DestinationPackage = ExistingAsset->GetOutermost();
	}

	// Close editors opened on existing asset if applicable
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	if (ExistingAsset && AssetEditorSubsystem->FindEditorForAsset(ExistingAsset, false) != nullptr)
	{
		AssetEditorSubsystem->CloseAllEditorsForAsset(ExistingAsset);
	}

	DestinationPackage->FullyLoad();

	UObject* DestinationAsset = ExistingAsset;

	FString OldAssetPathName;

	// If the object already exist, then we need to fix up the reference
	if ( ExistingAsset != nullptr && ExistingAsset != SourceAsset )
	{
		OldAssetPathName = ExistingAsset->GetPathName();

		DestinationAsset = FDatasmithImporterUtils::DuplicateObject( SourceAsset, DestinationPackage, ExistingAsset->GetFName() );

		// If mesh's label has changed, update its name
		if ( ExistingAsset->GetFName() != SourceAsset->GetFName() )
		{
			if (DestinationAsset->Rename(*AssetName, DestinationPackage, REN_Test))
			{
				DestinationAsset->Rename(*AssetName, DestinationPackage, REN_DontCreateRedirectors | REN_NonTransactional);
			}
			// Something wrong happened during a previous re-import of this asset
			// There already is a object in DestinationPackage. See UE-306372 for details.
			// Correct things by making package and asset names equal as it should have always been.
			else
			{
				const UObject* ExistingObject = StaticFindObject(nullptr, DestinationPackage, AssetName, EFindObjectFlags::ExactClass);
				UE_LOGF(LogDatasmithImport, Warning, "Renaming %ls to %ls failed because the latter already exists.", *DestinationAsset->GetFullName(), *ExistingObject->GetFullName());
				
				FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");

				FString NewAssetName;
				FString DestinationPackagePath;
				AssetToolsModule.Get().CreateUniqueAssetName(DestinationPackage->GetPathName(), TEXT(""), DestinationPackagePath, NewAssetName);

				DestinationPackage = CreatePackage(*DestinationPackagePath);
				DestinationAsset->Rename(*NewAssetName, DestinationPackage, REN_DontCreateRedirectors | REN_NonTransactional);

				UE_LOGF(LogDatasmithImport, Log, "Asset has been renamed to %ls.", *(DestinationPackagePath + TEXT(".") + NewAssetName));
			}
		}

		if ( UStaticMesh* DestinationMesh = Cast< UStaticMesh >( DestinationAsset ) )
		{
			// This is done during the mesh build process but we need to redo it after the DuplicateObject since the links are now valid
			for ( TObjectIterator< UStaticMeshComponent > It; It; ++It )
			{
				if ( It->GetStaticMesh() == DestinationMesh )
				{
					It->FixupOverrideColorsIfNecessary( true );
					It->InvalidateLightingCache();
				}
			}
		}
	}
	else
	{
		SourceAsset->Rename( *AssetName, DestinationPackage, REN_DontCreateRedirectors | REN_NonTransactional );
		DestinationAsset = SourceAsset;
	}

	DestinationAsset->SetFlags( RF_Public );
	DestinationAsset->MarkPackageDirty();

	if ( !ExistingAsset )
	{
		FAssetRegistryModule::AssetCreated( DestinationAsset );
	}
	else if ( !OldAssetPathName.IsEmpty() )
	{
		FAssetRegistryModule::AssetRenamed( DestinationAsset, OldAssetPathName );
	}

	return DestinationAsset;
}

void FDatasmithImporterImpl::CheckAssetPersistenceValidity(UObject* Asset, FDatasmithImportContext& ImportContext)
{
	if (Asset == nullptr)
	{
		return;
	}

	UPackage* Package = Asset->GetOutermost();
	const FString PackageName = Package->GetName();

	CheckAssetPersistenceValidity(PackageName, ImportContext, Asset->IsA<UWorld>() ? FPackageName::GetMapPackageExtension() : FPackageName::GetAssetPackageExtension());
}

/** Set the texture mode on each texture element based on its usage in the materials */
void FDatasmithImporterImpl::SetTexturesMode( FDatasmithImportContext& ImportContext )
{
	const int32 TexturesCount = ImportContext.FilteredScene->GetTexturesCount();
	const int32 MaterialsCount = ImportContext.FilteredScene->GetMaterialsCount();

	FFeedbackContext* FeedbackContext = ImportContext.FeedbackContext;
	for ( int32 TextureIndex = 0; TextureIndex < TexturesCount && !ImportContext.bUserCancelled; ++TextureIndex )
	{
		if (FDatasmithImporterImpl::HasUserCancelledTask(ImportContext.FeedbackContext))
		{
			ImportContext.bUserCancelled = true;
		}

		TSharedPtr< IDatasmithTextureElement > TextureElement = ImportContext.FilteredScene->GetTexture( TextureIndex );
		if (TextureElement->GetTextureMode() == EDatasmithTextureMode::Ies)
		{
			continue;
		}

		const FString TextureName = ObjectTools::SanitizeObjectName( TextureElement->GetName() );

		for ( int32 MaterialIndex = 0; MaterialIndex < MaterialsCount; ++MaterialIndex )
		{
			const TSharedPtr< IDatasmithBaseMaterialElement >& BaseMaterialElement = ImportContext.FilteredScene->GetMaterial( MaterialIndex );

			if ( BaseMaterialElement->IsA( EDatasmithElementType::Material ) )
			{
				const TSharedPtr< IDatasmithMaterialElement >& MaterialElement = StaticCastSharedPtr< IDatasmithMaterialElement >( BaseMaterialElement );

				for ( int32 s = 0; s < MaterialElement->GetShadersCount(); ++s )
				{
					const TSharedPtr< IDatasmithShaderElement >& ShaderElement = MaterialElement->GetShader(s);

					if ( FCString::Strlen( ShaderElement->GetDiffuseTexture() ) > 0 && ShaderElement->GetDiffuseTexture() == TextureName)
					{
						TextureElement->SetTextureMode(EDatasmithTextureMode::Diffuse);
					}
					else if ( FCString::Strlen( ShaderElement->GetReflectanceTexture() ) > 0 && ShaderElement->GetReflectanceTexture() == TextureName)
					{
						TextureElement->SetTextureMode(EDatasmithTextureMode::Specular);
					}
					else if ( FCString::Strlen( ShaderElement->GetNormalTexture() ) > 0 && ShaderElement->GetNormalTexture() == TextureName)
					{
						if (!ShaderElement->GetNormalTextureSampler().bInvert)
						{
							TextureElement->SetTextureMode(EDatasmithTextureMode::Normal);
						}
						else
						{
							TextureElement->SetTextureMode(EDatasmithTextureMode::NormalGreenInv);
						}
					}
				}
			}
			else if ( BaseMaterialElement->IsA( EDatasmithElementType::UEPbrMaterial ) )
			{
				const TSharedPtr< IDatasmithUEPbrMaterialElement >& MaterialElement = StaticCastSharedPtr< IDatasmithUEPbrMaterialElement >( BaseMaterialElement );

				TFunction< bool( IDatasmithMaterialExpression* ) > IsTextureConnected;
				IsTextureConnected = [ &TextureName, &IsTextureConnected ]( IDatasmithMaterialExpression* MaterialExpression ) -> bool
				{
					if ( !MaterialExpression )
					{
						return false;
					}

					if ( MaterialExpression->IsSubType( EDatasmithMaterialExpressionType::Texture ) )
					{
						IDatasmithMaterialExpressionTexture* TextureExpression = static_cast< IDatasmithMaterialExpressionTexture* >( MaterialExpression );

						if ( TextureExpression->GetTexturePathName() == TextureName )
						{
							return true;
						}
					}

					for ( int32 InputIndex = 0; InputIndex < MaterialExpression->GetInputCount(); ++InputIndex )
					{
						IDatasmithMaterialExpression* ConnectedExpression = MaterialExpression->GetInput( InputIndex )->GetExpression();

						if ( ConnectedExpression && IsTextureConnected( ConnectedExpression ) )
						{
							return true;
						}
					}

					return false;
				};

				if ( IsTextureConnected( MaterialElement->GetBaseColor().GetExpression() ) )
				{
					TextureElement->SetTextureMode(EDatasmithTextureMode::Diffuse);
				}
				else if ( IsTextureConnected( MaterialElement->GetSpecular().GetExpression() ) )
				{
					TextureElement->SetTextureMode(EDatasmithTextureMode::Specular);
				}
				else if ( IsTextureConnected( MaterialElement->GetNormal().GetExpression() ) )
				{
					if ( TextureElement->GetTextureMode() != EDatasmithTextureMode::Bump
					  && TextureElement->GetTextureMode() != EDatasmithTextureMode::NormalGreenInv)
					{
						TextureElement->SetTextureMode(EDatasmithTextureMode::Normal);
					}
				}
			}
		}
	}
}

void FDatasmithImporterImpl::CompileMaterial( UObject* Material, FMaterialUpdateContext* MaterialUpdateContext)
{
	if ( !Material->IsA< UMaterialInterface >() && !Material->IsA< UMaterialFunctionInterface >() )
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithImporterImpl::CompileMaterial);

	TUniquePtr<FMaterialUpdateContext> LocalMaterialUpdateContext;
	if (MaterialUpdateContext == nullptr)
	{
		LocalMaterialUpdateContext = MakeUnique<FMaterialUpdateContext>();
		MaterialUpdateContext = LocalMaterialUpdateContext.Get();
	}

	if ( UMaterialInterface* MaterialInterface = Cast< UMaterialInterface >( Material ) )
	{
		MaterialUpdateContext->AddMaterialInterface( MaterialInterface );
	}

	if ( UMaterialInstanceConstant* ConstantMaterialInstance = Cast< UMaterialInstanceConstant >( Material ) )
	{
		// If BlendMode override property has been changed, make sure this combination of the parent material is compiled
		if ( ConstantMaterialInstance->BasePropertyOverrides.bOverride_BlendMode == true )
		{
			ConstantMaterialInstance->ForceRecompileForRendering();
		}
		else
		{
			// If a switch is overriden, we need to recompile
			FStaticParameterSet StaticParameters;
			ConstantMaterialInstance->GetStaticParameterValues( StaticParameters );

			for ( FStaticSwitchParameter& Switch : StaticParameters.StaticSwitchParameters )
			{
				if ( Switch.bOverride )
				{
					ConstantMaterialInstance->ForceRecompileForRendering();
					break;
				}
			}
		}
	}

	Material->PreEditChange( nullptr );
	Material->PostEditChange();
}

void FDatasmithImporterImpl::FixReferencesForObject( UObject* Object, const TMap< UObject*, UObject* >& ReferencesToRemap )
{
	if ( ReferencesToRemap.Num() > 0 )
	{
		constexpr EArchiveReplaceObjectFlags ReplaceFlags = (EArchiveReplaceObjectFlags::IgnoreOuterRef | EArchiveReplaceObjectFlags::IgnoreArchetypeRef);
		FArchiveReplaceObjectRef< UObject > ArchiveReplaceObjectRef( Object, ReferencesToRemap, ReplaceFlags);
	}
}

/**
 * Creates templates to apply the values from the SourceObject on the DestinationObject.
 *
 * @returns An array of template pairs. The key is the template for the object, the value is a template to force apply to the object,
 *			it contains the values from the key and any overrides that were present on the DestinationObject.
 */
TArray< FDatasmithImporterImpl::FMigratedTemplatePairType > FDatasmithImporterImpl::MigrateTemplates( UObject* SourceObject, UObject* DestinationObject, const TMap< UObject*, UObject* >* ReferencesToRemap, bool bIsForActor )
{
	TArray< FMigratedTemplatePairType > Results;

	if ( !SourceObject )
	{
		return Results;
	}

	TMap< TSubclassOf< UDatasmithObjectTemplate >, TObjectPtr<UDatasmithObjectTemplate> >* SourceTemplates = FDatasmithObjectTemplateUtils::FindOrCreateObjectTemplates( SourceObject );

	if ( !SourceTemplates )
	{
		return Results;
	}

	for ( const TPair< TSubclassOf< UDatasmithObjectTemplate >, TObjectPtr<UDatasmithObjectTemplate> >& SourceTemplatePair : *SourceTemplates )
	{
		if ( bIsForActor == SourceTemplatePair.Value->bIsActorTemplate )
		{
			FMigratedTemplatePairType& Result = Results.AddDefaulted_GetRef();

			TStrongObjectPtr< UDatasmithObjectTemplate > SourceTemplate{ NewObject< UDatasmithObjectTemplate >(GetTransientPackage(), SourceTemplatePair.Key.Get()) }; // The SourceTemplate is the one we will persist so set its outer as DestinationObject

			SourceTemplate->Load(SourceObject);

			if ( ReferencesToRemap )
			{
				FixReferencesForObject(SourceTemplate.Get(), *ReferencesToRemap);
			}

			Result.Key = SourceTemplate;

			if (IsValid(DestinationObject) && !DestinationObject->IsUnreachable())
			{
				Result.Value = TStrongObjectPtr< UDatasmithObjectTemplate >(UDatasmithObjectTemplate::GetDifference( DestinationObject, SourceTemplate.Get()));
			}
			else
			{
				Result.Value = SourceTemplate;
			}
		}
	}

	return Results;
}

/**
 * Applies the templates created from MigrateTemplates to DestinationObject.
 *
 * For an Object A that should be duplicated over an existing A', for which we want to keep the Datasmith overrides:
 * - Call MigrateTemplates(A, A')
 * - Duplicate A over A'
 * - ApplyMigratedTemplates(A')
 */
void FDatasmithImporterImpl::ApplyMigratedTemplates( TArray< FMigratedTemplatePairType >& MigratedTemplates, UObject* DestinationObject )
{
	for ( FMigratedTemplatePairType& MigratedTemplate : MigratedTemplates )
	{
		UDatasmithObjectTemplate* SourceTemplate = MigratedTemplate.Key.Get();
		UDatasmithObjectTemplate* DestinationTemplate = MigratedTemplate.Value.Get();

		DestinationTemplate->Apply(DestinationObject, true); // Restore the overrides
		FDatasmithObjectTemplateUtils::SetObjectTemplate(DestinationObject, SourceTemplate); // Set SourceTemplate as our template so that any differences are considered overrides

	}
}

UObject* FDatasmithImporterImpl::FinalizeAsset( UObject* SourceAsset, const TCHAR* AssetPath, UObject* ExistingAsset, TMap< UObject*, UObject* >* ReferencesToRemap )
{
	if ( ReferencesToRemap )
	{
		FixReferencesForObject( SourceAsset, *ReferencesToRemap );
	}

	TArray< FMigratedTemplatePairType > MigratedTemplates = MigrateTemplates( SourceAsset, ExistingAsset, ReferencesToRemap, false );

	UObject* FinalAsset = PublicizeAsset( SourceAsset, AssetPath, ExistingAsset ); //-V595

	ApplyMigratedTemplates( MigratedTemplates, FinalAsset );

	if ( ReferencesToRemap && SourceAsset && SourceAsset != FinalAsset )
	{
		ReferencesToRemap->Add( SourceAsset, FinalAsset );
	}

	return FinalAsset;
}

void FDatasmithImporterImpl::DeleteImportSceneActorIfNeeded(FDatasmithActorImportContext& ActorContext, bool bForce)
{
	auto& ImportSceneActor = ActorContext.ImportSceneActor;
	if ( !ActorContext.FinalSceneActors.Contains(ImportSceneActor) || bForce )
	{
		if ( ImportSceneActor )
		{

			TArray< TSoftObjectPtr< AActor > > RelatedActors;
			ImportSceneActor->RelatedActors.GenerateValueArray( RelatedActors );

			ImportSceneActor->Scene = nullptr;
			ImportSceneActor->RelatedActors.Empty();

			while(RelatedActors.Num() > 0)
			{
				TSoftObjectPtr< AActor > ActorPtr = RelatedActors.Pop(EAllowShrinking::No);
				if(AActor* RelatedActor = ActorPtr.Get())
				{
					FDatasmithImporterUtils::DeleteActor( *RelatedActor );
				}
			}

			FDatasmithImporterUtils::DeleteActor( *ImportSceneActor );

			// Null also the ImportSceneActor from the Actor Context because it's a ref to it.
			ImportSceneActor = nullptr;
		}
	}
}

UActorComponent* FDatasmithImporterImpl::PublicizeComponent(UActorComponent& SourceComponent, UActorComponent* DestinationComponent, AActor& DestinationActor, TMap< UObject*, UObject* >& ReferencesToRemap, TArray<uint8>& ReusableBuffer)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithImporterImpl::PublicizeComponent);

	if ( !SourceComponent.HasAnyFlags( RF_Transient | RF_TextExportTransient | RF_DuplicateTransient ) )
	{
		if (!DestinationComponent || !IsValidChecked(DestinationComponent) || DestinationComponent->IsUnreachable())
		{
			if (DestinationComponent)
			{
				// Change the name of the old component so that the new object won't recycle the old one.
				DestinationComponent->Rename(nullptr, nullptr, REN_DontCreateRedirectors | REN_NonTransactional);
			}

			if ( UActorComponent* OldComponent = static_cast<UActorComponent*>( FindObjectWithOuter( &DestinationActor, UActorComponent::StaticClass(), SourceComponent.GetFName() ) ) )
			{
				OldComponent->DestroyComponent( true );
				// Change the name of the old component so that the new object won't recycle the old one.
				OldComponent->Rename(nullptr, nullptr, REN_DontCreateRedirectors | REN_NonTransactional);
			}

			// Create a new component
			DestinationComponent = NewObject<UActorComponent>(&DestinationActor, SourceComponent.GetClass(), SourceComponent.GetFName(), RF_Transactional);
		}

		check(DestinationComponent);

		if ( &SourceComponent != DestinationComponent )
		{
			CopyObject( SourceComponent, *DestinationComponent, ReusableBuffer );

			if ( DestinationComponent->GetFName() != SourceComponent.GetFName() )
			{
				DestinationComponent->Rename( *SourceComponent.GetName() );
			}
		}

		ReferencesToRemap.Add(&SourceComponent, DestinationComponent);

		PublicizeSubObjects( SourceComponent, *DestinationComponent, ReferencesToRemap, ReusableBuffer );

		return DestinationComponent;
	}

	return nullptr;
}

USceneComponent* FDatasmithImporterImpl::FinalizeSceneComponent(FDatasmithImportContext& ImportContext, USceneComponent& SourceComponent, AActor& DestinationActor, USceneComponent* DestinationParent, TMap<UObject *, UObject *>& ReferencesToRemap, TArray<uint8>& ReusableBuffer, TArray<TPair<USceneComponent&, TArray<FMigratedTemplatePairType>>>& ComponentsToApplyMigratedTemplate)
{
	// Need exact class (UHierarchicalInstancedStaticMeshComponent should not be reused for UStaticMeshComponent)
	USceneComponent* DestinationComponent = static_cast<USceneComponent*>( StaticFindObjectFastSafe(SourceComponent.GetClass(), &DestinationActor, SourceComponent.GetFName(), EFindObjectFlags::ExactClass) );
	
	FName SourceComponentDatasmithId = FDatasmithImporterUtils::GetDatasmithElementId(&SourceComponent);

	if ( SourceComponentDatasmithId.IsNone() )
	{
		// This component is not tracked by datasmith
		if ( !DestinationComponent || !IsValidChecked(DestinationComponent) || DestinationComponent->IsUnreachable() )
		{
			DestinationComponent = static_cast<USceneComponent*> ( PublicizeComponent(SourceComponent, DestinationComponent, DestinationActor, ReferencesToRemap, ReusableBuffer) );
			if ( DestinationComponent )
			{
				// Put back the components in a proper state
				DestinationComponent->UpdateComponentToWorld();
			}
		}
	}
	else
	{

		if ( !DestinationComponent )
		{
			// Improvement suggestion. We should build a cache in finalize scene components (It could be done in O(n*k)).
			// Currently this can be highly inefficient on the first import of a dataprep scene where some actors would have a absurd amount components.
			//  O(n^2*k) where n is the number of components on the actor and k is the avergage lenght of a datasmith ID (we need to hash the string representing the ID to convert it to a FName).
			//
			// Look at components of the actor, we might find the scene component we are looking for.
			for ( UActorComponent* Component : DestinationActor.GetInstanceComponents() )
			{
				if ( Component && ( Component->GetClass() == SourceComponent.GetClass() ) )
				{
					FName ComponentDatasmithId = FDatasmithImporterUtils::GetDatasmithElementId( Component );
					if ( ComponentDatasmithId.IsEqual( SourceComponentDatasmithId ) )
					{
						DestinationComponent = static_cast< USceneComponent* >( Component );
						break;
					}
				}
			}

			// If a DestinationComponent was found using DatasmithElementId (without using the name), make sure that there is no other component with the SourceComponent Name.
			if (DestinationComponent)
			{
				if (UActorComponent* OldComponent = static_cast<UActorComponent*>(FindObjectWithOuter(&DestinationActor, UActorComponent::StaticClass(), SourceComponent.GetFName())))
				{
					// It's okay to rename to null at this point as either the OldComponent was flagged for deletion by DeleteNonImportedDatasmithElementFromSceneActor()
					// at this point OR this component is in the reimported scene and we'll find it back using the the DatasmithElementId.
					OldComponent->Rename(nullptr, OldComponent->GetOuter(), REN_DontCreateRedirectors | REN_NonTransactional);
				}
			}
		}

		TArray< FMigratedTemplatePairType > MigratedTemplates = MigrateTemplates(
			&SourceComponent,
			DestinationComponent,
			&ReferencesToRemap,
			false
		);

		DestinationComponent = static_cast< USceneComponent* > ( PublicizeComponent( SourceComponent, DestinationComponent, DestinationActor, ReferencesToRemap, ReusableBuffer ) );

		if ( DestinationComponent )
		{
			ComponentsToApplyMigratedTemplate.Emplace( *DestinationComponent, MoveTemp(MigratedTemplates) );
		}
	}

	USceneComponent* AttachParentForChildren = DestinationComponent  ? nullptr : DestinationParent;
	for ( USceneComponent* Child : SourceComponent.GetAttachChildren() )
	{
		// Only finalize components that are from the same actor
		if ( Child && Child->GetOuter() == SourceComponent.GetOuter() )
		{
			if ( USceneComponent* DestinationChild = FinalizeSceneComponent( ImportContext, *Child, DestinationActor, AttachParentForChildren, ReferencesToRemap, ReusableBuffer, ComponentsToApplyMigratedTemplate))
			{
				if ( AttachParentForChildren )
				{
					DestinationChild->AttachToComponent(AttachParentForChildren, FAttachmentTransformRules::KeepRelativeTransform);
				}
			}
		}
	}

	return DestinationComponent;
}

void FDatasmithImporterImpl::FinalizeComponents(FDatasmithImportContext& ImportContext, AActor& SourceActor, AActor& DestinationActor, TMap<UObject *, UObject *>& ReferencesToRemap, TArray<uint8>& ReusableBuffer, TArray<TPair<USceneComponent&, TArray<FMigratedTemplatePairType>>>& ComponentsToApplyMigratedTemplate)
{
	USceneComponent* ParentComponent = nullptr;

	// Find the parent component
	UObject** ObjectPtr = ReferencesToRemap.Find( SourceActor.GetRootComponent()->GetAttachParent() );
	if ( ObjectPtr )
	{
		ParentComponent = Cast<USceneComponent>( *ObjectPtr );
	}

	// Finalize the scene components recursively
	{
		USceneComponent* RootComponent = SourceActor.GetRootComponent();
		if ( RootComponent )
		{
			FinalizeSceneComponent( ImportContext, *RootComponent, DestinationActor, ParentComponent, ReferencesToRemap, ReusableBuffer, ComponentsToApplyMigratedTemplate );
		}
	}


	for ( UActorComponent* SourceComponent : SourceActor.GetComponents() )
	{
		// Only the non scene component haven't been finalize
		if ( SourceComponent && !SourceComponent->GetClass()->IsChildOf<USceneComponent>() )
		{
			UActorComponent* DestinationComponent = static_cast< UActorComponent* >( FindObjectWithOuter( &DestinationActor, SourceComponent->GetClass(), SourceComponent->GetFName() ) );
			if (!DestinationComponent)
			{
				DestinationComponent = PublicizeComponent( *SourceComponent, DestinationComponent, DestinationActor, ReferencesToRemap, ReusableBuffer );
			}
		}
	}
}

void FDatasmithImporterImpl::PublicizeSubObjects(UObject& SourceObject, UObject& DestinationObject, TMap< UObject*, UObject* >& ReferencesToRemap, TArray<uint8>& ReusableBuffer)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithImporterImpl::PublicizeSubObjects);
	TArray<UObject*> SourceSubObjects;
	ForEachObjectWithOuter( &SourceObject, [&SourceSubObjects](UObject* SourceSubObject)
		{
			SourceSubObjects.Add(SourceSubObject);
		}
		, EGetObjectsFlags::None);

	for(UObject* SourceSubObject : SourceSubObjects)
	{
		// We don't want to deal with components since this done in finalize components
		if ( !SourceSubObject->IsA<UActorComponent>() && !SourceSubObject->HasAnyFlags( RF_Transient | RF_TextExportTransient | RF_DuplicateTransient | RF_BeginDestroyed | RF_FinishDestroyed ) )
		{
			UObject* DestinationSubObject = FindObjectFast<UObject>( &DestinationObject, SourceSubObject->GetFName() );

			if ( !DestinationSubObject || !IsValidChecked(DestinationSubObject) || DestinationSubObject->IsUnreachable() || DestinationSubObject->GetClass() !=  SourceSubObject->GetClass() )
			{
				if ( DestinationSubObject )
				{
					// Move away the existing object.
					DestinationSubObject->Rename( nullptr, GetTransientPackage(), REN_DontCreateRedirectors | REN_NonTransactional );
				}

				DestinationSubObject = NewObject<UObject>( &DestinationObject, SourceSubObject->GetClass(), SourceSubObject->GetFName(), SourceSubObject->GetFlags() );
			}

			CopyObject( *SourceSubObject, *DestinationSubObject, ReusableBuffer );

			ReferencesToRemap.Add( SourceSubObject, DestinationSubObject );

			PublicizeSubObjects( *SourceSubObject, *DestinationSubObject, ReferencesToRemap, ReusableBuffer );
		}

	}
}

void FDatasmithImporterImpl::CopyObject(UObject& Source, UObject& Destination, TArray<uint8>& TempBuffer)
{
	// Keep the previous allocation
	TempBuffer.Reset();
	using namespace UE::DatasmithImporterImpl::Private;
	FDatasmithWriter DatasmithWriter( Source, TempBuffer );
	FDatasmithReader DatasmithReader( Destination, TempBuffer );
}

void FDatasmithImporterImpl::GatherUnsupportedVirtualTexturesAndMaterials(const TMap<TSharedRef< IDatasmithBaseMaterialElement >, UMaterialInterface*>& ImportedMaterials, TSet<UTexture2D*>& VirtualTexturesToConvert, TArray<UMaterial*>& MaterialsToRefreshAfterVirtualTextureConversion)
{
	//Multimap cache to avoid parsing the same base material multiple times.
	TMultiMap<UMaterial*, FMaterialParameterInfo> TextureParametersToConvertMap;

	//Loops through all imported material instances and add to VirtualTexturesToConvert all the texture parameters that don't support texturing in the base material.
	for (const TPair< TSharedRef< IDatasmithBaseMaterialElement >, UMaterialInterface* >& ImportedMaterialPair : ImportedMaterials)
	{
		UMaterialInterface* CurrentMaterialInterface = ImportedMaterialPair.Value;
		UMaterial* BaseMaterial = CurrentMaterialInterface->GetMaterial();

		if(!TextureParametersToConvertMap.Contains(BaseMaterial))
		{
			bool bRequiresTextureCheck = false;
			TArray<FMaterialParameterInfo> OutParameterInfo;
			TArray<FGuid> Guids;
			BaseMaterial->GetAllTextureParameterInfo(OutParameterInfo, Guids);

			for (int32 ParameterInfoIndex = 0; ParameterInfoIndex < OutParameterInfo.Num(); ++ParameterInfoIndex)
			{
				UTexture* TextureParameter = nullptr;

				if (BaseMaterial->GetTextureParameterValue(OutParameterInfo[ParameterInfoIndex], TextureParameter) && VirtualTexturesToConvert.Contains(Cast<UTexture2D>(TextureParameter)))
				{
					bRequiresTextureCheck = true;
					TextureParametersToConvertMap.Add(BaseMaterial, OutParameterInfo[ParameterInfoIndex]);
				}
			}

			if (bRequiresTextureCheck)
			{
				MaterialsToRefreshAfterVirtualTextureConversion.Add(BaseMaterial);
			}
			else
			{
				//Adding a dummy MaterialParameterInfo so that we don't have to parse this Base Material again.
				TextureParametersToConvertMap.Add(BaseMaterial, FMaterialParameterInfo());

				//If no unsupported texture parameters were found, it's possible that a texture needing conversion is simply not exposed as a parameter, so we still need to check for those.
				for (UObject* ReferencedTexture : BaseMaterial->GetReferencedTextures())
				{
					if (VirtualTexturesToConvert.Contains(Cast<UTexture2D>(ReferencedTexture)))
					{
						MaterialsToRefreshAfterVirtualTextureConversion.Add(BaseMaterial);
						break;
					}
				}
			}
		}

		for (auto ParameterInfoIt = TextureParametersToConvertMap.CreateKeyIterator(BaseMaterial); ParameterInfoIt; ++ParameterInfoIt)
		{
			UTexture* TextureParameter = nullptr;

			if (CurrentMaterialInterface->GetTextureParameterValue(ParameterInfoIt.Value(), TextureParameter) &&
				TextureParameter && TextureParameter->VirtualTextureStreaming)
			{
				if (UTexture2D* TextureToConvert = Cast<UTexture2D>(TextureParameter))
				{
					VirtualTexturesToConvert.Add(TextureToConvert);
				}
			}
		}
	}
}

void FDatasmithImporterImpl::ConvertUnsupportedVirtualTexture(FDatasmithImportContext& ImportContext, TSet<UTexture2D*>& VirtualTexturesToConvert, const TMap<UObject*, UObject*>& ReferencesToRemap)
{
	TArray<UMaterial*> MaterialsToRefreshAfterVirtualTextureConversion;
	GatherUnsupportedVirtualTexturesAndMaterials(ObjectPtrDecay(ImportContext.ImportedMaterials), ImportContext.AssetsContext.VirtualTexturesToConvert, MaterialsToRefreshAfterVirtualTextureConversion);

	if (VirtualTexturesToConvert.Num() != 0)
	{
		for (UTexture2D*& TextureToConvert : VirtualTexturesToConvert)
		{
			if (UObject* const* RemappedTexture = ReferencesToRemap.Find(TextureToConvert))
			{
				TextureToConvert = Cast<UTexture2D>(*RemappedTexture);
			}

			ImportContext.LogWarning(FText::Format(LOCTEXT("DatasmithVirtualTextureConverted", "The imported texture {0} could not be imported as texture as it is not supported in all the materials using it."), FText::FromString(TextureToConvert->GetName())));
		}

		for (int32 MaterialIndex = 0; MaterialIndex < MaterialsToRefreshAfterVirtualTextureConversion.Num(); ++MaterialIndex)
		{
			if (UObject* const* RemappedMaterial = ReferencesToRemap.Find(MaterialsToRefreshAfterVirtualTextureConversion[MaterialIndex]))
			{
				MaterialsToRefreshAfterVirtualTextureConversion[MaterialIndex] = Cast<UMaterial>(*RemappedMaterial);
			}
		}

		TArray<UTexture2D*> TexturesToConvertList(VirtualTexturesToConvert.Array());
		IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
		AssetTools.ConvertVirtualTextures(TexturesToConvertList, true, &MaterialsToRefreshAfterVirtualTextureConversion);
	}
}

FDatasmithImporterImpl::FScopedFinalizeActorChanges::FScopedFinalizeActorChanges(AActor* InFinalizedActor, FDatasmithImportContext& InImportContext)
	: ImportContext(InImportContext)
	, FinalizedActor(InFinalizedActor)
{
	// In order to allow modification on components owned by ExistingActor, unregister all of them
	FinalizedActor->UnregisterAllComponents( /* bForReregister = */true);

	//Some new components might be created when finalizing the actor, only validate those that we unregistered.
	for (UActorComponent* Component : FinalizedActor->GetComponents())
	{
		ComponentsToValidate.Add(Component);
	}
}

FDatasmithImporterImpl::FScopedFinalizeActorChanges::~FScopedFinalizeActorChanges()
{
	for (UActorComponent* Component : FinalizedActor->GetComponents())
	{
		if (Component->IsRegistered() && ComponentsToValidate.Contains(Component))
		{
			ensureMsgf(false, TEXT("All components should still be unregistered at this point. Otherwise some datasmith templates might not have been applied properly."));
			break;
		}
	}

	FinalizedActor->PostEditImport();
	FinalizedActor->PostEditChange();
}

bool FDatasmithImporterImpl::CheckAssetPersistenceValidity(const FString& PackageName, FDatasmithImportContext& ImportContext, const FString& Extension)
{
	FText OutReason;
	if(!CheckAssetPersistenceValidity(PackageName, ImportContext, Extension, OutReason))
	{
		ImportContext.LogWarning(OutReason);
		return false;
	}

	return true;
}

bool FDatasmithImporterImpl::CheckAssetPersistenceValidity(const FString& PackageName, FDatasmithImportContext& ImportContext, const FString& Extension, FText& OutReason)
{
	// Check that package can be saved
	const FString BasePackageFileName = FPackageName::LongPackageNameToFilename( PackageName );
	const FString AbsolutePathToAsset = FPaths::ConvertRelativePathToFull( BasePackageFileName );

	// Create fake filename of same length of final asset file name to test ability to write
	const FString FakeAbsolutePathToAsset = AbsolutePathToAsset + Extension;

	OutReason = FText::GetEmpty();

	// Verify asset file name does not exceed OS' maximum path length
	if( FPlatformMisc::GetMaxPathLength() < FakeAbsolutePathToAsset.Len() )
	{
		OutReason = FText::Format(LOCTEXT("DatasmithImportInvalidLength", "Saving may partially fail because path for asset {0} is too long. Rename before saving."), FText::FromString( PackageName ));
	}
	// Verify user can overwrite existing file
	else if( IFileManager::Get().FileExists( *FakeAbsolutePathToAsset ) )
	{
		FFileStatData FileStatData = IFileManager::Get().GetStatData( *FakeAbsolutePathToAsset );
		if ( FileStatData.bIsReadOnly )
		{
			// Check to see if the file is not under source control
			bool bCanCheckedOut = false;

			ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
			if (SourceControlProvider.IsAvailable() && SourceControlProvider.IsEnabled())
			{
				SourceControlProvider.Execute(ISourceControlOperation::Create<FUpdateStatus>(), FakeAbsolutePathToAsset);
				FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(FakeAbsolutePathToAsset, EStateCacheUsage::Use);
				if( SourceControlState.IsValid() && SourceControlState->CanCheckout() )
				{
					// User will be prompted to check out this file when they save the asset. No need to warn.
					bCanCheckedOut = true;
				}
			}

			if(!bCanCheckedOut)
			{
				OutReason = FText::Format(LOCTEXT("DatasmithImportInvalidSaving", "Saving may partially fail because file asset {0} cannot be overwritten. Check your privileges."), FText::FromString( PackageName ));
			}
		}
	}
	// Verify user has privileges to write in folder where asset file will be stored
	else if (GUnrealEd != nullptr && !GUnrealEd->HasMountWritePermissionForPackage(PackageName))
	{
		OutReason = FText::Format(LOCTEXT("DatasmithImportInvalidFolder", "Cannot write in folder {0} to store asset {1}. Check access to folder."), FText::FromString( FPaths::GetPath( FakeAbsolutePathToAsset ) ), FText::FromString( PackageName ));
	}

	// Check that package can be cooked
	// Value for MaxGameNameLen directly taken from ContentBrowserUtils::GetPackageLengthForCooking
	constexpr int32 MaxGameNameLen = 20;

	// Pad out the game name to the maximum allowed
	const FString GameName = FApp::GetProjectName();
	FString GameNamePadded = GameName;
	while (GameNamePadded.Len() < MaxGameNameLen)
	{
		GameNamePadded += TEXT(" ");
	}

	const FString AbsoluteGamePath = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir());
	const FString AbsoluteGameCookPath = AbsoluteGamePath / TEXT("Saved") / TEXT("Cooked") / TEXT("Windows") / GameName;

	FString AssetPathWithinCookDir = AbsolutePathToAsset;
	FPaths::RemoveDuplicateSlashes(AssetPathWithinCookDir);
	AssetPathWithinCookDir.RemoveFromStart(AbsoluteGamePath, ESearchCase::CaseSensitive);

	// Test that the package can be cooked based on the current project path
	FString AbsoluteCookPathToAsset = AbsoluteGameCookPath / AssetPathWithinCookDir;

	AbsoluteCookPathToAsset.ReplaceInline(*GameName, *GameNamePadded, ESearchCase::CaseSensitive);

	// Get the longest path allowed by the system or use 260 as the longest which is the shortest max path of any platforms that support cooking
	const int32 MaxCookPath = GetDefault<UEditorExperimentalSettings>()->bEnableLongPathsSupport ? FPlatformMisc::GetMaxPathLength() : 260 /* MAX_PATH */;

	if (AbsoluteCookPathToAsset.Len() > MaxCookPath)
	{
		OutReason = FText::Format(LOCTEXT("DatasmithImportInvalidCooking", "Cooking may fail because path for asset {0} is too long. Rename before cooking."), FText::FromString(PackageName));
	}

	return OutReason.IsEmpty();
}

#undef LOCTEXT_NAMESPACE
