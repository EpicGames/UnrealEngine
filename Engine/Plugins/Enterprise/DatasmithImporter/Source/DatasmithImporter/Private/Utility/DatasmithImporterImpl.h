// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UObject/StrongObjectPtr.h"

class AActor;
class FFeedbackContext;
class FMaterialUpdateContext;
class IDatasmithBaseMaterialElement;
class UActorComponent;
class UDatasmithObjectTemplate;
class UMaterial;
class UMaterialInterface;
class USceneComponent;
class UTexture2D;

struct FDatasmithActorImportContext;
struct FDatasmithImportContext;
struct FScopedSlowTask;

class FDatasmithImporterImpl
{
public:
	static void ReportProgress(FScopedSlowTask* SlowTask, const float ExpectedWorkThisFrame, const FText& Text);

	static bool HasUserCancelledTask(FFeedbackContext* FeedbackContext);

	static UObject* PublicizeAsset( UObject* SourceAsset, const TCHAR* DestinationPath, UObject* ExistingAsset );

	/**
	 * Verifies the input asset can successfully be saved and/or cooked.
	 * The code below is heavily inspired from:
	 *			- ContentBrowserUtils::IsValidPackageForCooking
	 * #ueent_todo Make ContentBrowserUtils::IsValidPackageForCooking public. Not game to do it so late in 4.22
	 */
	static bool DATASMITHIMPORTER_API CheckAssetPersistenceValidity(const FString& PackageName, FDatasmithImportContext& ImportContext, const FString& Extension, FText& OutReason);
	static bool DATASMITHIMPORTER_API CheckAssetPersistenceValidity(const FString& PackageName, FDatasmithImportContext& ImportContext, const FString& Extension);

	static void CheckAssetPersistenceValidity(UObject* Asset, FDatasmithImportContext& ImportContext);

	/** Set the texture mode on each texture element based on its usage in the materials */
	static void SetTexturesMode( FDatasmithImportContext& ImportContext );

	static void CompileMaterial( UObject* Material, FMaterialUpdateContext* MaterialUpdateContext = nullptr);

	static void FixReferencesForObject( UObject* Object, const TMap< UObject*, UObject* >& ReferencesToRemap );

	typedef  TPair< TStrongObjectPtr< UDatasmithObjectTemplate >, TStrongObjectPtr< UDatasmithObjectTemplate > > FMigratedTemplatePairType;

	/**
	 * Creates templates to apply the values from the SourceObject on the DestinationObject.
	 *
	 * @returns An array of template pairs. The key is the template for the object, the value is a template to force apply to the object,
	 *			it contains the values from the key and any overrides that were present on the DestinationObject.
	 */
	static TArray< FMigratedTemplatePairType > MigrateTemplates( UObject* SourceObject, UObject* DestinationObject, const TMap< UObject*, UObject* >* ReferencesToRemap, bool bIsForActor );

	/**
	 * Applies the templates created from MigrateTemplates to DestinationObject.
	 *
	 * For an Object A that should be duplicated over an existing A', for which we want to keep the Datasmith overrides:
	 * - Call MigrateTemplates(A, A')
	 * - Duplicate A over A'
	 * - ApplyMigratedTemplates(A')
	 */
	static void ApplyMigratedTemplates( TArray< FMigratedTemplatePairType >& MigratedTemplates, UObject* DestinationObject );

	static UObject* FinalizeAsset( UObject* SourceAsset, const TCHAR* AssetPath, UObject* ExistingAsset, TMap< UObject*, UObject* >* ReferencesToRemap );

	static void DeleteImportSceneActorIfNeeded(FDatasmithActorImportContext& ActorContext, bool bForce = false);

	static UActorComponent* PublicizeComponent(UActorComponent& SourceComponent, UActorComponent* DestinationComponent, AActor& DestinationActor, TMap< UObject*, UObject* >& ReferencesToRemap, TArray<uint8>& ReusableBuffer);

	static USceneComponent* FinalizeSceneComponent(FDatasmithImportContext& ImportContext, USceneComponent& SourceComponent, AActor& DestinationActor, USceneComponent* DestinationParent, TMap<UObject *, UObject *>& ReferencesToRemap, TArray<uint8>& ReusableBuffer, TArray<TPair<USceneComponent&, TArray<FMigratedTemplatePairType>>>& ComponentsToApplyMigratedTemplat);

	static void FinalizeComponents(FDatasmithImportContext& ImportContext, AActor& SourceActor, AActor& DestinationActor, TMap<UObject *, UObject *>& ReferencesToRemap, TArray<uint8>& ReusableBuffer, TArray<TPair<USceneComponent&, TArray<FMigratedTemplatePairType>>>& ComponentsToApplyMigratedTemplat);

	static void PublicizeSubObjects(UObject& SourceObject, UObject& DestinationObject, TMap< UObject*, UObject* >& ReferencesToRemap, TArray<uint8>& ReusableBuffer);

	static void CopyObject(UObject& Source, UObject& Destination, TArray<uint8>& TempBuffer);

	static void GatherUnsupportedVirtualTexturesAndMaterials(const TMap<TSharedRef< IDatasmithBaseMaterialElement >, UMaterialInterface*>& ImportedMaterials, TSet<UTexture2D*>& VirtualTexturesToConvert, TArray<UMaterial*>& MaterialsToRefreshAfterVirtualTextureConversion);

	static void ConvertUnsupportedVirtualTexture(FDatasmithImportContext& ImportContext, TSet<UTexture2D*>& VirtualTexturesToConvert, const TMap<UObject*, UObject*>& ReferencesToRemap);

	/**
	 * Helper struct used to manage and validate the actor's state when modifying it during the finalization.
	 */
	struct FScopedFinalizeActorChanges
	{
		FDatasmithImportContext& ImportContext;
		TSet<UActorComponent*> ComponentsToValidate;
		AActor* FinalizedActor;

		FScopedFinalizeActorChanges(AActor* InFinalizedActor, FDatasmithImportContext& InImportContext);

		~FScopedFinalizeActorChanges();
	};
};
