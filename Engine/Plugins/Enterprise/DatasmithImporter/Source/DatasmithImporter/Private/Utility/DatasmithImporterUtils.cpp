// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utility/DatasmithImporterUtils.h"

#include "DatasmithActorImporter.h"
#include "DatasmithAreaLightActor.h"
#include "DatasmithAssetImportData.h"
#include "DatasmithAssetUserData.h"
#include "DatasmithImportContext.h"
#include "DatasmithScene.h"
#include "DatasmithSceneActor.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithSceneSource.h"
#include "DatasmithSceneXmlReader.h"
#include "DatasmithSceneXmlWriter.h"
#include "DatasmithTranslatorManager.h"
#include "Engine/TextureLightProfile.h"
#include "LevelVariantSets.h"

#include "ObjectTemplates/DatasmithActorTemplate.h"
#include "ObjectTemplates/DatasmithAreaLightActorTemplate.h"
#include "ObjectTemplates/DatasmithCineCameraComponentTemplate.h"
#include "ObjectTemplates/DatasmithLightComponentTemplate.h"
#include "ObjectTemplates/DatasmithMaterialInstanceTemplate.h"
#include "ObjectTemplates/DatasmithPointLightComponentTemplate.h"
#include "ObjectTemplates/DatasmithObjectTemplate.h"
#include "ObjectTemplates/DatasmithSceneComponentTemplate.h"
#include "ObjectTemplates/DatasmithStaticMeshTemplate.h"
#include "ObjectTemplates/DatasmithSkyLightComponentTemplate.h"
#include "ObjectTemplates/DatasmithStaticMeshComponentTemplate.h"

#include "ActorEditorUtils.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "CineCameraActor.h"
#include "CineCameraComponent.h"
#include "Components/HierarchicalInstancedStaticMeshComponent.h"
#include "Components/LightComponent.h"
#include "Components/PointLightComponent.h"
#include "Components/SpotLightComponent.h"
#include "Editor.h"
#include "Engine/DirectionalLight.h"
#include "Engine/RectLight.h"
#include "Engine/Level.h"
#include "Engine/Light.h"
#include "Engine/PostProcessVolume.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture2D.h"
#include "Engine/SpotLight.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "IMessageLogListing.h"
#include "Landscape.h"
#include "Layers/LayersSubsystem.h"
#include "Layers/Layer.h"
#include "LevelSequence.h"
#include "Lightmass/LightmassPortal.h"
#include "Logging/TokenizedMessage.h"
#include "Materials/MaterialInstanceConstant.h"
#include "MessageLogModule.h"
#include "Modules/ModuleManager.h"
#include "Misc/FileHelper.h"
#include "Misc/SecureHash.h"
#include "Serialization/MemoryWriter.h"
#include "UObject/ObjectRedirector.h"

extern UNREALED_API UEditorEngine* GEditor;

DEFINE_LOG_CATEGORY(LogDatasmithImport);

#define LOCTEXT_NAMESPACE "DatasmithImporterUtils"


namespace DatasmithImporterUtilsImpl
{

int32 GDatasmithMaxAssetPathLength = 260-100; // a base value that should suits most cases
static FAutoConsoleVariableRef CVarDatasmithMaxAssetPathLength(
	TEXT("ds.MaxAssetPathLength"),
	GDatasmithMaxAssetPathLength,
	TEXT("Datasmith will try to limit asset path length to this value. Default: 160")
);

int32 GetUsableMaxAssetPathLength()
{
	const int32 MinWorkable = 60; // Datasmith generates nested path, imports won't work correctly if constraints are too hard...
	static int32 ProjectConstraint; // Deduced from os limit and project path
	static bool RunOnce = [&]()
	{
		const FString ProjectContentDir = FPaths::ProjectContentDir();
		const FString FullPathProjectContentDir = FPaths::ConvertRelativePathToFull(ProjectContentDir);
		ProjectConstraint = FPlatformMisc::GetMaxPathLength() - FullPathProjectContentDir.Len();

		if (ProjectConstraint < MinWorkable)
		{
			UE_LOGF(LogDatasmithImport, Error,
				"Datasmith can encounter import issues due to a Content path too long, and an OS limitation on path length.\n" "Content path: '%ls'\n" "System max path length: %d.\n"
				, *FullPathProjectContentDir, FPlatformMisc::GetMaxPathLength());
		}

		if (GDatasmithMaxAssetPathLength > ProjectConstraint)
		{
			UE_LOGF(LogDatasmithImport, Warning,
				"The Datasmith.MaxAssetPathLength value (%d) is too high for the current setup.\n" "Content path: '%ls'\n" "System max path length: %d.\n"
				, GDatasmithMaxAssetPathLength, *FullPathProjectContentDir, FPlatformMisc::GetMaxPathLength());
		}

		return true;
	}();

	return FMath::Max(FMath::Min(GDatasmithMaxAssetPathLength, ProjectConstraint), MinWorkable);
}

}

TSharedPtr< IDatasmithScene > FDatasmithImporterUtils::LoadDatasmithScene( UDatasmithScene* DatasmithSceneAsset )
{
	if ( DatasmithSceneAsset->DatasmithSceneBulkData.GetElementCount() > 0 )
	{
		TArrayView< uint8 > Bytes = MakeArrayView( reinterpret_cast< uint8* >( DatasmithSceneAsset->DatasmithSceneBulkData.Lock( LOCK_READ_ONLY ) ),
			DatasmithSceneAsset->DatasmithSceneBulkData.GetElementCount() );

		FUTF8ToTCHAR Converter( reinterpret_cast< ANSICHAR* >( Bytes.GetData() ) );

		FString XmlBuffer( Converter.Get() );

		TSharedRef< IDatasmithScene > DatasmithScene = FDatasmithSceneFactory::CreateScene( *DatasmithSceneAsset->GetName() );

		FDatasmithSceneXmlReader DatasmithSceneXmlReader;
		DatasmithSceneXmlReader.ParseBuffer( XmlBuffer, DatasmithScene );

		DatasmithSceneAsset->DatasmithSceneBulkData.Unlock();

		return DatasmithScene;
	}

	return TSharedPtr< IDatasmithScene >();
}

void FDatasmithImporterUtils::SaveDatasmithScene( TSharedRef<IDatasmithScene> DatasmithScene, UDatasmithScene* DatasmithSceneAsset )
{
	if ( !DatasmithSceneAsset )
	{
		return;
	}

	TArray< uint8 > Bytes;
	FMemoryWriter MemoryWriter( Bytes, true );

	FDatasmithSceneXmlWriter DatasmithSceneXmlWriter;
	DatasmithSceneXmlWriter.Serialize( DatasmithScene, MemoryWriter );

	DatasmithSceneAsset->DatasmithSceneBulkData.Lock( LOCK_READ_WRITE );

	uint8* Dest = reinterpret_cast< uint8* >( DatasmithSceneAsset->DatasmithSceneBulkData.Realloc( Bytes.Num() ) );

	FPlatformMemory::Memcpy( Dest, Bytes.GetData(), Bytes.Num() );

	DatasmithSceneAsset->DatasmithSceneBulkData.Unlock();
}

ADatasmithSceneActor* FDatasmithImporterUtils::CreateImportSceneActor( FDatasmithImportContext& ImportContext, FTransform WorldTransform )
{
	TSharedRef< IDatasmithActorElement > SceneActorElement = FDatasmithSceneFactory::CreateActor( *ImportContext.SceneName );
	SceneActorElement->SetLabel( *ImportContext.SceneName );

	SceneActorElement->SetTranslation( WorldTransform.GetLocation() );
	SceneActorElement->SetRotation( WorldTransform.GetRotation() );
	SceneActorElement->SetScale( WorldTransform.GetScale3D() );

	ADatasmithSceneActor* SceneActor = Cast< ADatasmithSceneActor >( FDatasmithActorImporter::ImportActor( ADatasmithSceneActor::StaticClass(), SceneActorElement, ImportContext, EDatasmithImportActorPolicy::Full ) );
	if (!ensure( SceneActor ))
	{
		return nullptr;
	}

	SceneActor->SpriteScale = 0.1f;

	USceneComponent* RootComponent = SceneActor->GetRootComponent();

	if ( !RootComponent )
	{
		RootComponent = NewObject<USceneComponent>(SceneActor, *ImportContext.SceneName, RF_Transactional);
		RootComponent->SetWorldTransform( WorldTransform );

		SceneActor->SetRootComponent( RootComponent );
		SceneActor->AddInstanceComponent( RootComponent );

		RootComponent->Mobility = EComponentMobility::Static;
		RootComponent->bVisualizeComponent = true;

		RootComponent->RegisterComponent();
	}

	FVector Geolocation = ImportContext.Scene->GetGeolocation();

	if (Geolocation.X != TNumericLimits<double>::Max())
	{
		UDatasmithAssetUserData::SetDatasmithUserDataValueForKey(SceneActor, TEXT("Geolocation_Latitude"),  FString::SanitizeFloat(Geolocation.X));
	}

	if (Geolocation.Y != TNumericLimits<double>::Max())
	{
		UDatasmithAssetUserData::SetDatasmithUserDataValueForKey(SceneActor, TEXT("Geolocation_Longitude"),  FString::SanitizeFloat(Geolocation.Y));
	}

	if (Geolocation.Z != TNumericLimits<double>::Max())
	{
		UDatasmithAssetUserData::SetDatasmithUserDataValueForKey(SceneActor, TEXT("Geolocation_Elevation"),  FString::SanitizeFloat(Geolocation.Z));
	}

	ImportContext.ActorsContext.ImportSceneActor = SceneActor;

	return SceneActor;
}

TArray< ADatasmithSceneActor* > FDatasmithImporterUtils::FindSceneActors( UWorld* World, UDatasmithScene* DatasmithScene)
{
	if( World == nullptr || DatasmithScene == nullptr )
	{
		return TArray< ADatasmithSceneActor* >();
	}

	auto IsValidSceneActor = [DatasmithScene, World]( AActor* Actor )
	{
		ADatasmithSceneActor* NullSceneActor = nullptr;

		if (Actor != nullptr)
		{
			// Don't consider transient actors in non-play worlds
			// Don't consider the builder brush
			// Don't consider the WorldSettings actor, even though it is technically editable
			bool bIsValidActor = Actor->IsEditable() && !Actor->IsTemplate() && !Actor->HasAnyFlags(RF_Transient) && !FActorEditorUtils::IsABuilderBrush(Actor) && !Actor->IsA(AWorldSettings::StaticClass());

			if ( bIsValidActor )
			{
				if( ADatasmithSceneActor* SceneActor = Cast< ADatasmithSceneActor >( Actor ) )
				{
					// A scene can be used by multiple loaded levels. Only the SceneActors of the current level are valid
					return (SceneActor->Scene == DatasmithScene && SceneActor->GetLevel() == World->GetCurrentLevel()) ? SceneActor : NullSceneActor;
				}

				return NullSceneActor;
			}
		}

		return NullSceneActor;
	};

	TArray< ADatasmithSceneActor* > SceneActors;
	const EActorIteratorFlags Flags = EActorIteratorFlags::SkipPendingKill;
	for ( TActorIterator< AActor > It( World, AActor::StaticClass(), Flags ); It; ++It )
	{
		AActor* Actor = *It;
		if ( ADatasmithSceneActor* SceneActor = IsValidSceneActor( Actor ) )
		{
			SceneActors.Add( SceneActor );
		}
	}

	return SceneActors;
}

void FDatasmithImporterUtils::DeleteNonImportedDatasmithElementFromSceneActor(ADatasmithSceneActor& SourceSceneActor, ADatasmithSceneActor& DestinationSceneActor, const TSet< FName >& IgnoredDatasmithActors)
{
	// We need to remove the children in a depth first manner because removing an actor will reattach its existing children to its parent.
	// This operation makes the object template dirty so it should only be done if we intend to keep that child actor.
	struct FDepthSortPredicate
	{
		bool operator()(const int32 A, const int32 B) const
		{
			return A > B;
		}
	};

	TSortedMap< int32, TArray< FName >, FDefaultAllocator, FDepthSortPredicate > DepthMap;

	for ( const auto& NameActorPair : DestinationSceneActor.RelatedActors )
	{
		if ( AActor* RelatedActor = NameActorPair.Value.Get() )
		{
			int32 Depth = 0;

			AActor* ParentActor = RelatedActor->GetAttachParentActor();
			while ( ParentActor )
			{
				++Depth;
				ParentActor = ParentActor->GetAttachParentActor();
			}

			TArray< FName >& ActorNamesAtDepth = DepthMap.FindOrAdd( Depth );
			ActorNamesAtDepth.Add( NameActorPair.Key );
		}
	}

	for ( auto DepthNamesPair : DepthMap )
	{
		for ( FName& ActorUniqueId : DepthNamesPair.Value )
		{
			AActor* Actor = DestinationSceneActor.RelatedActors[ ActorUniqueId ].Get();
			const TSoftObjectPtr< AActor > SourceActorPtr = SourceSceneActor.RelatedActors.FindRef( ActorUniqueId );
			const bool bIsSourceActorValid = SourceActorPtr.IsValid() && !SourceActorPtr->IsPendingKillPending();

			if ( Actor && !IgnoredDatasmithActors.Contains( ActorUniqueId ) )
			{
				if ( bIsSourceActorValid )
				{
					// Check if we need to delete some components.

					AActor* SourceActor = SourceActorPtr.Get();
					check( SourceActor );

					// Collects the imported components
					TSet< FName > ImportedDatasmithComponents;
					ImportedDatasmithComponents.Reserve( SourceActor->GetComponents().Num() );
					for ( UActorComponent* SourceComponent : SourceActor->GetComponents() )
					{
						FName DatasmithId = GetDatasmithElementId( SourceComponent );
						if ( !DatasmithId.IsNone() )
						{
							ImportedDatasmithComponents.Add( DatasmithId );
						}
					}

					// Collects the components to be removed
					TArray< UActorComponent* > ComponentsToRemove;
					for ( UActorComponent* Component : Actor->GetComponents() )
					{
						FName DatasmithId = GetDatasmithElementId( Component );
						if ( !( DatasmithId.IsNone() || ImportedDatasmithComponents.Contains( DatasmithId ) || IgnoredDatasmithActors.Contains(ActorUniqueId) ) )
						{
							ComponentsToRemove.Add( Component );
						}
					}

					// Removed the non imported components
					for ( UActorComponent* ComponentToRemove : ComponentsToRemove )
					{
						// Some components can destroy other components when being destroyed
						if ( !ComponentToRemove->IsBeingDestroyed() )
						{
							ComponentToRemove->DestroyComponent( true );
						}
					}
				}
				else
				{
					// Deleting the non imported actor

					// Make a copy because the array in RootComponent will get modified during the process
					TArray< USceneComponent* > AttachChildren = Actor->GetRootComponent()->GetAttachChildren();
					USceneComponent* AttachParent = Actor->GetRootComponent()->GetAttachParent();
					for ( USceneComponent* ChildComponent : AttachChildren )
					{
						if ( ChildComponent->GetOwner() != Actor && !ChildComponent->GetOwner()->IsActorBeingDestroyed() )
						{
							// If the component has a template pointing to the parent about to be deleted, update the template
							// to the new parent to avoid creating a template override where there was none.
							if ( UDatasmithSceneComponentTemplate* ComponentTemplate = FDatasmithObjectTemplateUtils::GetObjectTemplate<UDatasmithSceneComponentTemplate>( ChildComponent ) )
							{
								if ( ComponentTemplate->AttachParent == ChildComponent->GetAttachParent() )
								{
									ComponentTemplate->AttachParent = AttachParent;
								}
							}

							// Reattach our children to our parent
							ChildComponent->AttachToComponent( AttachParent, FAttachmentTransformRules::KeepWorldTransform );
						}
					}

					DeleteActor( *Actor );
					DestinationSceneActor.RelatedActors.Remove( ActorUniqueId );
				}
			}
		}
	}
}


void FDatasmithImporterUtils::DeleteActor( AActor& Actor )
{
	UWorld* ActorWorld = Actor.GetWorld();
	if ( ActorWorld )
	{
		if ( ActorWorld == GWorld )
		{
			if ( GEditor )
			{
				ULayersSubsystem* LayersSubsystem = GEditor->GetEditorSubsystem<ULayersSubsystem>();
				LayersSubsystem->DisassociateActorFromLayers( &Actor );
			}
		}

		// Clean up all references to external assets within the actor and its components since those will be deleted later
		// #ueent_remark: Underlying question why the actor and its components are still reachable after being 'deleted'
		{
			struct FObjectExternalReferenceCleaner : public FArchiveUObject
			{
				FObjectExternalReferenceCleaner()	{}

				virtual FArchive& operator<<(UObject*& ObjRef) override
				{
					if(ObjRef != nullptr)
					{
						// Set to null any pointer to an external asset
						if( ObjRef->HasAnyFlags( RF_Standalone | RF_Public ) )
						{
							ObjRef = nullptr;
						}
					}

					return *this;
				}
			};

			TArray< UObject* > SubObjectsArray;
			GetObjectsWithOuter( &Actor, SubObjectsArray, EGetObjectsFlags::IncludeNestedObjects);

			for(UObject* SubObject : SubObjectsArray)
			{
				if(SubObject)
				{
					FObjectExternalReferenceCleaner Ar;
					SubObject->Serialize(Ar);
				}
			}

			{
				FObjectExternalReferenceCleaner Ar;
				Actor.Serialize(Ar);
			}
		}

		// Make sure actor is deselected before deletion
		if (GEditor && Actor.IsSelected())
		{
			GEditor->SelectActor( &Actor, false, true );
		}
		// Actually delete the actor
		ActorWorld->EditorDestroyActor( &Actor, true );

		// Move the actor to the transient package so its object name can be use
		Actor.UObject::Rename( nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
	}
}

void FDatasmithImporterUtils::AddUniqueLayersToWorld(UWorld* World, const TSet< FName >& LayerNames)
{
	if (!World || !IsValidChecked(World) || World->IsUnreachable() || LayerNames.Num() == 0)
	{
		return;
	}

	TSet< FName > ExistingLayers;
	for (ULayer* Layer : World->Layers)
	{
		ExistingLayers.Add(Layer->GetLayerName());
	}

	ULayersSubsystem* LayersSubsystem = GEditor ? GEditor->GetEditorSubsystem<ULayersSubsystem>() : nullptr;
	for (const FName& LayerName : LayerNames)
	{
		if (!ExistingLayers.Contains(LayerName))
		{
			// Use the ILayers if we are adding the layers to the currently edited world
			if (LayersSubsystem && GWorld && World == GWorld.GetReference())
			{
				LayersSubsystem->CreateLayer(LayerName);
			}
			else
			{
				ULayer* NewLayer = NewObject<ULayer>(World, NAME_None, RF_Transactional);
				check(NewLayer != NULL);

				World->Modify();
				World->Layers.Add(NewLayer);

				NewLayer->SetLayerName(LayerName);
				NewLayer->SetVisible(true);
			}
		}
	}
}


int32 FDatasmithImporterUtils::GetAssetNameMaxCharCount(const UPackage* ParentPackage)
{
	// can be tweaked, the goal is to be more restrictive than the filesystem
	// so that a project can be shared / moved without breaking the constraint
	int32 MaxAssetPathLength = DatasmithImporterUtilsImpl::GetUsableMaxAssetPathLength();

	// internal limit of FNames + room for prefix, separators and null char. (Asset names occur twice in paths)
	int32 InternalNameConstraint = (NAME_SIZE - 100);

	int32 PackageLength = 1 + (ParentPackage ? ParentPackage->GetPathName().Len() : 20);
	int32 Budget = FMath::Min((InternalNameConstraint - PackageLength) / 2, MaxAssetPathLength - PackageLength);
	Budget = FMath::Min(Budget, 255 - 10); // a filename cannot be longer than 255, and we keep a small buffer for the extension
	return FMath::Max(0, Budget);
}


bool FDatasmithImporterUtils::CanCreateAsset(const FString& AssetPathName, const UClass* AssetClass, FText& OutFailReason)
{
	switch(CanCreateAsset(AssetPathName, AssetClass))
	{
		case EAssetCreationStatus::CS_NameTooShort:
		{
			OutFailReason = FText::Format(LOCTEXT("AssetPathTooShort", "Invalid asset path {0} (name too short). Skipping this asset..."), FText::FromString(AssetPathName));
			return false;
		}

		case EAssetCreationStatus::CS_NameTooLong:
		{
			OutFailReason = FText::Format(LOCTEXT("AssetPathTooLong", "Path too long for asset {0}. Skipping this asset..."), FText::FromString(AssetPathName));
			return false;
		}

		case EAssetCreationStatus::CS_HasRedirector:
		{
			OutFailReason = FText::Format(LOCTEXT("FoundRedirectionForAsset", "Found redirection for asset {0}. Skipping this asset..."), FText::FromString(AssetPathName));
			return false;
		}

		case EAssetCreationStatus::CS_ClassMismatch:
		{
			IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
			const FAssetData AssetData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(AssetPathName));

			const FString FoundClassName(AssetData.GetClass()->GetFName().ToString());
			const FString ExpectedClassName(AssetClass->GetFName().ToString());
			OutFailReason = FText::Format(LOCTEXT("AssetClassMismatch", "Found asset {0} of class {1} instead of class {2}. Skipping this asset..."), FText::FromString(AssetPathName), FText::FromString(FoundClassName), FText::FromString(ExpectedClassName) );
			return false;
		}

		case EAssetCreationStatus::CS_CanCreate:
		default:
		{
			break;
		}
	}

	return true;
}

FDatasmithImporterUtils::EAssetCreationStatus FDatasmithImporterUtils::CanCreateAsset(const FString& AssetPathName, const UClass* AssetClass)
{
	if (AssetPathName.Len() >= NAME_SIZE)
	{
		return EAssetCreationStatus::CS_NameTooLong;
	}

	int32 IndexOfLastSlash = INDEX_NONE;
	AssetPathName.FindLastChar('/', IndexOfLastSlash);
	if (AssetPathName.Len() - IndexOfLastSlash <= 2)
	{
		FString AssetName = FPackageName::GetShortName(AssetPathName);
		return EAssetCreationStatus::CS_NameTooShort;
	}

	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();

	const FAssetData AssetData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(AssetPathName));

	// Asset does not exist yet. Safe to import
	if (!AssetData.IsValid())
	{
		return EAssetCreationStatus::CS_CanCreate;
	}

	// Warn and skip import of asset since it is an object redirection
	if (AssetData.IsRedirector())
	{
		return EAssetCreationStatus::CS_HasRedirector;
	}
	// Warn and skip re-import of asset since it is not of the expected class
	else if (!AssetData.GetClass()->IsChildOf(AssetClass))
	{
		return EAssetCreationStatus::CS_ClassMismatch;
	}

	return EAssetCreationStatus::CS_CanCreate;
}

UDatasmithScene* FDatasmithImporterUtils::FindDatasmithSceneForAsset( UObject* Asset )
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked< FAssetRegistryModule >( TEXT("AssetRegistry") );
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	TArray< FAssetData > DatasmithSceneAssets;
	AssetRegistry.GetAssetsByClass( UDatasmithScene::StaticClass()->GetClassPathName(), DatasmithSceneAssets, true );

	for ( FAssetData& DatasmithSceneAsset : DatasmithSceneAssets )
	{
		if ( UDatasmithScene* DatasmithScene = Cast< UDatasmithScene >( DatasmithSceneAsset.GetAsset() ) )
		{
			if ( UStaticMesh* StaticMesh = Cast< UStaticMesh >( Asset ) )
			{
				for ( const auto& AssetPair : *FDatasmithFindAssetTypeHelper< UStaticMesh >::GetAssetsMap( DatasmithScene ) )
				{
					if ( AssetPair.Value == StaticMesh )
					{
						return DatasmithScene;
					}
				}
			}
			else if ( UMaterialInterface* MaterialInterface = Cast< UMaterialInterface >( Asset ) )
			{
				for ( const auto& AssetPair : *FDatasmithFindAssetTypeHelper< UMaterialInterface >::GetAssetsMap( DatasmithScene ) )
				{
					if ( AssetPair.Value == MaterialInterface )
					{
						return DatasmithScene;
					}
				}
			}
			else if ( UTexture* Texture = Cast< UTexture >( Asset ) )
			{
				for ( const auto& AssetPair : *FDatasmithFindAssetTypeHelper< UTexture >::GetAssetsMap( DatasmithScene ) )
				{
					if ( AssetPair.Value == Texture )
					{
						return DatasmithScene;
					}
				}
			}
			else if ( ULevelSequence* LevelSequence = Cast< ULevelSequence >( Asset ) )
			{
				for ( const auto& AssetPair : *FDatasmithFindAssetTypeHelper< ULevelSequence >::GetAssetsMap( DatasmithScene ) )
				{
					if ( AssetPair.Value == LevelSequence )
					{
						return DatasmithScene;
					}
				}
			}
			else if ( ULevelVariantSets* LevelVariantSets = Cast< ULevelVariantSets >( Asset ) )
			{
				for ( const auto& AssetPair : *FDatasmithFindAssetTypeHelper< ULevelVariantSets >::GetAssetsMap( DatasmithScene ) )
				{
					if ( AssetPair.Value == LevelVariantSets )
					{
						return DatasmithScene;
					}
				}
			}
		}
	}

	return nullptr;
}

FName FDatasmithImporterUtils::GetDatasmithElementId( UObject* Object )
{
	FString Id = GetDatasmithElementIdString(Object);
	return Id.IsEmpty() ? NAME_None : FName(*Id);
}

FString FDatasmithImporterUtils::GetDatasmithElementIdString(UObject* Object)
{
	FString ElementId = UDatasmithAssetUserData::GetDatasmithUserDataValueForKey(Object, UDatasmithAssetUserData::UniqueIdMetaDataKey);

	if( Object != nullptr && ElementId.IsEmpty() )
	{
		const FString ObjectPath = FPaths::Combine( Object->GetOutermost()->GetName(), Object->GetName() );
		return FMD5::HashBytes( reinterpret_cast<const uint8*>(*ObjectPath), ObjectPath.Len() * sizeof(TCHAR) );
	}

	return ElementId;
}

namespace FDatasmithImporterUtilsHelper
{
	void SetupPointLightElement( const UPointLightComponent* PointLightComponent, IDatasmithPointLightElement* PointLightElement )
	{
		switch ( PointLightComponent->IntensityUnits )
		{
		case ELightUnits::Candelas:
			PointLightElement->SetIntensityUnits( EDatasmithLightUnits::Candelas );
			break;
		case ELightUnits::Lumens:
			PointLightElement->SetIntensityUnits( EDatasmithLightUnits::Lumens );
			break;
		case ELightUnits::EV:
			PointLightElement->SetIntensityUnits( EDatasmithLightUnits::EV );
			break;
		default:
			PointLightElement->SetIntensityUnits( EDatasmithLightUnits::Unitless );
			break;
		}

		PointLightElement->SetSourceRadius( PointLightComponent->SourceRadius );
		PointLightElement->SetSourceLength( PointLightComponent->SourceLength );
		PointLightElement->SetAttenuationRadius( PointLightComponent->AttenuationRadius );

		if( PointLightComponent->IESTexture != nullptr && PointLightComponent->IESTexture->AssetImportData != nullptr )
		{
			PointLightElement->SetIesFile( *PointLightComponent->IESTexture->AssetImportData->GetFirstFilename() );
			PointLightElement->SetUseIesBrightness( PointLightComponent->bUseIESBrightness );
			PointLightElement->SetIesBrightnessScale( PointLightComponent->IESBrightnessScale );
			// #ueent_todo: What about IES file rotation
		}
	}

	TSharedPtr< IDatasmithActorElement > ConvertAreaLightActorToActorElement( const ADatasmithAreaLightActor* AreaLightActor )
	{
		TSharedPtr< IDatasmithAreaLightElement > AreaLightElement = FDatasmithSceneFactory::CreateAreaLight( *AreaLightActor->GetName() );

		AreaLightElement->SetLightShape( (EDatasmithLightShape)AreaLightActor->LightShape );
		AreaLightElement->SetLength( AreaLightActor->Dimensions.X );
		AreaLightElement->SetWidth( AreaLightActor->Dimensions.Y );
		AreaLightElement->SetColor( AreaLightActor->Color );
		AreaLightElement->SetIntensity( AreaLightActor->Intensity );
		AreaLightElement->SetIntensityUnits( (EDatasmithLightUnits)AreaLightActor->IntensityUnits );
		AreaLightElement->SetTemperature( AreaLightActor->Temperature );
		AreaLightElement->SetUseTemperature( AreaLightActor->Temperature != 6500.0f );

		if( AreaLightActor->IESTexture != nullptr && AreaLightActor->IESTexture->AssetImportData != nullptr )
		{
			AreaLightElement->SetIesFile( *AreaLightActor->IESTexture->AssetImportData->GetFirstFilename() );
			AreaLightElement->SetUseIesBrightness( AreaLightActor->bUseIESBrightness );
			AreaLightElement->SetIesBrightnessScale( AreaLightActor->IESBrightnessScale );
			AreaLightElement->SetIesRotation( AreaLightActor->Rotation.Quaternion() );
		}

		AreaLightElement->SetSourceRadius( AreaLightActor->SourceRadius );
		AreaLightElement->SetSourceLength( AreaLightActor->SourceLength );
		AreaLightElement->SetAttenuationRadius( AreaLightActor->AttenuationRadius );

		return AreaLightElement;
	}

	TSharedPtr< IDatasmithActorElement > ConvertLightActorToActorElement(const ALight* LightActor)
	{
		TSharedPtr< IDatasmithLightActorElement > LightActorElement;

		const ULightComponent* LightComponent = LightActor->GetLightComponent();

		if(LightActor->IsA<ADirectionalLight>())
		{
			LightActorElement = FDatasmithSceneFactory::CreateDirectionalLight( *LightActor->GetName() );
		}
		else if(LightActor->IsA<ASpotLight>())
		{
			TSharedPtr< IDatasmithSpotLightElement > SpotLightActorElement = FDatasmithSceneFactory::CreateSpotLight( *LightActor->GetName() );

			const USpotLightComponent* SpotLightComponent = Cast<USpotLightComponent>(LightComponent);

			SetupPointLightElement( SpotLightComponent, SpotLightActorElement.Get() );

			SpotLightActorElement->SetInnerConeAngle( SpotLightComponent->InnerConeAngle );
			SpotLightActorElement->SetOuterConeAngle(SpotLightComponent->OuterConeAngle );

			LightActorElement = SpotLightActorElement;
		}
		else if(LightActor->IsA<ARectLight>())
		{
			UE_LOGF(LogDatasmithImport, Warning, "Import of RectLight is not yet supported");
			return TSharedPtr< IDatasmithLightActorElement >();
		}
		else
		{
			TSharedPtr< IDatasmithPointLightElement > PointLightActorElement = FDatasmithSceneFactory::CreatePointLight( *LightActor->GetName() );

			SetupPointLightElement( Cast<UPointLightComponent>(LightComponent), PointLightActorElement.Get() );

			LightActorElement = PointLightActorElement;
		}

		LightActorElement->SetEnabled( LightComponent->IsVisible() );
		LightActorElement->SetIntensity( LightComponent->Intensity );
		LightActorElement->SetColor( FLinearColor( LightComponent->LightColor ) );
		LightActorElement->SetUseTemperature( LightComponent->bUseTemperature );
		LightActorElement->SetTemperature( LightComponent->Temperature );

		if( LightComponent->LightFunctionMaterial != nullptr )
		{
			FString MaterialTag = FDatasmithImporterUtils::GetDatasmithElementIdString( LightComponent->LightFunctionMaterial );
			TSharedPtr< IDatasmithMaterialIDElement > MaterialIDElement = FDatasmithSceneFactory::CreateMaterialId( *MaterialTag );
			LightActorElement->SetLightFunctionMaterial( MaterialIDElement );
		}

		return LightActorElement;
	}

	TSharedPtr< IDatasmithPostProcessElement > ConvertPostProcessToPostProcessElement( const FPostProcessSettings& PostProcessSettings)
	{
		TSharedPtr< IDatasmithPostProcessElement > PostProcessElement = FDatasmithSceneFactory::CreatePostProcess();

		PostProcessElement->SetTemperature( PostProcessSettings.WhiteTemp );
		PostProcessElement->SetVignette( PostProcessSettings.VignetteIntensity );
		PostProcessElement->SetSaturation( PostProcessSettings.ColorSaturation.W );

		if( PostProcessSettings.AutoExposureMethod == EAutoExposureMethod::AEM_Manual )
		{
			PostProcessElement->SetCameraISO( PostProcessSettings.CameraISO );
			PostProcessElement->SetCameraShutterSpeed( PostProcessSettings.CameraShutterSpeed );
			PostProcessElement->SetDepthOfFieldFstop( PostProcessSettings.DepthOfFieldFstop );
		}

		return PostProcessElement;
	}

	TSharedPtr< IDatasmithActorElement > ConvertCameraActorToActorElement(const ACineCameraActor* CameraActor)
	{
		TSharedPtr< IDatasmithCameraActorElement > CameraElement = FDatasmithSceneFactory::CreateCameraActor( *CameraActor->GetName() );

		const UCineCameraComponent* CineCameraComponent = CameraActor->GetCineCameraComponent();

		CameraElement->SetSensorWidth( CineCameraComponent->Filmback.SensorWidth );
		CameraElement->SetSensorAspectRatio( CineCameraComponent->Filmback.SensorWidth / CineCameraComponent->Filmback.SensorHeight );
		CameraElement->SetFocalLength( CineCameraComponent->CurrentFocalLength );
		CameraElement->SetFStop( CineCameraComponent->CurrentAperture );
		CameraElement->SetEnableDepthOfField( CineCameraComponent->FocusSettings.FocusMethod == ECameraFocusMethod::Manual );

		TSharedPtr< IDatasmithPostProcessElement > PostProcessElement = ConvertPostProcessToPostProcessElement( CineCameraComponent->PostProcessSettings );

		CameraElement->SetPostProcess( PostProcessElement );

		return CameraElement;
	}

	// #ueent_todo: Implement conversion of ALandscape to IDatasmithLandscapeElement
	TSharedPtr< IDatasmithActorElement > ConvertLandscapeActorToActorElement(const ALandscape* LandscapeActor)
	{
		TSharedPtr< IDatasmithLandscapeElement > LandscapeActorElement = FDatasmithSceneFactory::CreateLandscape( *LandscapeActor->GetName() );

		LandscapeActorElement->SetScale( LandscapeActor->GetActorRelativeScale3D() );

		return LandscapeActorElement;
	}

	void ExtractMetaDataFromActor(const AActor* Actor, TSharedPtr< IDatasmithActorElement >& ActorElement, TSharedPtr<IDatasmithScene>& SceneElement)
	{
		if(UActorComponent* ActorComponent = Actor->GetRootComponent())
		{
			if ( ActorComponent->GetClass()->ImplementsInterface( UInterface_AssetUserData::StaticClass() ) )
			{
				IInterface_AssetUserData* AssetUserData = Cast< IInterface_AssetUserData >( ActorComponent );

				if( UDatasmithAssetUserData* DatasmithUserData = AssetUserData->GetAssetUserData< UDatasmithAssetUserData >() )
				{
					const FName UniqueIdMetaDataKey( UDatasmithAssetUserData::UniqueIdMetaDataKey );

					TSharedPtr<IDatasmithMetaDataElement> MetaDataElement = FDatasmithSceneFactory::CreateMetaData( ActorElement->GetName() );

					for(const auto& MetaDataEntry : DatasmithUserData->MetaData)
					{
						if( MetaDataEntry.Key != UniqueIdMetaDataKey )
						{
							TSharedRef< IDatasmithKeyValueProperty > KeyValueProperty = FDatasmithSceneFactory::CreateKeyValueProperty( *MetaDataEntry.Key.ToString() );

							KeyValueProperty->SetValue( *MetaDataEntry.Value );
							KeyValueProperty->SetPropertyType( EDatasmithKeyValuePropertyType::String );

							MetaDataElement->AddProperty( KeyValueProperty );
						}
					}

					// Add meta data element if there is anything
					if( MetaDataElement->GetPropertiesCount() > 0 )
					{
						MetaDataElement->SetAssociatedElement( ActorElement );
						SceneElement->AddMetaData( MetaDataElement );
					}
				}
			}
		}
	}

	TSharedPtr< IDatasmithActorElement > ConvertActorToActorElement( AActor* Actor, TSharedPtr<IDatasmithScene>& SceneElement )
	{
		if(Actor == nullptr)
		{
			return TSharedPtr< IDatasmithActorElement >();
		}

		FSoftObjectPath LightShapeBlueprintRef = FSoftObjectPath( TEXT("/DatasmithContent/Datasmith/DatasmithArealight.DatasmithArealight") );
		UBlueprint* LightShapeBlueprint = Cast< UBlueprint >( LightShapeBlueprintRef.TryLoad() );

		bool bNeedsTemplates = FDatasmithObjectTemplateUtils::GetObjectTemplate<UDatasmithActorTemplate>( Actor ) == nullptr;

		auto AddTemplate = [](UClass* TemplateClass, UObject* Source, UObject* Outer)
		{
			UDatasmithObjectTemplate* DatasmithTemplate = NewObject< UDatasmithObjectTemplate >( Outer, TemplateClass );
			DatasmithTemplate->Load( Source );
			FDatasmithObjectTemplateUtils::SetObjectTemplate( Outer, DatasmithTemplate );
		};

		auto CreateMeshActorElement = [&](FString& ElementName, UStaticMeshComponent* StaticMeshComponent)
		{
			TSharedPtr< IDatasmithMeshActorElement > StaticMeshActorElement = FDatasmithSceneFactory::CreateMeshActor( *ElementName );

			for( UMaterialInterface* MaterialInterface : StaticMeshComponent->OverrideMaterials )
			{
				if ( MaterialInterface )
				{
					StaticMeshActorElement->AddMaterialOverride( *MaterialInterface->GetName(), 0 );
				}
				StaticMeshActorElement->AddMaterialOverride( TEXT(""), 0 );
			}

			if ( UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh() )
			{
				FString StaticMeshTag = FDatasmithImporterUtils::GetDatasmithElementIdString( StaticMesh );
				StaticMeshActorElement->SetStaticMeshPathName(*StaticMeshTag);
			}

			if(bNeedsTemplates)
			{
				AddTemplate( UDatasmithStaticMeshComponentTemplate::StaticClass(), StaticMeshComponent, StaticMeshComponent );
			}

			return StaticMeshActorElement;
		};

		FString ActorName = Actor->GetName();

		if(bNeedsTemplates)
		{
			AddTemplate( UDatasmithActorTemplate::StaticClass(), Actor, Actor );

			USceneComponent* RootComponent = Actor->GetRootComponent();
			AddTemplate( UDatasmithSceneComponentTemplate::StaticClass(), RootComponent, RootComponent->GetOwner() );
			UDatasmithAssetUserData::SetDatasmithUserDataValueForKey(RootComponent, UDatasmithAssetUserData::UniqueIdMetaDataKey, Actor->GetName() );
		}

		TSharedPtr< IDatasmithActorElement > ActorElement;
		// #ueent_todo: Add proper support for all type of actors
		if( AStaticMeshActor* StaticMeshActor = Cast<AStaticMeshActor>(Actor) )
		{
			ActorElement = CreateMeshActorElement( ActorName, StaticMeshActor->GetStaticMeshComponent() );
		}
		else if( ADatasmithAreaLightActor* AreaLightActor = Cast<ADatasmithAreaLightActor>(Actor) )
		{
			if(bNeedsTemplates)
			{
				AddTemplate( UDatasmithAreaLightActorTemplate::StaticClass(), AreaLightActor, AreaLightActor );
			}

			ActorElement = ConvertAreaLightActorToActorElement( AreaLightActor );
		}
		else if( ALight* LightActor = Cast<ALight>(Actor) )
		{
			if(bNeedsTemplates)
			{
				ULightComponent* LightComponent = LightActor->GetLightComponent();
				AddTemplate( UDatasmithLightComponentTemplate::StaticClass(), LightComponent, LightComponent );

				if(UPointLightComponent* PointLightComponent = Cast< UPointLightComponent >(LightComponent))
				{
					AddTemplate( UDatasmithPointLightComponentTemplate::StaticClass(), LightComponent, LightComponent );
				}
			}

			ActorElement = ConvertLightActorToActorElement( LightActor );
		}
		else if( ALightmassPortal* LightmassActor = Cast<ALightmassPortal>(Actor) )
		{
			TSharedPtr< IDatasmithLightmassPortalElement > LightmassActorElement = FDatasmithSceneFactory::CreateLightmassPortal( *ActorName );
			ActorElement = LightmassActorElement;
		}
		else if( ACineCameraActor* CameraActor = Cast<ACineCameraActor>(Actor) )
		{
			if(bNeedsTemplates)
			{
				UCineCameraComponent* CineCameraComponent = CameraActor->GetCineCameraComponent();
				AddTemplate( UDatasmithCineCameraComponentTemplate::StaticClass(), CineCameraComponent, CineCameraComponent );
			}

			ActorElement = ConvertCameraActorToActorElement( CameraActor );
		}
		else if( ALandscape* LandscapeActor = Cast<ALandscape>(Actor) )
		{
			ActorElement = ConvertLandscapeActorToActorElement( LandscapeActor );
		}
		else if( APostProcessVolume* PostProcessVolume = Cast<APostProcessVolume>(Actor) )
		{
			TSharedPtr< IDatasmithPostProcessVolumeElement > PostProcessVolumeElement = FDatasmithSceneFactory::CreatePostProcessVolume( *ActorName );

			PostProcessVolumeElement->SetEnabled( PostProcessVolume->bEnabled );
			PostProcessVolumeElement->SetUnbound( PostProcessVolume->bUnbound );

			TSharedPtr< IDatasmithPostProcessElement > PostProcessElement = ConvertPostProcessToPostProcessElement( PostProcessVolume->Settings );

			PostProcessVolumeElement->SetSettings( PostProcessElement.ToSharedRef() );

			ActorElement = PostProcessVolumeElement;
		}
		else
		{
			EDatasmithElementType ActorType = EDatasmithElementType::None;
			// Is this a IDatasmithHierarchicalInstancedStaticMeshActorElement?
			if( Actor->GetInstanceComponents().Num() > 0 )
			{
				for(const UActorComponent* ActorComponent : Actor->GetInstanceComponents())
				{
					if(Cast<UHierarchicalInstancedStaticMeshComponent>(ActorComponent) != nullptr)
					{
						ActorType = EDatasmithElementType::HierarchicalInstanceStaticMesh;
						break;
					}
				}
			}

			switch(ActorType)
			{
			case EDatasmithElementType::HierarchicalInstanceStaticMesh:
				{
					const UHierarchicalInstancedStaticMeshComponent* HISMComponent = nullptr;
					for(const UActorComponent* ActorComponent : Actor->GetInstanceComponents())
					{
						if(Cast<UHierarchicalInstancedStaticMeshComponent>(ActorComponent) != nullptr)
						{
							HISMComponent = Cast<UHierarchicalInstancedStaticMeshComponent>(ActorComponent);
							break;
						}
					}
					check( HISMComponent );

					TSharedPtr< IDatasmithHierarchicalInstancedStaticMeshActorElement > HISMActorElement = FDatasmithSceneFactory::CreateHierarchicalInstanceStaticMeshActor( *ActorName );
					HISMActorElement->ReserveSpaceForInstances( HISMComponent->GetInstanceCount() );

					for(int32 Index = 0; Index < HISMComponent->GetInstanceCount(); ++Index)
					{
						FTransform InstanceTransform;
						HISMComponent->GetInstanceTransform( Index, InstanceTransform );

						HISMActorElement->AddInstance( InstanceTransform );
					}

					for(UMaterialInterface* MaterialInterface : HISMComponent->OverrideMaterials)
					{
						// Quick fix to avoid a crash. The logic for material override seems bogus for HISMComponent and StaticMeshComponent
						if (MaterialInterface)
						{
							HISMActorElement->AddMaterialOverride(*MaterialInterface->GetName(), 0);
						}
					}

					FString StaticMeshTag = FDatasmithImporterUtils::GetDatasmithElementIdString( HISMComponent->GetStaticMesh() );
					HISMActorElement->SetStaticMeshPathName( *StaticMeshTag );

					ActorElement = HISMActorElement;
				}
				break;
			case EDatasmithElementType::None:
			default:
				{
					ActorElement = FDatasmithSceneFactory::CreateActor( *ActorName );
					TArray<UStaticMeshComponent*> MeshComponents;
					Actor->GetComponents( MeshComponents );
					for(UStaticMeshComponent* MeshComponent : MeshComponents)
					{
						TSharedPtr< IDatasmithMeshActorElement > MeshActorElement = CreateMeshActorElement( ActorName, MeshComponent );
						MeshActorElement->SetIsAComponent(true);
						ActorElement->AddChild( MeshActorElement );
					}
				}
				break;
			}
		}

		if (!ActorElement.IsValid())
		{
			return TSharedPtr< IDatasmithActorElement >();
		}

		// Store actor's label
		ActorElement->SetLabel( *Actor->GetActorLabel() );

		// Store actor's transform
		const FTransform& WorldTransform = Actor->GetTransform();

		ActorElement->SetTranslation( WorldTransform.GetLocation() );
		ActorElement->SetRotation( WorldTransform.GetRotation() );
		ActorElement->SetScale( WorldTransform.GetScale3D() );

		// Store actor's layers
		if(Actor->Layers.Num() > 0)
		{
			FString CsvLayersNames = Actor->Layers[0].ToString();
			for(int32 Index = 1; Index < Actor->Layers.Num(); ++Index)
			{
				CsvLayersNames += TEXT(",") + Actor->Layers[Index].ToString();
			}

			ActorElement->SetLayer( *CsvLayersNames );
		}

		ExtractMetaDataFromActor( Actor, ActorElement, SceneElement );

		return ActorElement;
	}

	void AddActorElement(TSharedPtr<IDatasmithActorElement>& ParentActorElement, TSharedPtr<IDatasmithScene>& SceneElement, const TArray<AActor*>& ChildrenActors)
	{
		for(AActor* ChildActor : ChildrenActors)
		{
			TSharedPtr< IDatasmithActorElement > ChildActorElement = ConvertActorToActorElement( ChildActor, SceneElement );

			if (!ChildActorElement.IsValid())
			{
				continue;
			}

			ParentActorElement->AddChild( ChildActorElement );

			TArray<AActor*> ActorsToVisit;
			ChildActor->GetAttachedActors( ActorsToVisit );

			AddActorElement( ChildActorElement, SceneElement, ActorsToVisit );
		}
	}
}

void FDatasmithImporterUtils::FillSceneElement(TSharedPtr<IDatasmithScene>& SceneElement, const TArray<AActor*>& RootActors)
{
	for( AActor* RootActor : RootActors )
	{
		// Convert root actor to actor element
		TSharedPtr< IDatasmithActorElement > RootActorElement = FDatasmithImporterUtilsHelper::ConvertActorToActorElement( RootActor, SceneElement );

		if (!RootActorElement.IsValid())
		{
			continue;
		}

		// Add newly created actor element to scene element
		SceneElement->AddActor( RootActorElement );

		// Recursively parse children of root actor and add them to root actor element
		TArray<AActor*> ActorsToVisit;
		RootActor->GetAttachedActors( ActorsToVisit );

		FDatasmithImporterUtilsHelper::AddActorElement( RootActorElement, SceneElement, ActorsToVisit );
	}
}

TArray<FDatasmithImporterUtils::FFunctionAndMaterialsThatUseIt> FDatasmithImporterUtils::GetOrderedListOfMaterialsReferencedByMaterials(TSharedPtr< IDatasmithScene >& SceneElement)
{
	//This map is used to keep track of which materials are referencing which
	TMap<FString, TSet<FString, TStringPointerSetKeyFuncs_DEPRECATED<FString>>> MaterialToFunctionNameMap;
	//Mapping materials to their names for easy access.
	TMap<FString, TSharedPtr<IDatasmithUEPbrMaterialElement>> MaterialNameMap;


	MaterialToFunctionNameMap.Reserve(SceneElement->GetMaterialsCount());
	MaterialNameMap.Reserve(SceneElement->GetMaterialsCount());
	for (int32 MaterialIndex = 0; MaterialIndex < SceneElement->GetMaterialsCount(); ++MaterialIndex)
	{
		const TSharedPtr<IDatasmithBaseMaterialElement> BaseMaterialElement = SceneElement->GetMaterial(MaterialIndex);

		if (!BaseMaterialElement->IsA(EDatasmithElementType::UEPbrMaterial))
		{
			continue;
		}

		const TSharedPtr<IDatasmithUEPbrMaterialElement> UEPbrMaterialElement = StaticCastSharedPtr<IDatasmithUEPbrMaterialElement>(BaseMaterialElement);

		FString BaseMaterialName = BaseMaterialElement->GetName();
		uint32 BaseMaterialHash = GetTypeHash(BaseMaterialName);
		MaterialNameMap.FindOrAddByHash(BaseMaterialHash, MoveTemp(BaseMaterialName)) = UEPbrMaterialElement;

		for (int32 MaterialExpressionIndex = 0; MaterialExpressionIndex < UEPbrMaterialElement->GetExpressionsCount(); ++MaterialExpressionIndex)
		{
			if (UEPbrMaterialElement->GetExpression(MaterialExpressionIndex)->IsSubType(EDatasmithMaterialExpressionType::FunctionCall))
			{
				const FString FunctionPathName(StaticCast<IDatasmithMaterialExpressionFunctionCall*>(UEPbrMaterialElement->GetExpression(MaterialExpressionIndex))->GetFunctionPathName());
				if (FPaths::IsRelative(FunctionPathName))
				{
					uint32 FunctionPathNameHash = GetTypeHash(FunctionPathName);
					if (TSet<FString, TStringPointerSetKeyFuncs_DEPRECATED<FString>>* Values = MaterialToFunctionNameMap.FindByHash(FunctionPathNameHash, FunctionPathName))
					{
						check(!Values->ContainsByHash(BaseMaterialHash, BaseMaterialElement->GetName())); //Can't have inter-dependencies
					}

					MaterialToFunctionNameMap.FindOrAddByHash(BaseMaterialHash, BaseMaterialElement->GetName()).AddByHash(FunctionPathNameHash, FunctionPathName);
				}
			}
		}
	}


	TArray<FFunctionAndMaterialsThatUseIt> ReferencedMaterialAndTheirReferencer;
	ReferencedMaterialAndTheirReferencer.Reserve(MaterialToFunctionNameMap.Num());
	TMap<TSharedPtr<IDatasmithUEPbrMaterialElement>, int32> VisitedAndInsertionIndex;
	VisitedAndInsertionIndex.Reserve(MaterialToFunctionNameMap.Num());

	// Topological sort while also noting who was the direct dependents of a node also. Base on a DFS
	TFunction<void (const TSharedPtr<IDatasmithUEPbrMaterialElement>&, const TSet<FString, TStringPointerSetKeyFuncs_DEPRECATED<FString>>&, const TSharedPtr<IDatasmithUEPbrMaterialElement>&)> DepthFirstSearch;

	DepthFirstSearch = [&MaterialToFunctionNameMap, &MaterialNameMap, &ReferencedMaterialAndTheirReferencer, &VisitedAndInsertionIndex, &DepthFirstSearch] (const TSharedPtr<IDatasmithUEPbrMaterialElement>& CurrentElement, const TSet<FString, TStringPointerSetKeyFuncs_DEPRECATED<FString>>& Dependencies, const TSharedPtr<IDatasmithUEPbrMaterialElement>& Parent)
		{
			uint32 CurrentHash = GetTypeHash(CurrentElement);
			// Check if visited
			if (int32* Index = VisitedAndInsertionIndex.FindByHash(CurrentHash, CurrentElement))
			{
				// update output to add the dependent
				ReferencedMaterialAndTheirReferencer[*Index].Value.Add(Parent);
				return;
			}

			// Visit children
			for (const FString& Dependency : Dependencies)
			{
				uint32 DependencyHash = GetTypeHash(Dependency);
				const TSharedPtr<IDatasmithUEPbrMaterialElement>* NextElement = MaterialNameMap.FindByHash(DependencyHash, Dependency);
				if (NextElement)
				{
					if (const TSet<FString, TStringPointerSetKeyFuncs_DEPRECATED<FString>>* NextElementDependencies = MaterialToFunctionNameMap.FindByHash(DependencyHash, Dependency))
					{
						DepthFirstSearch(*NextElement, *NextElementDependencies, CurrentElement);
					}
					else
					{
						DepthFirstSearch(*NextElement, TSet<FString, TStringPointerSetKeyFuncs_DEPRECATED<FString>>(), CurrentElement);
					}
				}
			}

			// Mark Visited and add to output
			VisitedAndInsertionIndex.AddByHash(CurrentHash, CurrentElement, ReferencedMaterialAndTheirReferencer.Num());
			TArray<TSharedPtr<IDatasmithUEPbrMaterialElement>> Parents;
			Parents.Add(Parent);

			ReferencedMaterialAndTheirReferencer.Emplace(CurrentElement, MoveTemp(Parents));
		};

	for (const TPair<FString, TSet<FString, TStringPointerSetKeyFuncs_DEPRECATED<FString>>>& Pair : MaterialToFunctionNameMap)
	{
		const TSharedPtr<IDatasmithUEPbrMaterialElement>& CurrentElement = MaterialNameMap.FindChecked(Pair.Key);
		// Only if not yet visited
		if (!VisitedAndInsertionIndex.Contains(CurrentElement))
		{
			// Only DFS the dependencies because we don't want to add the top level material as they don't need to be functions
			for (const FString& Dependency : Pair.Value)
			{
				uint32 DependencyHash = GetTypeHash(Dependency);
				const TSharedPtr<IDatasmithUEPbrMaterialElement>* CurrentDependency = MaterialNameMap.FindByHash(DependencyHash, Dependency);
				if (CurrentDependency)
				{
					if (const TSet<FString, TStringPointerSetKeyFuncs_DEPRECATED<FString>>* DependenciesOfDependency = MaterialToFunctionNameMap.FindByHash(DependencyHash, Dependency))
					{
						DepthFirstSearch(*CurrentDependency, *DependenciesOfDependency, CurrentElement);
					}
					else
					{
						DepthFirstSearch(*CurrentDependency, TSet<FString, TStringPointerSetKeyFuncs_DEPRECATED<FString>>(), CurrentElement);
					}
				}
			}
		}
	}

	return ReferencedMaterialAndTheirReferencer;
}

FDatasmithImporterUtils::FDatasmithMaterialImportIterator::FDatasmithMaterialImportIterator(const FDatasmithImportContext& InImportContext)
	: ImportContext(InImportContext),
	CurrentIndex(0)
{
	if (!ImportContext.bIsAReimport || !ImportContext.SceneAsset)
	{
		return;
	}

	auto DoesParentWithSameNameExist = [this](TSharedPtr<IDatasmithBaseMaterialElement> Mat)
	{
		if (TSoftObjectPtr<UMaterialInterface>* MatInterface = ImportContext.SceneAsset->Materials.Find(Mat->GetName()))
		{
			if (UMaterialInstanceConstant* MaterialInstance = Cast< UMaterialInstanceConstant >(MatInterface->Get()))
			{
				const FString ParentDatasmithUniqueId = UDatasmithAssetUserData::GetDatasmithUserDataValueForKey(MaterialInstance->Parent, UDatasmithAssetUserData::UniqueIdMetaDataKey);
				return MaterialInstance->Parent && ParentDatasmithUniqueId == Mat->GetName();
			}
		}

		return false;
	};

	SortedMaterials.Reserve(ImportContext.FilteredScene->GetMaterialsCount());
	int32 NumMaterialMovedToFront = 0;

	for (int32 MaterialIndex = 0; MaterialIndex < ImportContext.FilteredScene->GetMaterialsCount(); ++MaterialIndex)
	{
		SortedMaterials.Add(ImportContext.FilteredScene->GetMaterial(MaterialIndex));

		// Materials that are already existing and that were used to create their parent material should be imported first otherwise we might create new parent materials instead of reusing the existing ones.
		if (DoesParentWithSameNameExist(SortedMaterials[MaterialIndex]))
		{
			if (NumMaterialMovedToFront != MaterialIndex)
			{
				SortedMaterials.Swap(NumMaterialMovedToFront,MaterialIndex);
			}
			NumMaterialMovedToFront++;
		}
	}
}

FDatasmithImporterUtils::FDatasmithMaterialImportIterator& FDatasmithImporterUtils::FDatasmithMaterialImportIterator::operator++()
{
	++CurrentIndex;
	return *this;
}

FDatasmithImporterUtils::FDatasmithMaterialImportIterator::operator bool() const
{
	if (ImportContext.bIsAReimport)
	{
		return CurrentIndex < SortedMaterials.Num();
	}
	else
	{
		return CurrentIndex < ImportContext.FilteredScene->GetMaterialsCount();
	}
}

const TSharedPtr<IDatasmithBaseMaterialElement>& FDatasmithImporterUtils::FDatasmithMaterialImportIterator::Value() const
{
	if (ImportContext.bIsAReimport)
	{
		return SortedMaterials[CurrentIndex];
	}
	else
	{
		//We need to use a const IDatasmithScene to be able to use the const-ref version on GetMaterial().
		const TSharedPtr<const IDatasmithScene>& DatasmithScene(ImportContext.FilteredScene);
		return DatasmithScene->GetMaterial(CurrentIndex);
	}
}

UObject* FDatasmithImporterUtils::StaticDuplicateObject(UObject* SourceObject, UObject* Outer, const FName Name)
{
	if (UStaticMesh* SourceMesh = Cast< UStaticMesh >(SourceObject))
	{
		const bool bIgnoreBulkData = false;
		return DuplicateStaticMesh(SourceMesh, Outer, Name, bIgnoreBulkData);
	}

	// Duplicate is used only to move our object from its temporary package into its final package replacing any asset
	// already at that location. This function also takes care of fixing internal dependencies among the object's children.
	// Since Duplicate has some rather heavy consequence, like calling PostLoad and doing all kind of stuff on an object
	// that is not even fully initialized yet, we might want to find an alternative way of moving our objects in future
	// releases but keep it for the current release cycle.
	return ::DuplicateObject< UObject >( SourceObject, Outer, Name );
}

UStaticMesh* FDatasmithImporterUtils::DuplicateStaticMesh(UStaticMesh* SourceStaticMesh, UObject* Outer, const FName Name, bool bIgnoreBulkData)
{
	// Since static mesh can be quite heavy, remove source models for cloning to reduce useless work.
	// Will be reinserted on the new duplicated asset or restored on the SourceStaticMesh if bIgnoreBulkData is true.
	TArray<FStaticMeshSourceModel> SourceModels = SourceStaticMesh->MoveSourceModels();
	FStaticMeshSourceModel HiResSourceModel = SourceStaticMesh->MoveHiResSourceModel();

	// Temporary flag to skip Postload during DuplicateObject
	SourceStaticMesh->SetFlags(RF_ArchetypeObject);

	// Duplicate is used only to move our object from its temporary package into its final package replacing any asset
	// already at that location. This function also takes care of fixing internal dependencies among the object's children.
	// Since Duplicate has some rather heavy consequence, like calling PostLoad and doing all kind of stuff on an object
	// that is not even fully initialized yet, we might want to find an alternative way of moving our objects in future
	// releases but keep it for the current release cycle.
	UStaticMesh* DuplicateMesh = ::DuplicateObject< UStaticMesh >(SourceStaticMesh, Outer, Name);

	// Get rid of our temporary flag
	SourceStaticMesh->ClearFlags(RF_ArchetypeObject);
	DuplicateMesh->ClearFlags(RF_ArchetypeObject);
	DuplicateMesh->GetHiResSourceModel().CreateSubObjects(DuplicateMesh);

	if (bIgnoreBulkData)
	{
		//We are not moving the source model bulk data, so we can simply copy the settings used in the DDC key.
		//That way we also avoid marking the source mesh as pending kill.
		for (FStaticMeshSourceModel& SourceModel : SourceModels)
		{
			FStaticMeshSourceModel& DuplicateSourceModel = DuplicateMesh->AddSourceModel();

			// Apply the SourceMesh settings to the duplicated SourceModels
			DuplicateSourceModel.BuildSettings = SourceModel.BuildSettings;
			DuplicateSourceModel.ReductionSettings = SourceModel.ReductionSettings;
			DuplicateSourceModel.ScreenSize = SourceModel.ScreenSize;
			DuplicateSourceModel.bImportWithBaseMesh = SourceModel.bImportWithBaseMesh;
			DuplicateSourceModel.SourceImportFilename = SourceModel.SourceImportFilename;
		}

		// Move back the source models to the original mesh
		SourceStaticMesh->SetSourceModels(MoveTemp(SourceModels));
		SourceStaticMesh->SetHiResSourceModel(MoveTemp(HiResSourceModel));
	}
	else
	{
		// The source mesh is stripped from it's source model, it is not buildable anymore.
		// -> MarkPendingKill to avoid use-after-move crash in the StaticMesh::Build()
		SourceStaticMesh->MarkAsGarbage();

		// Apply source models to the duplicated mesh
		DuplicateMesh->SetSourceModels(MoveTemp(SourceModels));
		DuplicateMesh->SetHiResSourceModel(MoveTemp(HiResSourceModel));
	}

	return DuplicateMesh;
}

FScopedLogger::FScopedLogger(FName LogTitle, const FText& LogLabel)
	: Title(LogTitle)
	, MessageLogModule(FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog"))
	, LogListing(MessageLogModule.GetLogListing(Title))
{
	LogListing->SetLabel(LogLabel);
}

FScopedLogger::~FScopedLogger()
{
	Dump(true);
}

TSharedRef<FTokenizedMessage> FScopedLogger::Push(EMessageSeverity::Type Severity, const FText& Message)
{
	FScopeLock Lock(&TokenizedMessageCS);

	TokenizedMessages.Add(FTokenizedMessage::Create(Severity, Message));

	return TokenizedMessages.Last();
}

void FScopedLogger::Dump(bool bClearPrevious)
{
	FScopeLock Lock(&TokenizedMessageCS);

	if (TokenizedMessages.Num() > 0)
	{
		if (bClearPrevious)
		{
			ClearLog();
		}

		LogListing->AddMessages(TokenizedMessages);
		LogListing->NotifyIfAnyMessages(NSLOCTEXT("DatasmithLoggerNotification", "Log", "There were issues with the import."), EMessageSeverity::Info);
		ClearPending();
	}
}

void FScopedLogger::ClearLog()
{
	LogListing->ClearMessages();
}

void FScopedLogger::ClearPending()
{
	FScopeLock Lock(&TokenizedMessageCS);

	TokenizedMessages.Empty();
}

bool FDatasmithImporterUtils::CreatePlmXmlSceneFromCADFiles(FString PlmXmlFileName, const TSet<FString>& FilesToProcess, TArray<FString>& FilesNotProcessed)
{
	// Find out which translator can import CAD files by retrieving translator for a JT file(this is expected to be 'DatasmithCADTranslator').
	// And then accept files which have this translator returned as compatible.
	FDatasmithSceneSource SomeCADFileSource;
	SomeCADFileSource.SetSourceFile("test.jt");
	TSharedPtr<IDatasmithTranslator> TranslatorForCADFiles = FDatasmithTranslatorManager::Get().SelectFirstCompatible(SomeCADFileSource);
	if (!TranslatorForCADFiles.IsValid())
	{
		UE_LOGF(LogDatasmithImport, Error, "Datasmith import error: no translator found for CAD files. Abort import.");
		return false;
	}

	FName CADTranslatorName = TranslatorForCADFiles->GetFName();
	TArray<FString> FilesToProcessWithPlmXml;
	for (const FString& FileName : FilesToProcess)
	{
		FDatasmithSceneSource Source;
		Source.SetSourceFile(FileName);
		TSharedPtr<IDatasmithTranslator> Translator = FDatasmithTranslatorManager::Get().SelectFirstCompatible(Source);

		bool bIsCADFile = Translator.IsValid() && (Translator->GetFName() == CADTranslatorName);
		if (bIsCADFile)
		{
			FilesToProcessWithPlmXml.Add(FileName);
		}
		else
		{
			FilesNotProcessed.Add(FileName);
			// XXX make warning for not processed
			// UE_LOGF(LogDatasmithImport, Warning, "Datasmith import error: '%ls' is not a CAD file, skipping.", *FileName);
		}
	}

	if (FilesToProcessWithPlmXml.Num() <= 0)
	{
		return false;
	}

	class FXmlWriter
	{
	public:
		FString Buffer;

		FXmlWriter()
		{
			Buffer = TEXT("<?xml version=\"1.0\" encoding=\"utf-8\"?>");
			Buffer += LINE_TERMINATOR;
		}

		class FTagGuard
		{
		public:
			FTagGuard(FXmlWriter& InWriter, FString Opening, FString InClosing)
				: Writer(InWriter)
				, Closing(InClosing)
			{
				Writer.Buffer += Opening;
				Writer.Buffer += LINE_TERMINATOR;
			}
			~FTagGuard()
			{
				Writer.Buffer += Closing;
				Writer.Buffer += LINE_TERMINATOR;
			}
		private:
			FXmlWriter& Writer;
			FString Closing;
		};
	};

	FXmlWriter Writer;
	FString& Buffer = Writer.Buffer;

	// Creating PLMXML file where each of files to process is referenced from a ProductRevisionView
	{
		FXmlWriter::FTagGuard PLMXMLTag(Writer, TEXT("<PLMXML xmlns=\"http://www.plmxml.org/Schemas/PLMXMLSchema\">"), "</PLMXML>");
		FXmlWriter::FTagGuard ProductDefTag(Writer, TEXT("<ProductDef id=\"id1\">"), TEXT("</ProductDef>"));

		// Used to assign unique ids to PLMXML entities being created
		int32 CurrentId = 2;

		// Collect all InstanceId to reference from InstanceGraph rootRefs
		TArray<FString> InstanceIds;
		InstanceIds.Reserve(FilesToProcessWithPlmXml.Num());
		for (const FString& FileName : FilesToProcessWithPlmXml)
		{
			InstanceIds.Add(FString::Printf(TEXT("id%d"), CurrentId++));
		}
		FXmlWriter::FTagGuard InstanceGraphTag(Writer, FString::Printf(TEXT("<InstanceGraph id=\"id2\" rootRefs=\"%s\">"), *FString::Join(InstanceIds, TEXT(" "))), TEXT("</InstanceGraph>"));

		for (int32 FileIndex = 0; FileIndex < FilesToProcessWithPlmXml.Num(); ++FileIndex)
		{
			const FString& FileName = FilesToProcessWithPlmXml[FileIndex];
			FString InstanceId = InstanceIds[FileIndex];
			FString InstanceName = FPaths::GetBaseFilename(FileName);
			FString PartId = FString::Printf(TEXT("id%d"), CurrentId++);
			FString RepresentationId = FString::Printf(TEXT("id%d"), CurrentId++);
			{
				FXmlWriter::FTagGuard ProductInstanceTag(Writer, FString::Printf(TEXT("<ProductInstance id=\"%s\" name=\"%s\" partRef=\"#%s\">"), *InstanceId, *InstanceName, *PartId), TEXT("</ProductInstance>"));
			}
			{
				FXmlWriter::FTagGuard ProductRevisionViewTag(Writer, FString::Printf(TEXT("<ProductRevisionView id=\"%s\" name=\"%s\">"), *PartId, *InstanceName), TEXT("</ProductRevisionView>"));
				//  omitting 'format' attribute, it's optional anyway
				FXmlWriter::FTagGuard RepresentationTag(Writer, FString::Printf(TEXT("<Representation id=\"%s\" location=\"%s\">"), *RepresentationId, *FileName), TEXT("</Representation>"));
			}
		}
	}

	if (!FFileHelper::SaveStringToFile(Buffer, *PlmXmlFileName, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM))
	{
		UE_LOGF(LogDatasmithImport, Error, "Datasmith import error: Failed to create PlmXml file '%ls' for parallel loading ...", *PlmXmlFileName);
		return false;
	}
	return true;
}

#undef LOCTEXT_NAMESPACE // "DatasmithImporterUtils"

