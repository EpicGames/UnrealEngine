// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utility/DatasmithTextureResize.h"

#include "DatasmithUtils.h"

#if WITH_FREEIMAGE_LIB

#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformProcess.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

THIRD_PARTY_INCLUDES_START
	#include "FreeImage.h"
THIRD_PARTY_INCLUDES_END

#if PLATFORM_WINDOWS
#   define TCHAR_TO_FICHAR TCHAR_TO_WCHAR
#   define FreeImage_GetFIFFromFilename FreeImage_GetFIFFromFilenameU
#   define FreeImage_GetFileType        FreeImage_GetFileTypeU
#   define FreeImage_Load               FreeImage_LoadU
#   define FreeImage_Save               FreeImage_SaveU
#else
#   define TCHAR_TO_FICHAR TCHAR_TO_UTF8
#endif

namespace
{
	int32 PreviousPowerOfTwo(int32 Reference)
	{
		int32 PreviousValue = 2;
		int32 NextValue = PreviousValue << 1;
		while (Reference > NextValue)
		{
			PreviousValue = NextValue;
			NextValue = PreviousValue << 1;
		}

		return PreviousValue;
	}

	int32 NextPowerOfTwo(int32 Reference)
	{
		return PreviousPowerOfTwo(Reference) << 1;
	}

	int32 NearestPowerOfTwo(int32 Reference)
	{
		int32 PreviousValue = PreviousPowerOfTwo(Reference);

		int32 NextValue = PreviousValue << 1;

		if ((NextValue - Reference)<(Reference - PreviousValue))
		{
			return NextValue;
		}
		else
		{
			return PreviousValue;
		}
	}

	int32 ToPowerOfTwo(int32 Reference, EDSResizeTextureMode ResizeTexturesMode)
	{
		switch (ResizeTexturesMode)
		{
		case EDSResizeTextureMode::NearestPowerOfTwo:
			return NearestPowerOfTwo(Reference);
		case EDSResizeTextureMode::PreviousPowerOfTwo:
			return PreviousPowerOfTwo(Reference);
		case EDSResizeTextureMode::NextPowerOfTwo:
			return NextPowerOfTwo(Reference);
		default:
			return Reference;
		}
	}

	class NinePoints
	{
	public:
		NinePoints(){ aa = ab = ac = ba = bb = bc = ca = cb = cc = 0; }
		float aa, ab, ac, ba, bb, bc, ca, cb, cc;
	};

	int32 Tile(int32 x, int32 MaxTile)
	{
		if (x < 0)
		{
			x = MaxTile - 1;
		}

		if (x >= MaxTile)
		{
			x = 0;
		}

		return x;
	}

	class FFreeImageWrapper
	{
	public:
		static bool IsValid() { return FreeImageDllHandle != nullptr; }

		static void FreeImage_Initialise(); // Loads and inits FreeImage on first call

	private:
		static void* FreeImageDllHandle; // Lazy init on first use, never release for now
	};

	void* FFreeImageWrapper::FreeImageDllHandle = nullptr;

	void FFreeImageWrapper::FreeImage_Initialise()
	{
		if (FreeImageDllHandle != nullptr)
		{
			return;
		}

		// Push/PopDllDirectory are soooooo not threadsafe!
		// Must load library in main thread before doing parallel processing
		check(IsInGameThread());

		if (FreeImageDllHandle == nullptr)
		{
			FString FreeImageDir = FPaths::Combine(FPaths::EngineDir(), TEXT("Binaries/ThirdParty/FreeImage"), FPlatformProcess::GetBinariesSubdirectory());
			FString FreeImageLibDir = FPaths::Combine(FreeImageDir, TEXT(FREEIMAGE_LIB_FILENAME));
			FPlatformProcess::PushDllDirectory(*FreeImageDir);
			FreeImageDllHandle = FPlatformProcess::GetDllHandle(*FreeImageLibDir);
			FPlatformProcess::PopDllDirectory(*FreeImageDir);
		}

		if (FreeImageDllHandle)
		{
			::FreeImage_Initialise();
		}
	}
}

class DatasmithTextureResizeInternal
{
public:
	static FIBITMAP* ConvertToNormalMap(FIBITMAP* Bump, FREE_IMAGE_FORMAT FifW);
	static FIBITMAP* PyramidProcess(FIBITMAP* Grey, int32 Level = 1);
	static void GetBitmapPixelInfo(FIBITMAP* Bitmap, int32& BitsPerPixel, int32& ChannelCount);
};

bool FDatasmithTextureResize::IsSupportedTextureExtension(const FString& Extension)
{
	return Extension == TEXT(".bmp") ||
			Extension == TEXT(".float") ||
			Extension == TEXT(".pcx") ||
			Extension == TEXT(".tga") ||
			Extension == TEXT(".jpg") ||
			Extension == TEXT(".jpeg") ||
			Extension == TEXT(".exr") ||
			Extension == TEXT(".dds") ||
			Extension == TEXT(".hdr");
}

void FDatasmithTextureResize::Initialize()
{
	FFreeImageWrapper::FreeImage_Initialise();
}

bool FDatasmithTextureResize::GetBestTextureExtension(const TCHAR* Source, FString& Extension)
{
	Extension = FPaths::GetExtension(Source, true).ToLower();
	if (IsSupportedTextureExtension(Extension))
	{
		return true;
	}

	Extension.Reset();

	Initialize();

	//image format
	FREE_IMAGE_FORMAT FileType = FIF_UNKNOWN;

	//check the file signature and deduce its format
	FileType = FreeImage_GetFileType(TCHAR_TO_FICHAR(Source), 0);

	//if still unknown, try to guess the file format from the file extension
	if (FileType == FIF_UNKNOWN)
	{
		FileType = FreeImage_GetFIFFromFilename(TCHAR_TO_FICHAR(Source));
	}

	//if still unknown, return failure
	if (FileType == FIF_UNKNOWN)
	{
		return false;
	}

	//check that the plugin has reading capabilities and load the file
	if (!FreeImage_FIFSupportsReading(FileType))
	{
		return false;
	}

	//pointer to the image, once loaded
	FIBITMAP* Bitmap = FreeImage_Load(FileType, TCHAR_TO_FICHAR(Source), 0);

	//if the image failed to load, return failure
	if (!Bitmap)
	{
		return false;
	}

	int32 BitsPerPixel = 0;
	int32 ChannelCount = -1;
	DatasmithTextureResizeInternal::GetBitmapPixelInfo(Bitmap, BitsPerPixel, ChannelCount);

	// No valid extension if channel count cannot be determine
	if (ChannelCount < 1)
	{
	}
	// RGB16 565 mode or 555 mode
	else if(BitsPerPixel % ChannelCount != 0)
	{
		Extension = TEXT(".tga");
	}
	else
	{
		// Set best extension to png if image has 3 or less channels per pixels
		Extension = ChannelCount < 4 ? TEXT(".png") : TEXT(".tga");
	}

	FreeImage_Unload(Bitmap);

	return true;
}

EDSTextureUtilsError FDatasmithTextureResize::ResizeTexture(const TCHAR* Source, const TCHAR* Destination, EDSResizeTextureMode Mode, uint32 MaxSize, bool bGenerateNormalMap)
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	if (!PlatformFile.FileExists(Source) || FString(Source).Len() < 3)
	{
		return EDSTextureUtilsError::FileNotFound;
	}

	Initialize();

	if ( !FFreeImageWrapper::IsValid() )
	{
		PlatformFile.CopyFile(Destination, Source, EPlatformFileRead::AllowWrite, EPlatformFileWrite::AllowRead);
		return EDSTextureUtilsError::FreeImageNotFound;
	}

	//image format
	FREE_IMAGE_FORMAT FileType = FIF_UNKNOWN;

	//check the file signature and deduce its format
	FileType = FreeImage_GetFileType(TCHAR_TO_FICHAR(Source), 0);

	//if still unknown, try to guess the file format from the file extension
	if (FileType == FIF_UNKNOWN)
	{
		FileType = FreeImage_GetFIFFromFilename(TCHAR_TO_FICHAR(Source));
	}

	//if still unknown, return failure
	if (FileType == FIF_UNKNOWN)
	{
		return EDSTextureUtilsError::InvalidFileType;
	}

	//check that the plugin has reading capabilities and load the file
	if (!FreeImage_FIFSupportsReading(FileType))
	{
		return EDSTextureUtilsError::InvalidFileType;
	}

	//pointer to the image, once loaded
	FIBITMAP* Bitmap = FreeImage_Load(FileType, TCHAR_TO_FICHAR(Source), 0);

	//if the image failed to load, return failure
	if (!Bitmap)
	{
		return EDSTextureUtilsError::FileReadIssue;
	}

	//EDSTextureUtilsError ErrorCode = ::ResizeTexture(Bitmap, FileType, Destination, Mode, MaxSize, bGenerateNormalMap);

	//retrieve the image data
	BYTE* Bits = FreeImage_GetBits(Bitmap);
	if (Bits == nullptr)
	{
		return EDSTextureUtilsError::InvalidData;
	}

	//get the image Width and Height
	const uint32 OriginalWidth = FreeImage_GetWidth(Bitmap);
	const uint32 OriginalHeight = FreeImage_GetHeight(Bitmap);

	//if this somehow one of these failed (they shouldn't), return failure
	if ((Bits == 0) || (OriginalWidth == 0) || (OriginalHeight == 0))
	{
		return EDSTextureUtilsError::InvalidData;
	}

	FREE_IMAGE_FORMAT FifW = FreeImage_GetFIFFromFilename(TCHAR_TO_FICHAR(Destination));

	if (bGenerateNormalMap)
	{
		Bitmap = DatasmithTextureResizeInternal::ConvertToNormalMap(Bitmap, FifW);
		if (!Bitmap)
		{
			return EDSTextureUtilsError::InvalidData;
		}
	}

	uint32 NewWidth = OriginalWidth;
	uint32 NewHeight = OriginalHeight;

	if (OriginalHeight > MaxSize && OriginalHeight > OriginalWidth)
	{
		NewWidth = FMath::RoundToInt((float)MaxSize * (float)OriginalWidth / (float)OriginalHeight);
		NewHeight = MaxSize;
	}
	else if (OriginalWidth > MaxSize && OriginalWidth >= OriginalHeight)
	{
		NewHeight = FMath::RoundToInt((float)MaxSize * (float)OriginalHeight / (float)OriginalWidth);
		NewWidth = MaxSize;
	}

	FIBITMAP* RescaledImage = nullptr;
	if (Mode != EDSResizeTextureMode::NoResize)
	{
		NewWidth = ToPowerOfTwo(NewWidth, Mode);
		NewHeight = ToPowerOfTwo(NewHeight, Mode);

		// Skip resize if not necessary
		RescaledImage = (NewHeight != OriginalHeight || NewWidth != OriginalWidth) ? FreeImage_Rescale(Bitmap, NewWidth, NewHeight, FREE_IMAGE_FILTER::FILTER_LANCZOS3) : Bitmap;
	}
	else
	{
		RescaledImage = Bitmap;
	}

	if (FifW == FileType && bGenerateNormalMap == false && OriginalHeight == NewHeight && OriginalWidth == NewWidth)
	{
		PlatformFile.CopyFile(Destination, Source, EPlatformFileRead::AllowWrite, EPlatformFileWrite::AllowRead);

		FreeImage_Unload(Bitmap);

		return EDSTextureUtilsError::NoError;
	}

	BOOL bSuccessfullySaved = FALSE;

	if (FifW == FIF_EXR || FifW == FIF_HDR)
	{
		//FIBITMAP* InFloat = FreeImage_ConvertToRGBAF(RescaledImage);
		bSuccessfullySaved = FreeImage_Save(FifW, RescaledImage, TCHAR_TO_FICHAR(Destination), 0);
		//FreeImage_Unload(InFloat);
	}
	else if (bGenerateNormalMap == true)
	{
		FIBITMAP* InFloat = FreeImage_ConvertTo24Bits(RescaledImage);

		bSuccessfullySaved = FreeImage_Save(FifW, InFloat, TCHAR_TO_FICHAR(Destination), 0);
		FreeImage_Unload(InFloat);
	}
	else
	{
		// Most export format supports FIT_BITMAP image type, try it
		if (FreeImage_GetImageType(RescaledImage) == FIT_BITMAP)
		{
			bSuccessfullySaved = FreeImage_Save(FifW, RescaledImage, TCHAR_TO_FICHAR(Destination), 0);
		}

		if (!bSuccessfullySaved)
		{
			int32 BitsPerPixel = 0;
			int32 ChannelCount = -1;
			DatasmithTextureResizeInternal::GetBitmapPixelInfo(RescaledImage, BitsPerPixel, ChannelCount);

			// If saving in png format, try to keep as much info per pixel as possible
			if (FifW == FIF_PNG)
			{
				FIBITMAP* ConvertedBitmap = RescaledImage;
				if ((BitsPerPixel / ChannelCount) > 16)
				{
					ConvertedBitmap = FreeImage_ConvertToType(RescaledImage, FIT_UINT16, TRUE);
				}
				else if((BitsPerPixel / ChannelCount) < 16)
				{
					ConvertedBitmap = FreeImage_ConvertToType(RescaledImage, FIT_BITMAP, TRUE);
				}

				bSuccessfullySaved = FreeImage_Save(FifW, ConvertedBitmap, TCHAR_TO_FICHAR(Destination), 0);

				if (ConvertedBitmap != RescaledImage)
				{
					FreeImage_Unload(ConvertedBitmap);
				}
			}
			else
			{
				ChannelCount += (BitsPerPixel % ChannelCount) ? 1 : 0;
				FIBITMAP* In8BitPerChannel = ChannelCount > 3 ? FreeImage_ConvertTo32Bits(RescaledImage) : FreeImage_ConvertTo24Bits(RescaledImage);
				bSuccessfullySaved = FreeImage_Save(FifW, In8BitPerChannel, TCHAR_TO_FICHAR(Destination), 0);
				FreeImage_Unload(In8BitPerChannel);
			}
		}
	}

	// Free rescaled image data if applicable
	if (Mode != EDSResizeTextureMode::NoResize && RescaledImage != Bitmap)
	{
		FreeImage_Unload(RescaledImage);
	}

	//Free FreeImage's copy of the data
	FreeImage_Unload(Bitmap);

	return bSuccessfullySaved == TRUE ? EDSTextureUtilsError::NoError : EDSTextureUtilsError::FileNotSaved;
}

FIBITMAP* DatasmithTextureResizeInternal::PyramidProcess(FIBITMAP* Grey, int32 Level /*= 1*/)
{
	int32 Height = FreeImage_GetHeight(Grey);
	int32 Width = FreeImage_GetWidth(Grey);

	if (Height <= 8 || Width <= 8 || Level >3)
	{
		return nullptr;
	}

	FIBITMAP* GreyDown = FreeImage_Rescale(Grey, Width / 4, Height / 4, FREE_IMAGE_FILTER::FILTER_LANCZOS3);
	FIBITMAP* Previous = PyramidProcess(GreyDown, Level + 1);

	if (Previous == nullptr)
	{
		return Grey;
	}

	FIBITMAP* GreyUp = FreeImage_Rescale(Previous, Width, Height, FREE_IMAGE_FILTER::FILTER_BICUBIC);

	for (int32 y = 0; y < Height; y++)
	{
		float* Bits = (float*)FreeImage_GetScanLine(Grey, y);
		float* bitsUp = (float*)FreeImage_GetScanLine(GreyUp, y);

		for (int32 x = 0; x < Width; x++)
		{
			Bits[x] = (Bits[x] * 0.5f + bitsUp[x] * 0.5f) + bitsUp[x] * 1.5f;
		}
	}

	return Grey;
}

FIBITMAP* DatasmithTextureResizeInternal::ConvertToNormalMap(FIBITMAP* Bump, FREE_IMAGE_FORMAT FifW)
{
	int32 Height = FreeImage_GetHeight(Bump);
	int32 Width = FreeImage_GetWidth(Bump);
	FIBITMAP* Grey = FreeImage_ConvertToFloat(Bump);
	FIBITMAP* Normal = FreeImage_ConvertToRGBAF(Bump);
	FreeImage_Unload(Bump);

	/*for (int32 y = 0; y < Height; y++)
	{
		float *Bits = (float *)FFreeImageWrapper::FreeImage_GetScanLine(Grey, y);
		for (int32 x = 0; x < Width; x++)
			Bits[x] = (Bits[x] * Bits[x]);
	}*/

	FIBITMAP* GreyPyramid = PyramidProcess(Grey);

	if (!GreyPyramid)
	{
		return nullptr;
	}

	TArray<NinePoints> Values;

	Values.AddDefaulted(Height * Width);


	float MinVal = 99999;
	float MaxVal = -99999;
	for (int32 y = 0; y < Height; y++)
	{
		float* Bits = reinterpret_cast< float*>(FreeImage_GetScanLine(GreyPyramid, y));
		for (int32 x = 0; x < Width; x++)
		{   //memorize colors in a table
			Values[y*Width + x].bb = Bits[x];
			if (Values[y*Width + x].bb > MaxVal)
			{
				MaxVal = Values[y*Width + x].bb;
			}

			if (Values[y*Width + x].bb < MinVal)
			{
				MinVal = Values[y*Width + x].bb;
			}
		}
	}

	FreeImage_Unload(GreyPyramid);

	for (int32 y = 0; y < Height; y++)
	{
		for (int32 x = 0; x < Width; x++)    //equalize
		{
			Values[y*Width + x].bb = (Values[y*Width + x].bb - MinVal) / (MaxVal - MinVal);
		}
	}

	for (int32 y = 0; y < Height; y++)
	{
		for (int32 x = 0; x < Width; x++)
		{   //memorize colors in a table
			Values[y*Width + x].aa = Values[Tile(y - 1, Height)*Width + Tile(x - 1, Width)].bb;
			Values[y*Width + x].ba = Values[Tile(y - 1, Height)*Width +					 x].bb;
			Values[y*Width + x].ca = Values[Tile(y - 1, Height)*Width + Tile(x + 1, Width)].bb;
			Values[y*Width + x].ab = Values[				  y*Width + Tile(x - 1, Width)].bb;
			Values[y*Width + x].cb = Values[				  y*Width + Tile(x + 1, Width)].bb;
			Values[y*Width + x].ac = Values[Tile(y + 1, Height)*Width + Tile(x - 1, Width)].bb;
			Values[y*Width + x].bc = Values[Tile(y + 1, Height)*Width +					 x].bb;
			Values[y*Width + x].cc = Values[Tile(y + 1, Height)*Width + Tile(x + 1, Width)].bb;
		}
	}

	float Scale = 1.0f;
	float MaxValueN = 0.f;

	for (int32 y = 0; y < Height; y++)
	{
		FIRGBAF*Bits = (FIRGBAF*)FreeImage_GetScanLine(Normal, y);
		for (int32 x = 0; x < Width; x++)
		{   //memorize colors in a table
			int32 pos = y*Width + x;

			Bits[x].red = (Scale* -(Values[pos].cc - Values[pos].ac + 2.0f*(Values[pos].cb - Values[pos].ab) + Values[pos].ca - Values[pos].aa));
			if (FMath::Abs(Bits[x].red) > MaxValueN)
			{
				MaxValueN = FMath::Abs(Bits[x].red);
			}

			Bits[x].green = -(Scale* -(Values[pos].aa - Values[pos].ac + 2.0f*(Values[pos].ba - Values[pos].bc) + Values[pos].ca - Values[pos].cc));
			if (FMath::Abs(Bits[x].green) > MaxValueN)
			{
				MaxValueN = FMath::Abs(Bits[x].green);
			}

			Bits[x].blue = 1.0f;
			Bits[x].alpha = Values[y*Width + x].bb;
		}
	}

	MaxValueN = 1.0f / MaxValueN;
	for (int32 y = 0; y < Height; y++)
	{
		FIRGBAF* Bits = (FIRGBAF*)FreeImage_GetScanLine(Normal, y);

		for (int32 x = 0; x < Width; x++)
		{	//memorize colors in a table
			Bits[x].red = Bits[x].red * MaxValueN;
			if (Bits[x].red > 0)
			{
				Bits[x].red = FMath::Sqrt(Bits[x].red);
			}

			if (Bits[x].red < 0)
			{
				Bits[x].red = -FMath::Sqrt(FMath::Abs(Bits[x].red));
			}

			Bits[x].green = Bits[x].green * MaxValueN;
			if (Bits[x].green > 0)
			{
				Bits[x].green = FMath::Sqrt(Bits[x].green);
			}

			if (Bits[x].green < 0)
			{
				Bits[x].green = -FMath::Sqrt(FMath::Abs(Bits[x].green));
			}

			Bits[x].blue = 0.5f + 0.5f * FMath::Sqrt(1 - (Bits[x].red)*(Bits[x].red) + (Bits[x].green)*(Bits[x].green));

			Bits[x].green = 0.5f + 0.5f * Bits[x].green;
			Bits[x].red = 0.5f + 0.5f * Bits[x].red;
		}
	}

	if (FifW == FIF_EXR || FifW == FIF_HDR)
	{
		return Normal;
	}

	FIBITMAP* Normal8 = FreeImage_Allocate(Width, Height, 4 * sizeof(DWORD), 0, 0, 0);

	for (int32 y = 0; y < Height; y++)
	{
		FIRGBAF* Bits = (FIRGBAF*)FreeImage_GetScanLine(Normal, y);
		for (int32 x = 0; x < Width; x++)
		{   //memorize colors in a table
			RGBQUAD Color;
			Color.rgbRed = (BYTE)(255.f * (FMath::Max(0.0f, FMath::Min(1.0f, Bits[x].red))));
			Color.rgbGreen = (BYTE)(255.f *(1.0 - (FMath::Max(0.0f, FMath::Min(1.0f, Bits[x].green)))));
			Color.rgbBlue = (BYTE)(255.f * (FMath::Max(0.0f, FMath::Min(1.0f, Bits[x].blue))));
			FreeImage_SetPixelColor(Normal8, x, y, &Color);
		}
	}
	FreeImage_Unload(Normal);

	return Normal8;
}

void DatasmithTextureResizeInternal::GetBitmapPixelInfo(FIBITMAP* Bitmap, int32& BitsPerPixel, int32& ChannelCount)
{
	BitsPerPixel = FreeImage_GetBPP(Bitmap);
	ChannelCount = -1;
	switch (FreeImage_GetImageType(Bitmap))
	{
	case FIT_RGB16:
	case FIT_RGBF:
	{
		ChannelCount = 3;
		break;
	}
	case FIT_RGBAF:
	case FIT_RGBA16:
	{
		ChannelCount = 4;
		break;
	}
	case FIT_UINT16:
	case FIT_INT16:
	{
		ChannelCount = BitsPerPixel / 16;
		break;
	}
	case FIT_UINT32:
	case FIT_INT32:
	case FIT_FLOAT:
	{
		ChannelCount = BitsPerPixel / 32;
		break;
	}
	case FIT_DOUBLE:
	{
		ChannelCount = BitsPerPixel / 64;
		break;
	}
	case FIT_COMPLEX:
	{
		ChannelCount = BitsPerPixel / 128;
		break;
	}
	case FIT_BITMAP:
	{
		if(BitsPerPixel == 16)
		{
			// Make sure we are in RGB16 565 mode or RGB16 555 mode
			// That should be the only cases for a FIF_BITMAP
			unsigned RedMask = FreeImage_GetRedMask(Bitmap);
			unsigned GreenMask = FreeImage_GetGreenMask(Bitmap);
			unsigned BlueMask = FreeImage_GetBlueMask(Bitmap);
			if (  (RedMask == FI16_565_RED_MASK && GreenMask == FI16_565_GREEN_MASK && BlueMask == FI16_565_BLUE_MASK)
				|| (RedMask == FI16_555_RED_MASK && GreenMask == FI16_555_GREEN_MASK && BlueMask == FI16_555_BLUE_MASK))
			{
				ChannelCount = 3;
			}
		}
		else
		{
			ChannelCount = BitsPerPixel == 32 ? 4 : (BitsPerPixel == 24 ? 3 : 1);
		}
		break;
	}
	case FIT_UNKNOWN:
	default:
	{
		break;
	}
	}
}

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif

#endif // WITH_FREEIMAGE_LIB
