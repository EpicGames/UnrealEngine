// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorFactories/ActorFactory.h"

#include "ActorFactoryDatasmithScene.generated.h"

struct FActorSpawnParameters;

UCLASS(MinimalAPI)
class UActorFactoryDatasmithScene : public UActorFactory
{
	GENERATED_UCLASS_BODY()

public:
	static void SpawnRelatedActors( class ADatasmithSceneActor* DatasmithSceneActor, bool bReimportDeletedActors );

	//~ Begin UActorFactory Interface
	virtual bool CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg ) override;

protected:
	virtual AActor* SpawnActor( UObject* Asset, ULevel* InLevel, const FTransform& Transform, const FActorSpawnParameters& InSpawnParams) override;
	//~ End UActorFactory Interface
};