// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithImportOptions.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"

class AActor;
class AStaticMeshActor;
class FDatasmithActorUniqueLabelProvider;
class IDatasmithActorElement;
class IDatasmithCameraActorElement;
class UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.") IDatasmithClothActorElement;
class IDatasmithCustomActorElement;
class IDatasmithDecalActorElement;
class IDatasmithEnvironmentElement;
class IDatasmithHierarchicalInstancedStaticMeshActorElement;
class IDatasmithLandscapeElement;
class IDatasmithLightActorElement;
class IDatasmithMaterialIDElement;
class IDatasmithMeshActorElement;
class ILayers;
class UClass;
class UDatasmithSceneComponentTemplate;
class UDatasmithStaticMeshComponentTemplate;
class UHierarchicalInstancedStaticMeshComponent;
class USceneComponent;
class UStaticMeshComponent;
struct FCachedActorLabels;
struct FDatasmithImportContext;

class FDatasmithActorImporter
{
public:
	/**
	 * Imports an actor of class ActorClass based on ActorElement
	 *
	 * @param ActorClass		The class of the actor to import
	 * @param ActorElement		The Datasmith element containing the data for the actor
	 * @param ImportContext		The context in which we're importing
	 * @param ImportActorPolicy	How to handle existing and deleted actors
	 * @param PostSpawnFunc		Function to invoke right after spawning a new actor, before any additional setup is performed
	 *
	 * @return	Returns the imported actor. The root component is not registered to allow edits. The caller needs to register it when the setup is complete.
	 */
	static AActor* ImportActor( UClass* ActorClass, const TSharedRef< IDatasmithActorElement >& ActorElement, FDatasmithImportContext& ImportContext, EDatasmithImportActorPolicy ImportActorPolicy,
		TFunction< void( AActor* ) > PostSpawnFunc = TFunction< void( AActor* ) >() );

	/**
	 * Imports a scene component of class ComponentClass based on ActorElement
	 * Will add the component as an instance component of Outer if Outer is an Actor
	 *
	 * @param ComponentClass	The class of the scene component to import. Needs to be a USceneComponent or a child of USceneComponent
	 * @param ActorElement		The Datasmith element containing the data for the component
	 * @param ImportContext		The context in which we're importing
	 * @param Outer				The outer that will contain the component
	 *
	 * @return	Returns the importer scene component. The component is not registered to allow edits. The caller needs to register it when the setup is complete.
	 */
	static USceneComponent* ImportSceneComponent( UClass* ComponentClass, const TSharedRef< IDatasmithActorElement >& ActorElement, FDatasmithImportContext& ImportContext, UObject* Outer, FDatasmithActorUniqueLabelProvider& UniqueNameProvider );

	/**
	 * Spawns an actor with a SceneComponent
	 */
	static AActor* ImportBaseActor( FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithActorElement >& ActorElement );
	static USceneComponent* ImportBaseActorAsComponent( FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithActorElement >& ActorElement, UObject* Outer, FDatasmithActorUniqueLabelProvider& UniqueNameProvider );

	/**
	 * Spawns a static mesh actor
	 */
	static AStaticMeshActor* ImportStaticMeshActor( FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithMeshActorElement >& MeshActorElement );
	static UStaticMeshComponent* ImportStaticMeshComponent( FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithMeshActorElement >& MeshActorElement, UObject* Outer, FDatasmithActorUniqueLabelProvider& UniqueNameProvider );

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	static AActor* ImportClothActor( FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithClothActorElement >& ClothActorElement );
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
	 * Spawns a cine camera actor
	 */
	static AActor* ImportCameraActor( FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithCameraActorElement >& CameraActorElement );

	/**
	 * Spawns a light actor. The light class depends on the LightActorElement type.
	 */
	static AActor* ImportLightActor( FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithLightActorElement >& LightActorElement );

	/**
	 * Spawns a cine camera actor
	 */
	static AActor* ImportEnvironment( FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithEnvironmentElement >& EnvironmentElement );

	/**
	 * Spawns an actor with a class defined by the CustomActorElement class name
	 */
	static AActor* ImportCustomActor( FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithCustomActorElement >& CustomActorElement, FDatasmithActorUniqueLabelProvider& UniqueNameProvider );

	/**
	 * Spawns a Decal actor
	 */
	static AActor* ImportDecalActor( FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithDecalActorElement >& DecalActorElement, FDatasmithActorUniqueLabelProvider& UniqueNameProvider );

	/**
	 * Spawns a landscape actor
	 */
	static AActor* ImportLandscapeActor( FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithLandscapeElement >& LandscapeActorElement );

	/**
	 * Spawns an actor with a hierarchical instanced static mesh component
	 */
	static AActor* ImportHierarchicalInstancedStaticMeshAsActor( FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithHierarchicalInstancedStaticMeshActorElement >& HierarchicalInstancedStatictMeshActorElement, FDatasmithActorUniqueLabelProvider& UniqueNameProvider );

	static UHierarchicalInstancedStaticMeshComponent* ImportHierarchicalInstancedStaticMeshComponent( FDatasmithImportContext& ImportContext,
		const TSharedRef< IDatasmithHierarchicalInstancedStaticMeshActorElement >& HierarchicalInstancedStaticMeshActorElement, UObject* Outer, FDatasmithActorUniqueLabelProvider& UniqueNameProvider );

	/**
	 * Handle Actor common properties (Layers, Tags)
	 *
	 * @param ImportedActor	    The actual actor being imported
	 * @param ActorElement	    The Datasmith element containing the data for the component
	 * @param ImportContext	    The context in which we're importing
	 */
	static void SetupActorProperties(AActor* ImportedActor, const TSharedRef< IDatasmithActorElement >& ActorElement, FDatasmithImportContext& ImportContext);
	static void SetupSceneComponent( USceneComponent* SceneComponent, const TSharedRef< IDatasmithActorElement >& ActorElement, USceneComponent* Parent );

	/**
	 * Sets the properties of a static mesh component based on the MeshActorElement values
	 */
	static void SetupStaticMeshComponent( FDatasmithImportContext& ImportContext, UStaticMeshComponent* StaticMeshComponent, const TSharedRef< IDatasmithMeshActorElement >& MeshActorElement );

	/**
	 * Sets the properties of a hierarchical instanced static mesh component based on the HierarchicalInstancedStatictMeshActorElement values
	 */
	static void SetupHierarchicalInstancedStaticMeshComponent( FDatasmithImportContext& ImportContext, UHierarchicalInstancedStaticMeshComponent* HierarchicalInstancedStaticMeshComponent,
		const TSharedRef< IDatasmithHierarchicalInstancedStaticMeshActorElement >& HierarchicalInstancedStatictMeshActorElement );

protected:

	/**
	 * Parse Layer names to a Set of FNames
	 *
	 * @param CsvLayerNames		Comma separated string containing the list of names of the layers the actor pertains to.
	 * @return Returns the Set of valid Layer names
	 */
	static TSet<FName> ParseCsvLayers(const TCHAR* CsvLayersNames);

	/**
	 * Sets the materials on a static mesh actor to override the ones from the static mesh
	 */
	static void OverrideStaticMeshActorMaterials( const FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithMeshActorElement >& MeshActorElement, const UStaticMesh* StaticMesh, UDatasmithStaticMeshComponentTemplate* StaticMeshComponentTemplate );
	static void OverrideStaticMeshActorMaterial( const FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithMaterialIDElement >& SubMaterial, UDatasmithStaticMeshComponentTemplate* StaticMeshComponentTemplate, int32 MeshSubMaterialIdx );
};
