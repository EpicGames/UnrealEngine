// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithImportContext.h"
#include "DatasmithTranslatableSource.h"
#include "ObjectElements/DatasmithUSceneElement.h"

#include "Kismet/BlueprintFunctionLibrary.h"

#include "DatasmithBlueprintLibrary.generated.h"

class UDatasmithImportOptions;
class UDatasmithSceneElement;
class UStaticMesh;
class UStaticMeshComponent;
struct FDatasmithImportBaseOptions;
struct FDatasmithTranslatableSceneSource;


USTRUCT(BlueprintType)
struct DATASMITHIMPORTER_API FDatasmithImportFactoryCreateFileResult
{
	GENERATED_BODY()
	FDatasmithImportFactoryCreateFileResult();

	void FillFromImportContext(const FDatasmithImportContext& ImportContext);

	/** List of all created actors if user requires to use multiple actors. The root actor will be the first element of the array. */
	UPROPERTY(BlueprintReadWrite, Transient, Category = Imported)
	TArray<TObjectPtr<class AActor>> ImportedActors;

	/** Meshes created during the import process */
	UPROPERTY(BlueprintReadWrite, Transient, Category = Imported)
	TArray<TObjectPtr<class UObject>> ImportedMeshes;

	UPROPERTY(BlueprintReadWrite, Transient, Category = Result)
	bool bImportSucceed;

	UPROPERTY(BlueprintReadWrite, Transient, Category = Result)
	TObjectPtr<UDatasmithScene> Scene;
};


UCLASS()
class DATASMITHIMPORTER_API UDatasmithSceneElement : public UDatasmithSceneElementBase
{
	GENERATED_BODY()

public:
	/**
	 * Open an existing UDatasmith file from disk.
	 * @param	FilePath UDatasmith file path to open. ie: c:/MyFolder/MyFiles.udatasmith
	 * @return	The opened DatasmithScene, that can be modified and can be imported.
	 **/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Datasmith")
	static UDatasmithSceneElement* ConstructDatasmithSceneFromFile(const FString& FilePath);

	/**
	 * Open an existing Datasmith source from the SourceUri.
	 * @param	SourceUri Uri of the datasmith scene to open. ie: file://c:/MyFolder/MyFiles.udatasmith
	 * @return	The opened DatasmithScene, that can be modified and can be imported.
	 **/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Datasmith")
	static UDatasmithSceneElement* ConstructDatasmithSceneFromSourceUri(const FString& SourceUri);

	/**
	 * Open set of CAD files as actors in a single datasmith scene
	 * Importing set of files into single DatasmithScene asset(with ImportScene) is supported only for CAD files
	 * @return	The opened DatasmithScene, that can be modified and can be imported.
	 **/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Datasmith")
	static UDatasmithSceneElement* ConstructDatasmithSceneFromCADFiles(const TArray<FString>& FilePaths);

	/**
	 * Open an existing DatasmithScene asset file from disk.
	 * @param   AssetPath Path of the DFatasmithScene asset (eg. "/Game/MyAsset")
	 * @return  The opened DatasmithScene, that can be modified and can be reimported.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Datasmith")
	static UDatasmithSceneElement* GetExistingDatasmithScene(const FString& AssetPath);

	/**
	 * Trigger the translation phase, which populates the DatasmithScene.
	 * note that options should have been set before that and any modification to the scene made prior will be lost.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Datasmith")
	bool TranslateScene();

	/**
	 * Import a Datasmith Scene created with ConstructDatasmithSceneFromFile.
	 * @param	DestinationFolder	Destination of where you want the asset to be imported. ie: /Game/MyFolder1
	 * @return	A structure that contains the created actor or the blueprint actor depending of the options specified at the import.
	 **/
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	FDatasmithImportFactoryCreateFileResult ImportScene(const FString& DestinationFolder);

	/**
	 * Import a Datasmith Scene created with ConstructDatasmithSceneFromFiles into an array of scenes.
	 * @param	DestinationFolder	Destination of where you want the asset to be imported. ie: /Game/MyFolder1
	 * @return	An array of import results corresponding to array of input files
	 **/
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	TArray<FDatasmithImportFactoryCreateFileResult> ImportScenes(const FString& DestinationFolder);

	/**
	 * Reimport a scene opened with GetExistingDatasmithScene
	 * @return	A structure that contains the created actor or the blueprint actor depending of the options specified at the import.
	 */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	FDatasmithImportFactoryCreateFileResult ReimportScene();

	/**
	 * Fetch the typed options for this scene
	 * @param OptionType	Type of the option structure to get
	 * @return The option structure found
	 */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene", meta = (DeterminesOutputType = "OptionType"))
	UObject* GetOptions(UClass* OptionType=nullptr);

	/**
	 * List all the options structures applicable on this scene
	 * @return A map of options type to option object
	 */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	TMap<UClass*, UObject*> GetAllOptions();

	UE_DEPRECATED(4.23, "GetImportOptions is deprecated, use GetOptions instead.")
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene", meta=(DeprecatedFunction, DeprecationMessage="GetImportOptions is deprecated, use GetOptions instead."))
	UDatasmithImportOptions* GetImportOptions();

	/**
	 * Destroy reference to the udatasmith file. The Scene will no longer be available.
	 * DestroyScene is called automatically after ImportScene.
	 */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	void DestroyScene();

private:
	TSharedPtr<UE::DatasmithImporter::FExternalSource> ExternalSourcePtr;
	TUniquePtr<FDatasmithImportContext> ImportContextPtr;
	bool bTranslated = false;

	bool bMultifile = false;
	TArray<FString> FilePaths;
};


UCLASS(MinimalAPI, meta=(ScriptName="DatasmithStaticMeshLibrary"))
class UDatasmithStaticMeshBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * Sets the proper lightmap resolution to get the desired lightmap density ratio
	 *
	 * @param	Objects					List of static meshes and static mesh actors to update.
	 * @param	bApplyChanges			Indicates if changes must be apply or not.
	 * @param	IdealRatio				The desired lightmap density ratio
	 */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Static Mesh")
	static void ComputeLightmapResolution( const TArray< UObject* >& Objects, bool bApplyChanges, float IdealRatio = 0.2f );

	/**
	 * Setup the Lightmap UVs settings to enable or disable the lightmap generation on the static meshes found in the Assets list
	 *
	 * @param	Assets							List of objects to set the generate lightmap uvs flag on. Only Static Meshes and Static Mesh Components will be affected.
	 * @param	bApplyChanges					Indicates if changes must be apply or not.
	 * @param	bGenerateLightmapUVs			The value to set for the generate lightmap uvs flag.
	 * @param	LightmapResolutionIdealRatio	The desired lightmap density ratio
	 */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Static Mesh")
	static void SetupStaticLighting(const TArray< UObject* >& Objects, bool bApplyChanges, bool bGenerateLightmapUVs, float LightmapResolutionIdealRatio = 0.2f );

private:
	static void ComputeLightmapResolution(const TMap< UStaticMesh*, TSet< UStaticMeshComponent* > >& StaticMeshMap, bool bApplyChanges, float IdealRatio = 0.2f);

	static int32 ComputeLightmapResolution(UStaticMesh* StaticMesh, float IdealRatio, const FVector& StaticMeshScale);
};

