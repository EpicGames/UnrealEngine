// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"

class AActor;
struct FDatasmithImportContext;
class IDatasmithCameraActorElement;
class UCineCameraComponent;
class FDatasmithActorUniqueLabelProvider;

class FDatasmithCameraImporter
{
public:
	static AActor* ImportCameraActor( const TSharedRef< IDatasmithCameraActorElement >& CameraElement, FDatasmithImportContext& ImportContext );
	static void PostImportCameraActor( const TSharedRef< IDatasmithCameraActorElement >& CameraElement, FDatasmithImportContext& ImportContext );

	static UCineCameraComponent* ImportCineCameraComponent( const TSharedRef< IDatasmithCameraActorElement >& CameraElement, FDatasmithImportContext& ImportContext, UObject* Outer, FDatasmithActorUniqueLabelProvider& UniqueNameProvider );

protected:
	static void SetupCineCameraComponent( UCineCameraComponent* CineCameraComponent, const TSharedRef< IDatasmithCameraActorElement >& CameraElement );

};

