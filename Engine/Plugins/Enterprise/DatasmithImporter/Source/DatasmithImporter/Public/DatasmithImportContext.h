// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "Logging/TokenizedMessage.h"
#include "MaterialShared.h"
#include "Misc/SecureHash.h"
#include "UObject/ObjectMacros.h"
#include "UObject/StrongObjectPtr.h"

#include "DatasmithUtils.h"
#include "DatasmithImportOptions.h"
#include "InterchangeManager.h"

class AActor;
class ADatasmithSceneActor;
struct FDatasmithImportContext;
class FJsonObject;
class FMessageLogModule;
class IDatasmithBaseMaterialElement;
class UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.") IDatasmithClothElement;
class IDatasmithElement;
class IDatasmithLevelSequenceElement;
class IDatasmithLevelVariantSetsElement;
class IDatasmithMeshElement;
class IDatasmithScene;
class IDatasmithTextureElement;
class ILayers;
class IMessageLogListing;
class UBlueprint;
class UDatasmithScene;
class ULevelSequence;
class ULevelVariantSets;
class UMaterialInterface;
class UMaterialFunction;
class UPackage;
class USceneComponent;
class UStaticMesh;
class UWorld;
class UTexture;

class SDatasmithImportOptions;
class UDatasmithImportOptions;
class IDatasmithTranslator;

namespace UE::DatasmithImporter
{
	class FExternalSource;
}

/**
 * Provides unique actor label.
 * @note: Use a cache of known labels and a LUT of indices to speedup that step. Replaces FActorLabelUtilities::SetActorLabelUnique.
 */
class DATASMITHIMPORTER_API FDatasmithActorUniqueLabelProvider : public FDatasmithUniqueNameProvider
{
public:
	explicit FDatasmithActorUniqueLabelProvider(UWorld* World=nullptr);
	void PopulateLabelFrom(UWorld* World);
};

/**
 * Helper class that handle Messaging service and formating.
 * The instance destructor will display all messages pushed during its lifetime.
 */
class DATASMITHIMPORTER_API FScopedLogger
{
public:
	FScopedLogger(FName LogTitle, const FText& LogLabel);
	~FScopedLogger();

	FScopedLogger(const FScopedLogger&) = delete;
	FScopedLogger& operator = (const FScopedLogger&) = delete;
	FScopedLogger(FScopedLogger&&) = delete;
	FScopedLogger& operator = (FScopedLogger&&) = delete;

	/** Append a message */
	TSharedRef<FTokenizedMessage> Push(EMessageSeverity::Type Severity, const FText& Message);

	/** Display pending messages, show the log window */
	void Dump(bool bClearPrevious = false);

	/** Clear log window */
	void ClearLog();

private:
	void ClearPending();

private:
	FName Title;
	FMessageLogModule& MessageLogModule;
	TSharedPtr<IMessageLogListing> LogListing;
	TArray<TSharedRef<FTokenizedMessage>> TokenizedMessages;
	FCriticalSection TokenizedMessageCS;
};

struct FDatasmithActorImportContext
{
	explicit FDatasmithActorImportContext(UWorld* World = nullptr);

	bool Init();

	FDatasmithActorUniqueLabelProvider UniqueNameProvider;

	/** The Scene actor on which the actors will be imported */
	TObjectPtr<ADatasmithSceneActor> ImportSceneActor;

	/** The targeted scene actors for the finalize */
	TSet< ADatasmithSceneActor* > FinalSceneActors;

	/** The targeted scene that the importer is currently finalizing */
	ADatasmithSceneActor* CurrentTargetedScene;

	TObjectPtr<UWorld> ImportWorld;
	TObjectPtr<UWorld> FinalWorld;

	/** List of the datasmith actor element id that aren't imported because of the imports settings */
	TSet< FName > NonImportedDatasmithActors;
};

struct DATASMITHIMPORTER_API FDatasmithAssetsImportContext
{
	FDatasmithAssetsImportContext( FDatasmithImportContext& ImportContext );

	/**
	 * Inits the context by setting the paths and creating the packages to be ready to import assets.
	 */
	bool Init();

	/**
	* Updates all package paths based on a new root folder path e.g. "/Game/NewScene"
	*/
	void ReInit(const FString& NewRootFolder);

	/** The parent context that this context is a part of */
	FDatasmithImportContext* ParentContext;
	const FDatasmithImportContext& GetParentContext() const { return *ParentContext;  }
	FDatasmithImportContext& GetParentContext() { return *ParentContext; }


	/** Root folder path, used to derive the other paths */
	FString RootFolderPath;

	/** Package where new static meshes will be stored */
	TStrongObjectPtr< UPackage > StaticMeshesFinalPackage;

	/** Package where new materials will be stored */
	TStrongObjectPtr< UPackage > MaterialsFinalPackage;

	/** Package where new textures will be stored */
	TStrongObjectPtr< UPackage > TexturesFinalPackage;

	/** Package where new lights will be stored */
	TStrongObjectPtr< UPackage > LightPackage;

	/** Package where new level sequences will be stored */
	TStrongObjectPtr< UPackage > LevelSequencesFinalPackage;

	/** Package where new level sequences will be stored */
	TStrongObjectPtr< UPackage > LevelVariantSetsFinalPackage;

	/** Transient folder path, used to derive the other transient paths */
	FString TransientFolderPath;

	/** Package where the static meshes are stored until they are finalized */
	TStrongObjectPtr< UPackage > StaticMeshesImportPackage;

	/** Package where the textures are stored until they are finalized */
	TStrongObjectPtr< UPackage > TexturesImportPackage;

	/** Package where the materials are stored until they are finalized */
	TStrongObjectPtr< UPackage > MaterialsImportPackage;

	/** Package where the reference/parent materials are stored until they are finalized */
	TStrongObjectPtr< UPackage > ReferenceMaterialsImportPackage;

	/** Package where the material functions are stored until they are finalized */
	TStrongObjectPtr< UPackage > MaterialFunctionsImportPackage;

	/** Package where the level sequences are stored until they are finalized */
	TStrongObjectPtr< UPackage > LevelSequencesImportPackage;

	/** Package where the level variant sets are stored until they are finalized */
	TStrongObjectPtr< UPackage > LevelVariantSetsImportPackage;

	/** Ensures uniqueness of the name of the materials being imported */
	FDatasmithUniqueNameProvider MaterialNameProvider;

	/** Ensures uniqueness of the name of the materials being imported in the reference material package */
	FDatasmithUniqueNameProvider MaterialInstanceNameProvider;

	/** Ensures uniqueness of the name of the material functions being imported in the material function material package */
	FDatasmithUniqueNameProvider MaterialFunctionNameProvider;

	/** Ensures uniqueness of the name of the static meshes being imported */
	FDatasmithUniqueNameProvider StaticMeshNameProvider;

	/** Ensures uniqueness of the name of the textures being imported */
	FDatasmithUniqueNameProvider TextureNameProvider;

	/** Map of requirements on mesh building for each material */
	TMap<FString, int32> MaterialsRequirements;

	/** Set of virtual textures that needs to be converted back to non-virtual texture because not all feature support them yet. */
	TSet<UTexture2D*> VirtualTexturesToConvert;
};

struct DATASMITHIMPORTER_API FDatasmithImportContext
{
	UE_DEPRECATED(5.0, "Please use constructor using ExternalSource")
	FDatasmithImportContext(const FString& InFileName, bool bLoadConfig, const FName& LoggerName, const FText& LoggerLabel, TSharedPtr<IDatasmithTranslator> InSceneTranslator = nullptr);

	FDatasmithImportContext(const TSharedPtr<UE::DatasmithImporter::FExternalSource>& InExternalSource, bool bLoadConfig, const FName& LoggerName, const FText& LoggerLabel);

	/** Cached MD5 hash value for faster processing */
	FMD5Hash FileHash;

	/** Name of the scene being created */
	FString SceneName;

	/** Name of the scene being created */
	TSharedPtr<IDatasmithTranslator> SceneTranslator;

	/** Object flags to apply to newly imported objects */
	EObjectFlags ObjectFlags;

	/** Settings related to the import of a Datasmith scene */
	TObjectPtr<UDatasmithImportOptions> Options;

	/** Per-Translator settings related to the import of a Datasmith scene */
	TArray<TObjectPtr<UDatasmithOptionsBase>> AdditionalImportOptions;

	/** Stack of parent components we're currently importing under */
	TArray< USceneComponent* > Hierarchy;

	/** Scene asset */
	TObjectPtr<UDatasmithScene> SceneAsset;

	/** List of previously parsed IES files */
	TSet<FString> ParsedIesFiles;

	/** Indicates if the user has canceled the import process  */
	TAtomic<bool> bUserCancelled;

	bool bIsAReimport;
	bool bImportedViaScript;

	/** IDatasmithScene object requested to be imported to UE editor */
	TSharedPtr< IDatasmithScene > Scene;

	/** IDatasmithScene object filtered with only what will actually go through the import process*/
	TSharedPtr< IDatasmithScene > FilteredScene;

	/** Map of imported mesh for each mesh element */
	TMap< TSharedRef< IDatasmithMeshElement >, TObjectPtr<UStaticMesh> > ImportedStaticMeshes;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	TMap< TSharedRef< IDatasmithClothElement >, TObjectPtr<UObject> > ImportedClothes;

	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	TArray< UObject* > ImportedClothPresets;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Register IDatasmithMeshElement by their name so they can be searched faster */
	TMap< FString, TSharedRef < IDatasmithMeshElement > > ImportedStaticMeshesByName;

	/** Map of imported texture for each texture element */
	TMap< TSharedRef< IDatasmithTextureElement >, UE::Interchange::FAssetImportResultRef > ImportedTextures;

	/** Map of imported material for each material element */
	TMap< TSharedRef< IDatasmithBaseMaterialElement >, TObjectPtr<UMaterialInterface> > ImportedMaterials;

	/** Map of imported material function for each material element, they are only imported as a per-required basis */
	TMap< TSharedRef< IDatasmithBaseMaterialElement >, UMaterialFunction* > ImportedMaterialFunctions;

	/** Register IDatasmithMeshElement by their name so they can be searched faster */
	TMap< FString, TSharedRef < IDatasmithBaseMaterialElement > > ImportedMaterialFunctionsByName;

	/** List of potential parent materials with their hash. Used to create material instances from. */
	TMap< uint32, TObjectPtr<UMaterialInterface> > ImportedParentMaterials;

	/** Map of imported level sequences for each level sequence element */
	TMap< TSharedRef< IDatasmithLevelSequenceElement >, TObjectPtr<ULevelSequence> > ImportedLevelSequences;

	/** Map of imported level variant sets for each level variant set element */
	TMap< TSharedRef< IDatasmithLevelVariantSetsElement >, TObjectPtr<ULevelVariantSets> > ImportedLevelVariantSets;

	/** Feedback context provided by ImportFactory. Used to display import progress */
	FFeedbackContext* FeedbackContext;

	/** Actors specific import context */
	FDatasmithActorImportContext ActorsContext;

	/** Assets specific import context */
	FDatasmithAssetsImportContext AssetsContext;

	/** Generic extension */
	TSharedPtr< void > ContextExtension;

public:
	/**
	 * Initialize members required before loading anything
	 *
	 * @param Scene		The IDatasmithScene that we will be importing
	 * @param InParent	The package in which we are
	 * @param bSilent	Doesn't display the options dialog and skips other user input requests
	 */
	UE_DEPRECATED(5.0, "The options initialization should not require a scene. Please use other implemetantion of Init() and call InitScene() individually.")
	bool Init(TSharedRef< IDatasmithScene > InScene, const FString& InImportPath, EObjectFlags InFlags, FFeedbackContext* InWarn, const TSharedPtr<FJsonObject>& ImportSettingsJson, bool bSilent);

	/**
	 * Initialize members required before loading anything, must call InitScene() independendly when using this function.
	 *
	 * @param InImportPath			The path where the scene will be imported
	 * @param InFlags				Flags applied to all generated objects during the following import
	 * @param InWarn				Feedback context for the following import
	 * @param ImportSettingsJson	When bSilent, options as json
	 * @param bSilent				Doesn't display the options dialog and skips other user input requests
	 */
	bool Init(const FString& InImportPath, EObjectFlags InFlags, FFeedbackContext* InWarn, const TSharedPtr<FJsonObject>& ImportSettingsJson, bool bSilent);

	/**
	 * First part of the Init process, replaces Init, requires a call to SetupDestination and InitScene after that.
	 * Displays the options to the end-user (blocking call), and updates translator accordingly.
	 * When silent, Json options are parsed instead.
	 *
	 * @param InScene              Scene that the context will use for the translation and import
	 * @param ImportSettingsJson   When bSilent, options as json
	 * @param bSilent              Flag that prevents options to be displayed and edited
	 * @return false if the user canceled -> import should not occurs in that case.
	 */
	UE_DEPRECATED(5.0, "Please use other InitOptions() implementations, having a valid scene should not be a requirement for initializing the options.")
	bool InitOptions(TSharedRef< IDatasmithScene > InScene, const TSharedPtr<FJsonObject>& ImportSettingsJson, bool bSilent);

	/**
	 * First part of the Init process, requires a call to SetupDestination and InitScene after that.
	 * Displays the options to the end-user (blocking call), and updates translator accordingly.
	 * When silent, Json options are parsed instead.
	 *
	 * @param ImportSettingsJson   When bSilent, options as json
	 * @param InImportPath         Optional import path displayed when bSilent is false.
	 * @param bSilent              Flag that prevents options to be displayed and edited
	 * @return false if the user canceled -> import should not occurs in that case.
	 */
	bool InitOptions(const TSharedPtr<FJsonObject>& ImportSettingsJson, const TOptional<FString>& InImportPath, bool bSilent);

	/**
	 * Second part of the Init process, replaces Init, requires a call to InitOptions before that.
	 * Setup destination packages
	 *
	 * @param InImportPath   Destination package's name
	 * @param InFlags        Flags applied to all generated objects during the following import
	 * @param InWarn         Feedback context for the following import
	 * @param bSilent        When false, prompt the user to save dirty packages
	 * @return false if the user canceled -> import should not occurs in that case.
	 */
	bool SetupDestination(const FString& InImportPath, EObjectFlags InFlags, FFeedbackContext* InWarn, bool bSilent);

	/**
	 * Set the scene and initialize related members. Should be called before starting the import process.
	 *
	 * @param InScene	Scene that will be used for the import, it should already be translated/loaded.
	 */
	void InitScene(const TSharedRef<IDatasmithScene>& InScene);

	/**
	 * Replace or add options based on it's UClass.
	 */
	void UpdateImportOption(UDatasmithOptionsBase* Option);

	/** Push messages for display to the end-user */
	TSharedRef<FTokenizedMessage> LogError(const FText& InErrorMessage);
	TSharedRef<FTokenizedMessage> LogWarning(const FText& InWarningMessage, bool bPerformance = false);
	TSharedRef<FTokenizedMessage> LogInfo(const FText& InInfoMessage);

	bool ShouldImportActors() const;

	/** Add newly created Actor to context's map */
	void AddImportedActor(AActor* InActor);

	TArray<AActor*> GetImportedActors() const;

	/** Add newly created SceneComponent to context's map */
	void AddSceneComponent(const FString& InName, USceneComponent* InMeshComponent);

	/** Displays all the messages that were added through AddMessage in the MessageLogModule */
	void DisplayMessages();

private:
	void SetupBaseOptionsVisibility();
	void ResetBaseOptionsVisibility();

private:

	/** Handles messages logged during the import process */
	FScopedLogger Logger;

	/** Map of StaticMeshActor objects added to the World at import */
	TMap<FString, TObjectPtr<AActor>> ImportedActorMap;

	/** Map of SceneComponent objects added to the World at import */
	TMap<FString, TObjectPtr<USceneComponent>> ImportedSceneComponentMap;

	int32 CurrentSceneActorIndex;

private:
	/** GC UObject own by DatasmithImportContext */
	struct FInternalReferenceCollector : FGCObject
	{
		FInternalReferenceCollector(FDatasmithImportContext* InImportContext);

		//~ FGCObject interface
		virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		virtual FString GetReferencerName() const override
		{
			return TEXT("FDatasmithImportContext::FInternalReferenceCollector");
		}
	private:
		FDatasmithImportContext* ImportContext;
	};
	FInternalReferenceCollector ReferenceCollector;
	friend FInternalReferenceCollector;
};
