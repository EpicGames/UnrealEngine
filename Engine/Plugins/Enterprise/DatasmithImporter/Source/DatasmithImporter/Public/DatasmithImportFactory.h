// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithImporter.h"
#include "DatasmithImportOptions.h"

#include "EditorReimportHandler.h"
#include "Factories/ImportSettings.h"
#include "Factories/SceneImportFactory.h"
#include "Materials/MaterialInterface.h"

#include "DatasmithImportFactory.generated.h"

namespace UE::DatasmithImporter
{
	class FExternalSource;
}

namespace DatasmithImportFactoryImpl
{

	/**
	 * Imports the content described in the DatasmithScene object held by the incoming FDatasmithImportContext
	 * This is a helper to allow other import plugin to use the same mechanism to import content into UE
	 *
	 * @return true on successful import
	 */
	DATASMITHIMPORTER_API bool ImportDatasmithScene(FDatasmithImportContext& InContext, bool& bOutOperationCancelled);

} // ns DatasmithImportFactoryImpl


UCLASS()
class DATASMITHIMPORTER_API UDatasmithImportFactory : public USceneImportFactory, public IImportSettingsParser, public FReimportHandler
{
	GENERATED_BODY()

public:
	UDatasmithImportFactory();

	//~ Begin UFactory Interface
	virtual bool FactoryCanImport(const FString& Filename) override;
	virtual FText GetDisplayName() const override;
	/** Used on import path edition to select factories that can produce InClass type of asset. */
	virtual bool DoesSupportClass(UClass* InClass) override;
	virtual UClass* ResolveSupportedClass() override;
	virtual void CleanUp() override;
	virtual UObject* FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* InParms, FFeedbackContext* Warn, bool& bOutOperationCanceled) override;
	UObject* CreateFromExternalSource(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, const TSharedRef<UE::DatasmithImporter::FExternalSource>& InExternalSource, const TCHAR* InParms, FFeedbackContext* InWarn, bool& bOutOperationCanceled);
	virtual IImportSettingsParser* GetImportSettingsParser() override { return this; }
	//~ End UFactory Interface

	//~ Begin IImportSettings Interface
	virtual void ParseFromJson(TSharedRef<FJsonObject> InImportSettingsJson) override;
	//~ End IImportSettings Interface

	//~ Begin FReimportHandler Interface
	virtual bool CanReimport(UObject* Obj, TArray<FString>& OutFilenames) override;
	virtual void SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths) override;
	virtual EReimportResult::Type Reimport(UObject* Obj) override;
	virtual int32 GetPriority() const override;
	virtual TObjectPtr<UObject>* GetFactoryObject() const override
	{
		return &GCMark;
	}
	//~ End FReimportHandler Interface

private:
	/** Imports the Datasmith Scene **/
	bool Import(FDatasmithImportContext& ImportContext);

	/** Log Listing which holds import messages. Note: Displayed as GetDisplayName() in the UI. */
	FName GetLoggerName() const { return TEXT("DatasmithImport"); }

	/** @return true when the file extension is on the supported extension list. */
	bool IsExtensionSupported(const FString& Filename);

	/** filter filenames to keep files currently accessible */
	void ValidateFilesForReimport(TArray<FString>& Filenames);

	EReimportResult::Type ReimportMaterial(UMaterialInterface* Material);
	EReimportResult::Type ReimportScene(UDatasmithScene* SceneAsset);
	EReimportResult::Type ReimportStaticMesh(UStaticMesh* Mesh);

	/** Callback used to complete re-import of static mesh if it was edited */
	void OnObjectReimported(UObject* Object, UStaticMesh* StaticMesh);

private:
	TSharedPtr<FJsonObject> ImportSettingsJson;

	bool bShowOptions;

	/**
	 * Register that the user has canceled the import.
	 * Useful when multiple files have been selected.
	 */
	bool bOperationCanceled;

	mutable TObjectPtr<UObject> GCMark{this};
};
