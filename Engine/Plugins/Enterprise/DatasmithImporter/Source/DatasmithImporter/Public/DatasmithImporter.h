// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithContentEditorModule.h"
#include "DatasmithImportContext.h"

#include "Containers/ContainersFwd.h"
#include "Misc/Optional.h"

struct FDatasmithMaterialImporterContext;
struct FDatasmithStaticMeshImportOptions;
struct FDatasmithMeshElementPayload;
class FString;
class IDatasmithBaseMaterialElement;
class IDatasmithElement;
class IDatasmithLevelSequenceElement;
class IDatasmithLevelVariantSetsElement;
class IDatasmithMeshElement;
class IDatasmithScene;
class IDatasmithTextureElement;
class UDatasmithAdditionalData;
class ULevelSequence;
class ULevelVariantSets;
class UMaterialInterface;
class UObject;
class UPackage;
class UStaticMesh;
class UTexture;
class UTexture2D;
class UMaterial;
class UMaterialFunction;

struct DATASMITHIMPORTER_API FDatasmithImporter
{
	/**
	 * Imports the meshes associated with the mesh elements
	 */
	static void ImportStaticMeshes( FDatasmithImportContext& ImportContext );

	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	static void ImportClothes( FDatasmithImportContext& ImportContext );

	/**
	 * @return The imported Static Mesh
	 */
	static UStaticMesh* ImportStaticMesh( FDatasmithImportContext& ImportContext, TSharedRef< IDatasmithMeshElement > MeshElement, UStaticMesh* ExistingStaticMesh, FDatasmithMeshElementPayload* MeshPayload = nullptr);

	/**
	 * Copy a static mesh from its transient package into its final public package and builds it.
	 *
	 * @param SourceStaticMesh			The static mesh to finalize.
	 * @param StaticMeshesFolderPath	The path where to put the final static mesh, if we don't have an ExistingStaticMesh
	 * @param ExistingStaticMesh		An optional existing static mesh on which we will finalize over.
	 * @param ReferencesToRemap			A map of migrated objects. If SourceStaticMesh refers to any key objects, those references will be replaced by the corresponding map value.
	 * @param bBuild					Specify if the static mesh building should be performed.
	 *
	 * @return The resulting static mesh
	 */
	static UStaticMesh* FinalizeStaticMesh( UStaticMesh* SourceStaticMesh, const TCHAR* StaticMeshesFolderPath, UStaticMesh* ExistingStaticMesh, TMap< UObject*, UObject* >* ReferencesToRemap = nullptr, bool bBuild = true );

	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	static UObject* FinalizeCloth( UObject* SourceCloth, const TCHAR* FolderPath, UObject* ExistingCloth, TMap<UObject*, UObject*>* ReferencesToRemap = nullptr);

	/**
	 * Creates the AssetImportData associated with a UStaticMesh. Used when reimporting.
	 */
	static void CreateStaticMeshAssetImportData( FDatasmithImportContext& InContext, TSharedRef< IDatasmithMeshElement > MeshElement, UStaticMesh* ImportedStaticMesh, TArray<UDatasmithAdditionalData*>& AdditionalData );

	/**
	 * Imports the textures
	 */
	static void ImportTextures( FDatasmithImportContext& ImportContext );

	/**
	 * @return The imported texture
	 */
	static UTexture* ImportTexture( FDatasmithImportContext& ImportContext, class FDatasmithTextureImporter& DatasmithTextureImporter, TSharedRef< IDatasmithTextureElement > TextureElement, UTexture* ExistingTexture, const TArray<uint8>& TextureData, const FString& Extension );

	/**
	 * Copy a texture from its transient package into its final public package and builds it.
	 *
	 * @param SourceTexture			The texture to finalize.
	 * @param TexturesFolderPath	The path where to put the final texture, if we don't have an ExistingTexture
	 * @param ExistingTexture		An optional existing texture on which we will finalize over.
	 * @param ReferencesToRemap		A map of migrated objects. If SourceTexture refers to any key objects, those references will be replaced by the corresponding map value.
	 *
	 * @return The resulting texture
	 */
	static UTexture* FinalizeTexture( UTexture* SourceTexture, const TCHAR* TexturesFolderPath, UTexture* ExistingTexture, TMap< UObject*, UObject* >* ReferencesToRemap = nullptr );

	/**
	 * Imports the materials
	 */
	static void ImportMaterials( FDatasmithImportContext& ImportContext );

	/**
	 * @return The imported MaterialFunction
	 */
	static UMaterialFunction* ImportMaterialFunction( FDatasmithImportContext& ImportContext, TSharedRef< IDatasmithBaseMaterialElement > MaterialElement );

	/**
	 * Copy a material function from its transient package into its final public package.
	 *
	 * @param SourceMaterialFunction		The material function to finalize.
	 * @param MaterialFunctionFolderPath	The path where to put the final material function, if we don't have an ExistingMaterialFunction.
	 * @param ExistingMaterialFunction		An optional existing material function on which we will finalize over.
	 * @param ReferencesToRemap				A map of migrated objects. If SourceMaterialFunction refers to any key objects those references will be replaced by the corresponding map value.
	 *
	 * @return The resulting material function.
	 */
	static UMaterialFunction* FinalizeMaterialFunction(UObject* SourceMaterialFunction, const TCHAR* MaterialFunctionsFolderPath, UMaterialFunction* ExistingMaterialFunction, TMap< UObject*, UObject* >* ReferencesToRemap);

	/**
	 * @return The imported material
	 */
	static UMaterialInterface* ImportMaterial( FDatasmithImportContext& ImportContext, TSharedRef< IDatasmithBaseMaterialElement > MaterialElement, UMaterialInterface* ExistingMaterial );

	/**
	 * Copy a material from its transient package into its final public package and builds it.
	 * Also finalizes the parent material for instances when it's under the same path as the SourceMaterial.
	 *
	 * @param SourceMaterial		The material to finalize.
	 * @param MaterialFolderPath	The path where to put the final material, if we don't have an ExistingMaterial
	 * @param MaterialFolderPath	The transient path of the imported scene, used to finalize parent materials.
	 * @param MaterialFolderPath	The root folder where we want to import the scene, used to finalize parent materials.
	 * @param ExistingMaterial		An optional existing material on which we will finalize over.
	 * @param ReferencesToRemap		A map of migrated objects. If SourceMaterial refers to any key objects, those references will be replaced by the corresponding map value.
	 * @param MaterialUpdateContext		An optional material update context to batch updates and improve performance.
	 *
	 * @return The resulting material
	 */
	static UObject* FinalizeMaterial( UObject* SourceMaterial, const TCHAR* MaterialFolderPath, const TCHAR* TransientFolderPath, const TCHAR* RootFolderPath, UMaterialInterface* ExistingMaterial, TMap< UObject*, UObject* >* ReferencesToRemap = nullptr, FMaterialUpdateContext* MaterialUpdateContext = nullptr);

	/**
	 * Imports the actors
	 */
	static void ImportActors( FDatasmithImportContext& ImportContext );

	/**
	 * Imports the given actor element with its hierarchy
	 */
	static AActor* ImportActor( FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithActorElement >& ActorElement );

	/**
	 * Adds a new component to the root actor. The component class is determined by the ActorElement type.
	 *
	 * @param ImportContext		The context in which we're importing
	 * @param ActorElement		The Datasmith element containing the data for the component
	 * @param RootActor			The actor in which to add the component
	 */
	static void ImportActorAsComponent( FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithActorElement >& ActorElement, AActor* InRootActor, FDatasmithActorUniqueLabelProvider& UniqueNameProvider );

	/**
	 * Finalizes all the actors
	 */
	static void FinalizeActors( FDatasmithImportContext& ImportContext, TMap< UObject*, UObject* >* AssetReferencesToRemap );

	/**
	 * Migrates an actor from the import world to the final world. Generating templates based on their current values.
	 *
	 * @param ImportContext			The context used by the import process.
	 * @param SourceActor			The actor to finalize.
	 * @param ExistingActor			An optional existing actor on which we will finalize over.
	 * @param ReferencesToRemap		A map of migrated objects. If SourceActor refers to any key objects, those references will be replaced by the corresponding map value.
	 * @param ReusableBuffer		An array that will be used for the migration of the data from the temp actor to the final actor. The array doesn't have to be empty the function will clean it before using it and it will try to keep it's existing allocation.
	 *
	 * @return The resulting actor
	 */
	static AActor* FinalizeActor( FDatasmithImportContext& ImportContext, AActor& SourceActor, AActor* ExistingActor, TMap< UObject*, UObject* >& ReferencesToRemap, TArray<uint8>& ReusableBuffer );

	/**
	 * Imports the level sequences
	 */
	static void ImportLevelSequences( FDatasmithImportContext& ImportContext );

	/**
	 * Copy a level sequence from its transient package into its final public package and builds it.
	 */
	static ULevelSequence* FinalizeLevelSequence( ULevelSequence* SourceLevelSequence, const TCHAR* AnimationsFolderPath, ULevelSequence* ExistingLevelSequence );

	/**
	 * Imports the level variant sets
	 */
	static void ImportLevelVariantSets( FDatasmithImportContext& ImportContext );

	/**
	 * Copy a level sequence from its transient package into its final public package and builds it.
	 */
	static ULevelVariantSets* FinalizeLevelVariantSets( ULevelVariantSets* SourceLevelVariantSets, const TCHAR* VariantsFolderPath, ULevelVariantSets* ExistingLevelVariantSets );

	/**
	 * Imports the meta data associated to a DatasmithElement and sets it on the Object.
	 */
	static void ImportMetaDataForObject( FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithElement >& DatasmithElement, UObject* Object );

	/**
	 * Filter elements in the scene that needs to be imported
	 */
	static void FilterElementsToImport( FDatasmithImportContext& ImportContext );

	/**
	 * Publicize the import and fix the references on the assets and actors. Also, update the final datasmith scenes.
	 */
	static void FinalizeImport( FDatasmithImportContext& ImportContext, const TSet<UObject*>& ValidAssets );

private:

	/* This class is designed not to have state. Instance are meaningless */
	FDatasmithImporter() = delete;
};

