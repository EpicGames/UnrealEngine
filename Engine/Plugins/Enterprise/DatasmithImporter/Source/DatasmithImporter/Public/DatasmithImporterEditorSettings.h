// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithImporterEditorSettings.generated.h"

UCLASS(config=EditorSettings, MinimalAPI)
class UDatasmithImporterEditorSettings : public UObject
{
	GENERATED_BODY()

public:
	UDatasmithImporterEditorSettings();

	UPROPERTY(config)
	bool bOfflineImporter;

};
