// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"

namespace UE::DatasmithImporter
{
	class FExternalSource;
}

class UFactory;

class DATASMITHIMPORTER_API FDatasmithImporterHelper
{
public:
	/** 
	 * Start the import process for the given factory type.
	 * The importer helper will pop up a file dialog to pick the file to import.
	 * Next, it will prompt for the folder where the imported assets will be saved.
	 * The Datasmith import options then appears and finally the factory import starts.
	 */
	template <class TFactory>
	static void Import()
	{
		ImportInternal(GetTypedDefaultObject<TFactory>());
	}

	template <class TFactory>
	static void Import(const TSharedRef<UE::DatasmithImporter::FExternalSource>& ExternalSource)
	{
		ImportInternal(GetTypedDefaultObject<TFactory>(), ExternalSource);
	}

	template <class TFactory>
	static FString GetFilter()
	{
		GetTypedDefaultObject<TFactory>()->InitializeFormats();
		return GetFilterStringInternal(GetTypedDefaultObject<TFactory>());
	}

	template <class TFactory>
	static TArray<FString> GetFormats()
	{
		GetTypedDefaultObject<TFactory>()->InitializeFormats();
		return GetTypedDefaultObject<TFactory>()->Formats;
	}

private:
	
	template <class T>
	static T* GetTypedDefaultObject()
	{
		return T::StaticClass()->template GetDefaultObject<T>();
	}

	static void ImportInternal(UFactory* Factory);
	static void ImportInternal(UFactory* Factory, const TSharedRef<UE::DatasmithImporter::FExternalSource>& ExternalSource);
	static FString GetFilterStringInternal(UFactory* Factory);
};
