// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"		// For inline LoadModuleChecked()
#include "DatasmithCore.h"

#include "Delegates/DelegateCombinations.h"
#include "Templates/Function.h"

#define DATASMITHIMPORTER_MODULE_NAME TEXT("DatasmithImporter")

/**
 * The public interface of the DatasmithImporter module
 */
class IDatasmithImporterModule : public IModuleInterface
{
public:

	/**
	 * Singleton-like access to IDatasmithImporter
	 *
	 * @return Returns DatasmithImporter singleton instance, loading the module on demand if needed
	 */
	static inline IDatasmithImporterModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IDatasmithImporterModule>(DATASMITHIMPORTER_MODULE_NAME);
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(DATASMITHIMPORTER_MODULE_NAME);
	}

	/**
	 * Restores the Datasmith imported values on an object
	 */
	virtual void ResetOverrides( UObject* Object ) = 0;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnGenerateDatasmithImportMenu, struct FToolMenuSection&);
	virtual FOnGenerateDatasmithImportMenu& OnGenerateDatasmithImportMenu() = 0;
};

