// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class AActor;
struct FDatasmithImportContext;
class IDatasmithLandscapeElement;

enum class EDatasmithImportActorPolicy : uint8;

class FDatasmithLandscapeImporter
{
public:
	static AActor* ImportLandscapeActor( const TSharedRef< IDatasmithLandscapeElement >& LandscapeActorElement, FDatasmithImportContext& ImportContext, EDatasmithImportActorPolicy ImportActorPolicy );
};
