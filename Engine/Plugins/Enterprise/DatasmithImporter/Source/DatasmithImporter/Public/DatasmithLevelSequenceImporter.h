// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/SharedPointer.h"

struct FDatasmithImportContext;
class IDatasmithLevelSequenceElement;
class ULevelSequence;

class FDatasmithLevelSequenceImporter
{
public:
	static ULevelSequence* ImportLevelSequence(const TSharedRef<IDatasmithLevelSequenceElement>& LevelSequenceElement, FDatasmithImportContext& InImportContext, ULevelSequence* ExistingLevelSequence);

	// Checks if we parsed and created all necessary ULevelSequence assets that need to be used as subsequences
	// by the IDatasmithSubsequenceAnimationElements contained in LevelSequenceElement
	static bool CanImportLevelSequence(const TSharedRef<IDatasmithLevelSequenceElement>& LevelSequenceElement, const FDatasmithImportContext& InImportContext);
};
