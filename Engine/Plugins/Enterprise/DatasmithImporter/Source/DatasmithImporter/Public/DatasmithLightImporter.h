// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Templates/SharedPointer.h"

class AActor;
class ADatasmithAreaLightActor;
class ALight;
class ALightmassPortal;
class AGroupActor;
class APointLight;
class AStaticMeshActor;
struct FDatasmithImportContext;
class FString;
class IDatasmithAreaLightElement;
class IDatasmithLightActorElement;
class IDatasmithLightmassPortalElement;
class IDatasmithPointLightElement;
class IDatasmithShaderElement;
class IDatasmithSpotLightElement;
class IDatasmithTextureElement;
struct FDatasmithImportContext;
struct FDatasmithAssetsImportContext;
class UDatasmithAreaLightComponent;
class ULightComponent;
class ULightmassPortalComponent;
class UPackage;
class UPointLightComponent;
class USceneComponent;
class USpotLightComponent;
class UTexture;
class UTextureLightProfile;
class UWorld;
class FDatasmithActorUniqueLabelProvider;

class DATASMITHIMPORTER_API FDatasmithLightImporter
{
public:
	static AActor* ImportLightActor( const TSharedRef< IDatasmithLightActorElement >& LightElement, FDatasmithImportContext& ImportContext );
	static USceneComponent* ImportLightComponent( const TSharedRef< IDatasmithLightActorElement >& LightElement, FDatasmithImportContext& ImportContext, UObject* Outer, FDatasmithActorUniqueLabelProvider& UniqueNameProvider );

	static AActor* CreateHDRISkyLight(const TSharedPtr< IDatasmithShaderElement >& ShaderElement, FDatasmithImportContext& ImportContext);

	static AActor* CreatePhysicalSky(FDatasmithImportContext& ImportContext);

private:
	static void SetTextureLightProfile( const TSharedRef< IDatasmithLightActorElement >& LightElement, class UDatasmithLightComponentTemplate* LightComponentTemplate, FDatasmithAssetsImportContext& AssetsContext );

	static void SetupLightComponent( ULightComponent* LightComponent, const TSharedPtr< IDatasmithLightActorElement >& LightElement, FDatasmithAssetsImportContext& AssetsContext );
	static void SetupPointLightComponent( UPointLightComponent* PointLightComponent, const TSharedRef< IDatasmithPointLightElement >& PointLightElement, FDatasmithAssetsImportContext& AssetsContext );
	static void SetupSpotLightComponent( USpotLightComponent* SpotLightComponent, const TSharedRef< IDatasmithSpotLightElement >& SpotLightElement, FDatasmithAssetsImportContext& AssetsContext );

	static AActor* ImportAreaLightActor( const TSharedRef< IDatasmithAreaLightElement >& AreaLightElement, FDatasmithImportContext& ImportContext );
	static USceneComponent* ImportAreaLightComponent( const TSharedRef< IDatasmithAreaLightElement >& AreaLightElement, FDatasmithImportContext& ImportContext, UObject* Outer, FDatasmithActorUniqueLabelProvider& UniqueNameProvider );

	static AActor* CreateAreaLightActor( const TSharedRef< IDatasmithAreaLightElement >& AreaLightElement, FDatasmithImportContext& ImportContext );
	static void SetupAreaLightActor( const TSharedRef< IDatasmithAreaLightElement >& AreaLightElement, FDatasmithImportContext& ImportContext, ADatasmithAreaLightActor* LightShapeActor );
	static ULightmassPortalComponent* ImportLightmassPortalComponent( const TSharedRef< IDatasmithLightmassPortalElement >& LightElement, FDatasmithImportContext& ImportContext, UObject* Outer, FDatasmithActorUniqueLabelProvider& UniqueNameProvider );
	static AActor* CreateSkyLight(const TSharedPtr< IDatasmithShaderElement >& ShaderElement, FDatasmithImportContext& ImportContext, bool bUseHDRMat);

};
