// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/ContainersFwd.h"
#include "Engine/EngineTypes.h"
#include "Math/Color.h"
#include "SceneTypes.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"
#include "UObject/StrongObjectPtr.h"

class IDatasmithCompositeTexture;
class IDatasmithMaterialElement;
class IDatasmithShaderElement;
class IDatasmithTextureElement;
class IDatasmithUEPbrMaterialElement;
struct FDatasmithAssetsImportContext;
class FDatasmithTextureSampler;
class UMaterialFunction;
class UMaterialExpressionFunctionOutput;
class UMaterialExpressionMakeMaterialAttributes;
class UMaterialExpressionTextureCoordinate;
class UMaterialExpressionTextureObject;
class UMaterialExpressionTextureSample;
class UTexture;
class UMaterial;
class UObject;
class FString;
class UPackage;
class UTextureLightProfile;
class UMaterialExpression;
class UMaterialInterface;
struct FExpressionInput;
class UMaterialExpressionMaterialFunctionCall;

enum class EDatasmithMaterialExpressionType : uint8;
enum class EDatasmithTextureMode : uint8;

enum class EDatasmithTextureSlot
{
	DIFFUSE,
	METALLIC,
	SPECULAR,
	ROUGHNESS,
	EMISSIVECOLOR,
	OPACITY,
	OPACITYMASK,
	NORMAL,
	WORLDPOSITIONOFFSET,
	SUBSURFACECOLOR,
	COATSPECULAR,
	COATROUGHNESS,
	AMBIANTOCCLUSION,
	REFRACTION,
	PIXELDEPTHOFFSET,
	SHADINGMODEL,
	MATERIALATTRIBUTES,
	NOSLOT
};

class DATASMITHIMPORTER_API FDatasmithMaterialExpressions
{
public:
	static UMaterialInterface* CreateDatasmithMaterial(UPackage* Package, const TSharedPtr< IDatasmithMaterialElement >& MaterialElement, FDatasmithAssetsImportContext& AssetsContext, UMaterial* ExistingMaterial, EObjectFlags ObjectFlags);
	static UMaterialFunction* CreateUEPbrMaterialFunction(UPackage* Package, const TSharedPtr< IDatasmithUEPbrMaterialElement >& MaterialElement, FDatasmithAssetsImportContext& AssetsContext, UMaterial* ExistingMaterial, EObjectFlags ObjectFlags);
	static UMaterialInterface* CreateUEPbrMaterial(UPackage* Package, const TSharedPtr< IDatasmithUEPbrMaterialElement >& MaterialElement, FDatasmithAssetsImportContext& AssetsContext, UMaterial* ExistingMaterial, EObjectFlags ObjectFlags);
	static UMaterialInterface* CreateUEPbrMaterialInstance(UPackage* Package, const TSharedPtr< IDatasmithUEPbrMaterialElement >& MaterialElement, FDatasmithAssetsImportContext& AssetsContext, UMaterialInterface* ParentMaterial, EObjectFlags ObjectFlags);
	static UMaterialInterface* CreateDatasmithEnvironmentMaterial(UPackage* Package, const TSharedPtr< IDatasmithShaderElement >& ShaderElement, FDatasmithAssetsImportContext& AssetsContext,
																  UMaterial* ExistingMaterial);

	static UMaterialExpression* AddCompExpression(const TSharedPtr<IDatasmithCompositeTexture>& Comp, UObject* UnrealMaterial, EDatasmithTextureSlot Slot, const FDatasmithAssetsImportContext& AssetsContext);

	static void ForEachParamsNameInMaterial(const TSharedPtr<IDatasmithUEPbrMaterialElement>& MaterialElement, const TFunctionRef<void (FName Expression, const EDatasmithMaterialExpressionType& ExpressionType, int32 Index)>& CallbackForEach);
private:
	static FLinearColor TemperatureToColor(float Kelvin);

	static UMaterialExpressionMakeMaterialAttributes* FindOrAddAttributesFromMatFunc(UMaterialFunction* Func);
	static UMaterialExpressionFunctionOutput* FindOrAddOutputFromMatFunc(UMaterialFunction* Func);

	static void GetSamplersRecursive(UMaterialExpression* Expression, TArray<UMaterialExpressionTextureSample*>& TextureSamplers);

	template<typename ExpressionsArrayType>
	static void GetSamplers(const ExpressionsArrayType& Expressions, TArray<FExpressionInput*> ExpressionInputs, TArray<UMaterialExpressionTextureSample*>& TextureSamplers)
	{
		for (UMaterialExpression* Expression : Expressions)
		{
			GetSamplersRecursive(Expression, TextureSamplers);
		}
	}

	static void GetTextureSamplersMaterial(UMaterialInterface* MaterialInterface, TArray<UMaterialExpressionTextureSample*>& TextureSamplers);
	static void GetTextureSamplersFunc(UMaterialFunction* MaterialFunc, TArray<UMaterialExpressionTextureSample*>& TextureSamplers);
	static void GetTextureSamplers(UObject* UnrealMatOrFunc, TArray<UMaterialExpressionTextureSample*>& TextureSamplers);

	static bool MatOrFuncDelExpression(UObject* Object, UMaterialExpression* ToBeConnected);

	static UMaterialExpression* Constant(UObject* UnrealMaterial, double Value);
	static UMaterialExpression* Multiply(UObject* UnrealMaterial, UMaterialExpression* ToBeConnectedA, double ValueA, UMaterialExpression* ToBeConnectedB, double ValueB);
	static UMaterialExpression* Divide(UObject* UnrealMaterial, UMaterialExpression* ToBeConnectedA, double ValueA, UMaterialExpression* ToBeConnectedB, double ValueB);
	static UMaterialExpression* Add(UObject* UnrealMaterial, UMaterialExpression* ToBeConnectedA, double ValueA, UMaterialExpression* ToBeConnectedB, double ValueB);
	static UMaterialExpression* Subtract(UObject* UnrealMaterial, UMaterialExpression* ToBeConnectedA, double ValueA, UMaterialExpression* ToBeConnectedB, double ValueB);
	static UMaterialExpression* Lerp(UObject* UnrealMaterial, UMaterialExpression* ToBeConnectedA, double ValueA, bool bAllowCleanUpA, UMaterialExpression* ToBeConnectedB, double ValueB, bool bAllowCleanUpB, UMaterialExpression* Alpha,
									 double ValueAlpha);

	static UMaterialExpression* HsvExpression(UObject* UnrealMaterial, UMaterialExpression* ToBeConnectedA, const TArray<double>& Hsv);
	static UMaterialExpression* HsvExpression(UObject* UnrealMaterial, UMaterialExpression* ToBeConnectedA, UMaterialExpression* ToBeConnectedB);
	static UMaterialExpression* HsvExpressionCustom(UObject* UnrealMaterial, UMaterialExpression* ToBeConnectedA, UMaterialExpression* ToBeConnectedB);

	static UMaterialExpression* Fresnel(UObject* UnrealMaterial, UMaterialExpression* ToBeConnectedA, double ValueA, UMaterialExpression* ToBeConnectedB, double ValueB);
	static UMaterialExpression* Power(UObject* UnrealMaterial, UMaterialExpression* ToBeConnectedA, UMaterialExpression* ToBeConnectedB, double ValueB);
	static UMaterialExpression* Desaturate(UObject* UnrealMaterial, UMaterialExpression* ToBeConnected);
	static UMaterialExpression* OneMinus(UObject* UnrealMaterial, UMaterialExpression* ToBeConnected);

	static UMaterialExpression* CalcIORComplex(double IORn, double IORk, UObject* UnrealMaterial, UMaterialExpression* ToBeConnected90, bool bAllowCleanUpExpr90 = true, UMaterialExpression* ToBeConnected0 = nullptr, bool bAllowCleanUpExpr0 = true);
	static UMaterialExpression* CalcIORSimple(double IORn, double IORk, UObject* UnrealMaterial, UMaterialExpression* ToBeConnected90, bool bAllowCleanUpExpr90 = true, UMaterialExpression* ToBeConnected0 = nullptr, bool bAllowCleanUpExpr0 = true);
	static UMaterialExpression* CalcIOR(double IORn, double IORk, UObject* UnrealMaterial, UMaterialExpression* ToBeConnected90, bool bAllowCleanUpExpr90 = true, UMaterialExpression* ToBeConnected0 = nullptr, bool bAllowCleanUpExpr0 = true);
	static UMaterialExpression* RefractionIOR(double IOR, UObject* UnrealMaterial);

	static UMaterialExpression* AddCroppedUVMappingExpression(UObject* UnrealMaterial, const FDatasmithTextureSampler& UV, UMaterialExpressionTextureCoordinate* ToBeConnected);

	static void CreateUEPbrMaterialGraph(const TSharedPtr< IDatasmithUEPbrMaterialElement >& MaterialElement, FDatasmithAssetsImportContext& AssetsContext, UObject* UnrealMaterialOrFunction);
	static EBlendMode GetUEPbrImportBlendMode(const TSharedPtr< IDatasmithUEPbrMaterialElement >& MaterialElement, const FDatasmithAssetsImportContext& AssetsContext);
	static EMaterialProperty DatasmithTextureSlotToMaterialProperty(EDatasmithTextureSlot InSlot);
	static FExpressionInput* GetMaterialOrFunctionSlot(UObject* UnrealMatOrFunc, EDatasmithTextureSlot InSlot);
	static void ConnectToSlot(UMaterialExpression* ToBeConnected, UObject* UnrealMatOrFunc, EDatasmithTextureSlot Slot, int32 InputChannel = 0);

	static UMaterialExpressionTextureObject* AddTextureObjectExpression(UTexture* Texture, UObject* UnrealMaterial);
	static UMaterialExpression* AddTextureExpression(UTexture* Texture, FDatasmithTextureSampler uv, double IOR, double IORk, UObject* UnrealMaterial, EDatasmithTextureSlot Slot, UMaterialExpressionTextureSample** OutTextureSample = nullptr);
	static UMaterialExpression* AddUVMirrorExpression(UObject* UnrealMaterial, UMaterialExpression* Expression, int XTile, int YTile);
	static UMaterialExpression* AddConstExpression(double Val, double IOR, double IORk, UObject* UnrealMaterial, EDatasmithTextureSlot Slot);
	static UMaterialExpression* AddRGBExpression(FLinearColor Col, double IOR, double IORk, UObject* UnrealMaterial, EDatasmithTextureSlot Slot, bool bForceEvenIfBlack = false);
	static UMaterialExpression* AddRGBAExpression(FLinearColor Col, double IOR, double IORk, UObject* UnrealMaterial, EDatasmithTextureSlot Slot, bool bForceEvenIfBlack = false);
	static UMaterialExpression* AddRegularExpression(const TSharedPtr<IDatasmithCompositeTexture>& Comp, UObject* UnrealMaterial, EDatasmithTextureSlot Slot, const FDatasmithAssetsImportContext& AssetsContext);
	static UMaterialExpression* AddHueTintExpression(UObject* UnrealMaterial, UMaterialExpression* Expression, float Amount, FLinearColor Color);
	static UMaterialExpression* AddCCorrectExpression(const TSharedPtr<IDatasmithCompositeTexture>& Comp, bool bUseGamma, UObject* UnrealMaterial, EDatasmithTextureSlot Slot, const FDatasmithAssetsImportContext& AssetsContext);
	static UMaterialExpression* AddMultiplyExpression(const TSharedPtr<IDatasmithCompositeTexture>& Comp, UObject* UnrealMaterial, EDatasmithTextureSlot Slot, const FDatasmithAssetsImportContext& AssetsContext);
	static UMaterialExpression* AddMixExpression(const TSharedPtr<IDatasmithCompositeTexture>& Comp, UObject* UnrealMaterial, EDatasmithTextureSlot Slot, const FDatasmithAssetsImportContext& AssetsContext);
	static UMaterialExpression* AddCompositeExpression(const TSharedPtr<IDatasmithCompositeTexture>& Comp, UObject* UnrealMaterial, EDatasmithTextureSlot Slot, const FDatasmithAssetsImportContext& AssetsContext);
	static UMaterialExpression* AddIorExpression(const TSharedPtr<IDatasmithCompositeTexture>& Comp, UObject* UnrealMaterial, EDatasmithTextureSlot Slot, const FDatasmithAssetsImportContext& AssetsContext);
	static UMaterialExpression* AddFresnelExpression(const TSharedPtr<IDatasmithCompositeTexture>& Comp, UObject* UnrealMaterial, EDatasmithTextureSlot Slot, const FDatasmithAssetsImportContext& AssetsContext);

	// glass node includes custom reflection
	static void AddGlassNode(UMaterialExpression* RefleExpression, UMaterialExpression* OpacityExpression, double IOR, double IORRefra, UObject* UnrealMaterial);

	static void CreateDatasmithMaterialHelper(UPackage* Package, const TSharedPtr< IDatasmithShaderElement >& ShaderElement, const FDatasmithAssetsImportContext& AssetsContext, UObject* UnrealMaterial);
	static UMaterialInterface* CreateDatasmithMaterial(UPackage* Package, const TSharedPtr< IDatasmithShaderElement >& ShaderElement, FDatasmithAssetsImportContext& AssetsContext, UMaterial* ExistingMaterial, EObjectFlags ObjectFlags);
	static UMaterialFunction* CreateDatasmithMaterialFunc(UPackage* Package, const TSharedPtr< IDatasmithMaterialElement >& MaterialElement, const TSharedPtr< IDatasmithShaderElement >& ShaderElement,
														  const FDatasmithAssetsImportContext& AssetsContext, UMaterialFunction* FindMaterialFunc, EObjectFlags ObjectFlags);

	static void GatherConnectedExpressions(UMaterialExpression* Expression, TArray<UMaterialExpression*>& ThisRowExpressions, int32 ChildNumber);
	static UMaterialExpression* GetCorrectExpressionFromComp(const TSharedPtr<IDatasmithCompositeTexture>& Comp, UObject* UnrealMaterial, const FDatasmithAssetsImportContext& AssetsContext, int32 Index, bool bMask, bool bRgbNoAlpha);
	static void CreateDatasmithMaterialCoat(const FDatasmithAssetsImportContext& AssetsContext, const TSharedPtr< IDatasmithShaderElement >& ShaderElement, UObject* UnrealMaterial);
	static void ModulateNormalAmount(UObject* UnrealMatOrFunc, double Amount);
	static void CreateParallaxOffset(UObject* UnrealMatOrFunc, UMaterialExpressionTextureObject* TextureObject, FDatasmithTextureSampler UV, float Amount);
	static UMaterialExpressionMaterialFunctionCall* BlendFunctions(UMaterial* UnrealMaterial, const FDatasmithAssetsImportContext& AssetsContext, const TSharedPtr< IDatasmithShaderElement >& ShaderTop,
																   UMaterialExpressionMaterialFunctionCall* CallTop, UMaterialExpressionMaterialFunctionCall* CallBase, UMaterialFunction* BlendFunc);

	static UMaterialExpression* CreateExpression( class IDatasmithMaterialExpression* MaterialExpression, const FDatasmithAssetsImportContext& AssetsContext, UObject* UnrealMaterialOrFunction );
	static UMaterialExpression* CreateTextureExpression( class IDatasmithMaterialExpressionTexture& DatasmithTextureExpression, const FDatasmithAssetsImportContext& AssetsContext, UObject* UnrealMaterialOrFunction);
	static UMaterialExpression* CreateTextureCoordinateExpression( class IDatasmithMaterialExpressionTextureCoordinate& DatasmithTextureCoordinateExpression, const FDatasmithAssetsImportContext& AssetsContext, UObject* UnrealMaterialOrFunction);
	static UMaterialExpression* CreateFlattenNormalExpression( class IDatasmithMaterialExpressionFlattenNormal& DatasmithFlattenNormal, const FDatasmithAssetsImportContext& AssetsContext, UObject* UnrealMaterialOrFunction);
	static UMaterialExpression* CreateBoolExpression( class IDatasmithMaterialExpressionBool& DatasmithBool, const FDatasmithAssetsImportContext& AssetsContext, UObject* UnrealMaterialOrFunction);
	static UMaterialExpression* CreateColorExpression( class IDatasmithMaterialExpressionColor& DatasmithColor, const FDatasmithAssetsImportContext& AssetsContext, UObject* UnrealMaterialOrFunction);
	static UMaterialExpression* CreateScalarExpression( class IDatasmithMaterialExpressionScalar& DatasmithScalar, const FDatasmithAssetsImportContext& AssetsContext, UObject* UnrealMaterialOrFunction);
	static UMaterialExpression* CreateGenericExpression( class IDatasmithMaterialExpressionGeneric& DatasmithGeneric, const FDatasmithAssetsImportContext& AssetsContext, UObject* UnrealMaterialOrFunction);
	static UMaterialExpression* CreateFunctionCallExpression( class IDatasmithMaterialExpressionFunctionCall& DatasmithFunctionCall, const FDatasmithAssetsImportContext& AssetsContext, UObject* UnrealMaterialOrFunction);
	static UMaterialExpression* CreateCustomExpression( class IDatasmithMaterialExpressionCustom& DatasmithCustom, const FDatasmithAssetsImportContext& AssetsContext, UObject* UnrealMaterialOrFunction);

	static void ConnectExpression( const TSharedRef< IDatasmithUEPbrMaterialElement >& MaterialElement, TArray< TStrongObjectPtr< UMaterialExpression > >& MaterialExpressions, class IDatasmithMaterialExpression* MaterialExpression, FExpressionInput* MaterialInput, int32 OutputIndex);
	static void ConnectAnyExpression( const TSharedRef< IDatasmithUEPbrMaterialElement >& MaterialElement, TArray< TStrongObjectPtr< UMaterialExpression > >& MaterialExpressions, class IDatasmithMaterialExpression& DatasmithExpression, FExpressionInput* ExpressionInput, int32 OutputIndex);
};
