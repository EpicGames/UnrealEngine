// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"

class IDatasmithBaseMaterialElement;
class IDatasmithDecalMaterialElement;
class IDatasmithMaterialElement;
class IDatasmithMaterialInstanceElement;
struct FDatasmithImportContext;
class UMaterialInterface;
class UMaterialFunction;
class UTexture;

namespace EMaterialRequirements
{
	enum Type
	{
		RequiresNothing = 0x00,
		RequiresNormals = 0x01,
		RequiresTangents = 0x02,
	};
};

class FDatasmithMaterialImporter
{
public:
	static UMaterialFunction* CreateMaterialFunction( FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithBaseMaterialElement >& MaterialElement );

	static UMaterialInterface* CreateMaterial( FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithBaseMaterialElement >& BaseMaterialElement, UMaterialInterface* ExistingMaterial );

	static int32 GetMaterialRequirements(UMaterialInterface* MaterialInterface);

private:
	static UMaterialInterface* ImportMaterialInstance( FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithMaterialInstanceElement >& MaterialElement, UMaterialInterface* ExistingMaterial );
	static UMaterialInterface* ImportDecalMaterial( FDatasmithImportContext& ImportContext, const TSharedRef< IDatasmithDecalMaterialElement >& MaterialElement, UMaterialInterface* ExistingMaterial );
};