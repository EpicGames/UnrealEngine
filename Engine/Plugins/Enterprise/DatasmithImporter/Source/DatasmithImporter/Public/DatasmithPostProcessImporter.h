// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/SharedPointer.h"

enum class EDatasmithImportActorPolicy : uint8;
class AActor;
class IDatasmithPostProcessElement;
class IDatasmithPostProcessVolumeElement;
struct FDatasmithImportContext;
struct FDatasmithPostProcessSettingsTemplate;

class FDatasmithPostProcessImporter
{
public:
	static FDatasmithPostProcessSettingsTemplate CopyDatasmithPostProcessToUEPostProcess(const TSharedPtr< IDatasmithPostProcessElement >& Src);
	static AActor* ImportPostProcessVolume( const TSharedRef< IDatasmithPostProcessVolumeElement >& PostProcessVolumeElement, FDatasmithImportContext& ImportContext, EDatasmithImportActorPolicy ImportActorPolicy );
};
