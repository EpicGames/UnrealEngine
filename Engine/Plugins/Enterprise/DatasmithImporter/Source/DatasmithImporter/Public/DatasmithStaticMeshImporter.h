// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DatasmithImportOptions.h"

#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"

struct FDatasmithAssetsImportContext;
struct FDatasmithImportContext;
class FDatasmithMesh;
struct FDatasmithStaticMeshImportOptions;
struct FMeshDescription;
struct FStaticMeshSourceModel;
struct FOverlappingCorners;
class IDatasmithMeshElement;
class IDatasmithScene;
class IMeshUtilities;
class UObject;
class UPackage;
class UStaticMesh;
struct FDatasmithMeshElementPayload;

class DATASMITHIMPORTER_API FDatasmithStaticMeshImporter
{
public:
	/** Imports a static mesh from a Mesh Element */
	static UStaticMesh* ImportStaticMesh( const TSharedRef< IDatasmithMeshElement > MeshElement, FDatasmithMeshElementPayload& Payload, EObjectFlags ObjectFlags, const FDatasmithStaticMeshImportOptions& ImportOptions, FDatasmithAssetsImportContext& AssetsContext, UStaticMesh* ExistingMesh );

	static bool ShouldRecomputeNormals( const FMeshDescription& MeshDescription, int32 BuildRequirements );
	static bool ShouldRecomputeTangents( const FMeshDescription& MeshDescription, int32 BuildRequirements );

	/** Calculates a lightmap density ratio for each IDatasmithMeshElement in the IDatasmithScene. */
	static TMap< TSharedRef< IDatasmithMeshElement >, float > CalculateMeshesLightmapWeights( const TSharedRef< IDatasmithScene >& SceneElement );

	/** Setup a UStaticMesh from an IDatasmithMeshElement. */
	static void SetupStaticMesh( FDatasmithAssetsImportContext& AssetsContext, TSharedRef< IDatasmithMeshElement > MeshElement, UStaticMesh* StaticMesh, const FDatasmithStaticMeshImportOptions& StaticMeshImportOptions, float LightmapWeight );

	/** Cleanup any invalid data in mesh descriptions that might cause the editor to crash or behave erratically (i.e. Having vertex position with NaN values). */
	static void CleanupMeshDescriptions(TArray< FMeshDescription >& MeshDescriptions);

	/** Builds the lightmap UVs and tangents for all the imported meshes. */
	static void PreBuildStaticMeshes( FDatasmithImportContext& ImportContext );

	/**
	 * Performs threadable build step on the static mesh, to be called before BuildStaticMesh
	 *
	 * @param StaticMesh        Mesh to process
	 * @return false if the mesh has no triangles or all its triangles are degenerated. True otherwise
	 */
	static bool PreBuildStaticMesh( UStaticMesh* StaticMesh );

	/**
	 * Performs the actual building of the static mesh
	 */
	static void BuildStaticMesh( UStaticMesh* StaticMesh );

	/**
	 * Performs the actual building of the static meshes in batch for better efficiency
	 *
	 * @param		StaticMeshes		The list of all static meshes to build.
	 * @param		ProgressCallback	If provided, will be used to abort task and report progress to higher level functions (should return true to continue, false to abort).
	 */
	static void BuildStaticMeshes(const TArray< UStaticMesh* >& StaticMeshes, TFunction<bool(UStaticMesh*)> ProgressCallback = nullptr);

private:
	static void ProcessCollision( UStaticMesh* StaticMesh, const TArray< FVector3f >& VertexPositions );

	/**
	 * Applies the UMaterialInterfaces related to the IDatasmithMeshElement to the UStaticMesh.
	 */
	static void ApplyMaterialsToStaticMesh( const FDatasmithAssetsImportContext& AssetsContext, const TSharedRef< IDatasmithMeshElement >& MeshElement, class UDatasmithStaticMeshTemplate* StaticMeshTemplate );
};
