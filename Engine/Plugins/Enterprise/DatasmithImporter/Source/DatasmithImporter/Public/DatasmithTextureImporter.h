// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Async/Future.h"
#include "Containers/Array.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"
#include "UObject/StrongObjectPtr.h"

#include "InterchangeEngineFwd.h"
#include "InterchangePipelineBase.h"
#include "InterchangeSourceData.h"
#include "Async/TaskGraphInterfaces.h"

#include "DatasmithTextureImporter.generated.h"

struct FDatasmithImportContext;
class IDatasmithTextureElement;
class UTexture;
class UTextureFactory;

class FDatasmithTextureImporter : private FNoncopyable
{
public:
	FDatasmithTextureImporter(FDatasmithImportContext& InImportContext);
	~FDatasmithTextureImporter();

	UTexture* CreateTexture(const TSharedPtr<IDatasmithTextureElement>& TextureElement, const TArray<uint8>& TextureData, const FString& Extension);
	bool GetTextureData(const TSharedPtr<IDatasmithTextureElement>& TextureElement, TArray<uint8>& TextureData, FString& Extension);
	UTexture* CreateIESTexture(const TSharedPtr<IDatasmithTextureElement>& TextureElement);

	UE::Interchange::FAssetImportResultRef CreateTextureAsync(const TSharedPtr<IDatasmithTextureElement>& TextureElement);

private:
	bool ResizeTextureElement(const TSharedPtr<IDatasmithTextureElement>& TextureElement, FString& ResizedFilename);

private:
	FDatasmithImportContext& ImportContext;
	TStrongObjectPtr< UTextureFactory > TextureFact;
	FString TempDir;
};

UCLASS()
class UDatasmithTexturePipeline : public UInterchangePipelineBase
{
	GENERATED_BODY()

public:
	virtual void ExecutePipeline(UInterchangeBaseNodeContainer* BaseNodeContainer, const TArray<UInterchangeSourceData*>& SourceDatas, const FString& ContentBasePath) override;

	bool IsScripted() override
	{
		return false;
	}
	TSharedPtr< IDatasmithTextureElement > TextureElement;
};