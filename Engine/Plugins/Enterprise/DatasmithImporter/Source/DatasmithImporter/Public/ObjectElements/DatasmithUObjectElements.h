// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

#include "DatasmithDefinitions.h"
#include "DatasmithVariantElements.h"
#include "IDatasmithSceneElements.h"

#include "DatasmithUObjectElements.generated.h"

/*
 * UDatasmithObjectElement
 */
UCLASS(Abstract, BlueprintType, Transient)
class DATASMITHIMPORTER_API UDatasmithObjectElement : public UObject
{
	GENERATED_BODY()

public:
	/** Gets the element name */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	FString GetElementName() const;

	/** Gets the element label used in the UI */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	FString GetLabel() const;

	/** Sets the element label used in the UI */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetLabel(const FString& InLabel);

	/** Is the Element still valid for the Datasmith Scene */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	virtual bool IsElementValid() const;

protected:
	// Need this in order to recover the IElement from the UElement in
	// UDatasmithObjectPropertyCaptureElement::SetRecordedObject
	friend UDatasmithObjectPropertyCaptureElement;

	// TODO: They are not covariant because of the TWeakPtr, can't override in child
	virtual TWeakPtr<IDatasmithElement> GetIDatasmithElement() const PURE_VIRTUAL(UDatasmithObjectElement::GetIDatasmithElement, return nullptr;);
};

/**
 * UDatasmithKeyValueProperty
 */
UCLASS()
class DATASMITHIMPORTER_API UDatasmithKeyValueProperty : public UDatasmithObjectElement
{
	GENERATED_BODY()

public:
	/** Get the type of this property */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	EDatasmithKeyValuePropertyType GetPropertyType() const;

	/** Set the type of this property */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetPropertyType(EDatasmithKeyValuePropertyType InType);

	/** Get the value of this property */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	FString GetValue() const;

	/** Sets the value of this property */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetValue(const FString& Value);

public:
	TWeakPtr<IDatasmithKeyValueProperty> GetDatasmithKeyValueProperty() const { return KeyValueProperty; }
	void SetDatasmithKeyValueProperty(const TSharedPtr<IDatasmithKeyValueProperty>& InElement) { KeyValueProperty = InElement; }

	/** Is the Element still valid for the Datasmith Scene */
	virtual bool IsElementValid() const override;

protected:
	virtual TWeakPtr<IDatasmithElement> GetIDatasmithElement() const override { return KeyValueProperty; }

private:
	TWeakPtr<IDatasmithKeyValueProperty> KeyValueProperty;
};

/*
 * UDatasmithActorElement
 */
UCLASS(Abstract)
class DATASMITHIMPORTER_API UDatasmithActorElement : public UDatasmithObjectElement
{
	GENERATED_BODY()

public:
	/** Get absolute translation of this entity */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	FVector GetTranslation() const;

	/** Set absolute translation of this entity */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetTranslation(FVector Value);

	/** Get absolute scale of this entity */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	FVector GetScale() const;

	/** Set absolute scale of this entity */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetScale(FVector Value);

	/** Get rotation (in quaternion format) of this entity */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	FQuat GetRotation() const;

	/** Set rotation (in quaternion format) of this entity */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetRotation(FQuat Value);

	/** Get the the name of the layer that contains this entity */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	FString GetLayer() const;

	/** Set the the the layer that contains this entity, layer will be auto-created from its name */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetLayer(const FString& InLayer);

	/** Get the tags of an Actor element */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	TArray<FString> GetTags() const;

	/** Set the tags of an Actor element */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetTags(const TArray<FString>& InTags);

	/** Adds a child to the Actor Element*/
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void AddChild(UDatasmithActorElement* InChild);

	/** Get the number of children on this actor */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	int32 GetChildrenCount() const;

	/** Get the children of the mesh actor. Use AddChild -or- RemoveChild to modify the actor's children */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	TArray<UDatasmithActorElement*> GetChildren() const;

	/** Remove a new child from the Actor Element */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void RemoveChild(UDatasmithActorElement* InChild);

	/** Get the actor's visibility */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	bool GetVisibility() const;

	/** Set the actor's visibility */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetVisibility(bool bInVisibility);

public:
	virtual TWeakPtr<IDatasmithActorElement> GetIDatasmithActorElement() const PURE_VIRTUAL(UDatasmithActorElement::GetIDatasmithActorElement, return nullptr;);
};


/**
 * UDatasmithMeshElement defines an actual geometry.
 */
UCLASS()
class DATASMITHIMPORTER_API UDatasmithMeshElement : public UDatasmithObjectElement
{
	GENERATED_BODY()

public:
	/** Get the output filename, it can be absolute or relative to the scene file */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	FString GetFile() const;

	/** Get the bounding box width */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	float GetBoundingBoxWidth() const;

	/** Get the bounding box height */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	float GetBoundingBoxHeight() const;

	/** Get the bounding box depth */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	float GetBoundingBoxDepth() const;

	/** Get the bounding box represented by a Vector. X is Width, Y is Height, Z is Depth. */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	FVector GetBoundingBoxSize() const;

	/** Get the total surface area */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	float GetLightMapArea() const;

	/** Get the UV channel that will be used for the lightmap */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	int32 GetLightmapCoordinateIndex() const;

	/**
	 * Set the UV channel that will be used for the lightmap
	 * Note: If the lightmap coordinate index is something greater than -1 it will make the importer skip the lightmap generation
	 */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	void SetLightmapCoordinateIndex(int32 UVChannel);

	/** Get the source UV channel that will be used at import to generate the lightmap UVs */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	int32 GetLightmapSourceUV() const;

	/** Set the source UV channel that will be used at import to generate the lightmap UVs */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	void SetLightmapSourceUV(int32 UVChannel);

	/** Set the material name to associate with slot SlotId */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetMaterial(const FString& MaterialName, int32 SlotId);

	/** Get the material name in the material slot SlotId */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	FString GetMaterial(int32 SlotId);

public:
	TWeakPtr<IDatasmithMeshElement> GetDatasmithMeshElement() const { return MeshElement; }
	void SetDatasmithMeshElement(const TSharedPtr<IDatasmithMeshElement>& InElement) { MeshElement = InElement; }

	/** Is the Element still valid for the Datasmith Scene */
	virtual bool IsElementValid() const override;

protected:
	virtual TWeakPtr<IDatasmithElement> GetIDatasmithElement() const override { return MeshElement; }

private:
	TWeakPtr<IDatasmithMeshElement> MeshElement;
};


/*
 * UDatasmithMeshActorElement
 */
UCLASS()
class DATASMITHIMPORTER_API UDatasmithMeshActorElement : public UDatasmithActorElement
{
	GENERATED_BODY()

public:
	/** Adds a new material to the Actor Element */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void AddMaterialOverride(class UDatasmithMaterialIDElement* Material);

	/** Get the amount of materials on this mesh */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	int32 GetMaterialOverridesCount() const;

	/** Get the i-th material of this actor */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	TArray<UDatasmithMaterialIDElement*> GetMaterials() const;

	/** Get the amount of materials on this mesh */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void RemoveMaterialOverride(class UDatasmithMaterialIDElement* Material);

	/** Get the path name of the StaticMesh associated with the actor */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	FString GetStaticMeshPathName() const;

	/**
	 * Set the path name of the StaticMesh that the actor is using
	 * It can be either a package path to refer to an existing mesh or a mesh name to refer to a MeshElement in the DatasmithScene
	 */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetStaticMeshPathName(const FString& InStaticMeshName);

	bool IsStaticMeshPathRelative() const;

	/** Get the Datasmith MeshElement associated with the actor. The Mesh can be a direct reference to an Unreal Mesh. If it's the case it will return an invalid MeshElement. */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	UDatasmithMeshElement* GetMeshElement();

	/**
	 * Get the Bounding Box of the Actor as a Vector. X is Width, Y is Height, Z is Depth.
	 * The value will are taken from the MeshElement and are factored by the Actor Scale.
	 * Bounding Box size can only be calculated if the Mesh can be found.
	 */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	FVector GetBoundingBoxSize() const;

public:
	TWeakPtr<IDatasmithMeshActorElement> GetDatasmithMeshActorElement() const { return MeshActorElemement; }
	void SetDatasmithMeshActorElement(const TSharedPtr<IDatasmithMeshActorElement>& InElement) { MeshActorElemement = InElement; }

	/** Is the Element still valid for the Datasmith Scene */
	virtual bool IsElementValid() const override;

	virtual TWeakPtr<IDatasmithElement> GetIDatasmithElement() const override { return MeshActorElemement; }
	virtual TWeakPtr<IDatasmithActorElement> GetIDatasmithActorElement() const override { return MeshActorElemement; }

private:
	TWeakPtr<IDatasmithMeshActorElement> MeshActorElemement;
};


/*
 * UDatasmithLightActorElement
 */
UCLASS()
class DATASMITHIMPORTER_API UDatasmithLightActorElement : public UDatasmithActorElement
{
	GENERATED_BODY()

public:
	/** Return true on light enabled, false otherwise */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	bool IsEnabled() const;

	/** Set enable property of the light */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	void SetEnabled(bool bIsEnabled);

	/** Get light intensity */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	float GetIntensity() const;

	/** Set light intensity */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	void SetIntensity(float Intensity);

	/** Get light color on linear mode */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	FLinearColor GetColor() const;

	/** Set light color on linear mode */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	void SetColor(FLinearColor Color);

	/** Get the light temperature in Kelvin */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	float GetTemperature() const;

	/** Set the light temperature in Kelvin */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	void SetTemperature(float Temperature);

	/** Get if the light color is controlled by temperature */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	bool GetUseTemperature() const;

	/** Set if the light color is controlled by temperature */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	void SetUseTemperature(bool bUseTemperature);

	/** Get the path of the Ies definition file */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	FString GetIesFile() const;

	/** Set the path of the Ies definition file */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	void SetIesFile(const FString& IesFile);

	/** Set if this light is controlled by Ies definition file */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	bool GetUseIes() const;

	/** Get if this light is controlled by Ies definition file */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	void SetUseIes(bool bUseIes);

	/** Get the Ies brightness multiplier */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	float GetIesBrightnessScale() const;

	/** Set the Ies brightness multiplier */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	void SetIesBrightnessScale(float IesBrightnessScale);

	/** Get if the emissive amount of the ies is controlled by the brightness scale */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	bool GetUseIesBrightness() const;

	/** Set if the emissive amount of the ies is controlled by the brightness scale */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	void SetUseIesBrightness(bool bUseIesBrightness);

public:
	TWeakPtr<IDatasmithLightActorElement> GetDatasmithLightActorElement() const { return LightActorElement; }
	void SetDatasmithLightActorElement(const TSharedPtr<IDatasmithLightActorElement>& InElement) { LightActorElement = InElement; }

	/** Is the Element still valid for the Datasmith Scene */
	virtual bool IsElementValid() const override;

	virtual TWeakPtr<IDatasmithElement> GetIDatasmithElement() const override { return LightActorElement; }
	virtual TWeakPtr<IDatasmithActorElement> GetIDatasmithActorElement() const override { return LightActorElement; }

private:
	TWeakPtr<IDatasmithLightActorElement> LightActorElement;
};


/*
 * UDatasmithCameraActorElement
 */
UCLASS()
class DATASMITHIMPORTER_API UDatasmithCameraActorElement : public UDatasmithActorElement
{
	GENERATED_BODY()

public:
	/** Get camera sensor width in millimeters */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	float GetSensorWidth() const;

	/** Set camera sensor width in millimeters */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	void SetSensorWidth(float SensorWidth);

	/** Get framebuffer aspect ratio (width/height) */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	float GetSensorAspectRatio() const;

	/** Set framebuffer aspect ratio (width/height) */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	void SetSensorAspectRatio(float SensorAspectRatio);

	/** Get camera focus distance in centimeters */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	float GetFocusDistance() const;

	/** Set camera focus distance in centimeters */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	void SetFocusDistance(float FocusDistance);

	/** Get camera FStop also known as FNumber */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	float GetFStop() const;

	/** Set camera FStop also known as FNumber */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	void SetFStop(float FStop);

	/** Get camera focal length in millimeters */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	float GetFocalLength() const;

	/** Set camera focal length in millimeters */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	void SetFocalLength(float FocalLength);

	/** Get camera's postprocess */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	UDatasmithPostProcessElement* GetPostProcess();

	/** Get camera look at actor name */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	FString GetLookAtActor() const;

	/** Set camera look at actor name */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	void SetLookAtActor(const FString& ActorPathName);

	/** Get camera look at allow roll state */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	bool GetLookAtAllowRoll() const;

	/** Set camera look at allow roll state */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	void SetLookAtAllowRoll(bool bAllow);

public:
	TWeakPtr<IDatasmithCameraActorElement> GetDatasmithCameraActorElement() const { return CameraActorElement; }
	void SetDatasmithCameraActorElement(const TSharedPtr<IDatasmithCameraActorElement>& InElement) { CameraActorElement = InElement; }

	/** Is the Element still valid for the Datasmith Scene */
	virtual bool IsElementValid() const override;

	virtual TWeakPtr<IDatasmithElement> GetIDatasmithElement() const override { return CameraActorElement; }
	virtual TWeakPtr<IDatasmithActorElement> GetIDatasmithActorElement() const override { return CameraActorElement; }

private:
	TWeakPtr<IDatasmithCameraActorElement> CameraActorElement;
};

/**
 * UDatasmithCustomActorElement
 */
UCLASS()
class DATASMITHIMPORTER_API UDatasmithCustomActorElement : public UDatasmithActorElement
{
	GENERATED_BODY()

public:
	/** The blueprint to instantiate. */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	FString GetClassOrPathName() const;

	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetClassOrPathName(const FString& InPathName);

	/** Get the total amount of properties in this blueprint actor */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	int32 GetPropertiesCount() const;

	/** Get the property i-th of this blueprint actor */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	UDatasmithKeyValueProperty* GetProperty(int32 i);

	/** Get a property by its name if it exists */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	UDatasmithKeyValueProperty* GetPropertyByName(const FString& InName);

	/** Add a property to this blueprint actor*/
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void AddProperty(UDatasmithKeyValueProperty* Property);

	/** Removes a property from this blueprint actor, doesn't preserve ordering */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void RemoveProperty(UDatasmithKeyValueProperty* Property);

public:
	TWeakPtr<IDatasmithCustomActorElement> GetDatasmithCustomActorElement() const { return CustomActorElement; }
	void SetDatasmithCustomActorElement(const TSharedPtr<IDatasmithCustomActorElement>& InElement) { CustomActorElement = InElement; }

	/** Is the Element still valid for the Datasmith Scene */
	virtual bool IsElementValid() const override;
	bool IsElementValid(const TWeakPtr<IDatasmithKeyValueProperty>& Element) const;

	UDatasmithKeyValueProperty* FindOrAddElement(const TSharedPtr<IDatasmithKeyValueProperty>& InElement);

protected:
	virtual TWeakPtr<IDatasmithElement> GetIDatasmithElement() const override { return CustomActorElement; }
	virtual TWeakPtr<IDatasmithActorElement> GetIDatasmithActorElement() const override { return CustomActorElement; }

	TMap< TWeakPtr< IDatasmithKeyValueProperty >, UDatasmithKeyValueProperty* > Properties;

private:
	TWeakPtr<IDatasmithCustomActorElement> CustomActorElement;
};

/**
 * UDatasmithBaseMaterialElement
 */
UCLASS()
class DATASMITHIMPORTER_API UDatasmithBaseMaterialElement : public UDatasmithObjectElement
{
	GENERATED_BODY()

public:
	TWeakPtr<IDatasmithBaseMaterialElement> GetDatasmithBaseMaterialElement() const { return BaseMaterialElemement; }
	void SetDatasmithBaseMaterialElement(const TSharedPtr<IDatasmithBaseMaterialElement>& InElement) { BaseMaterialElemement = InElement; }

	/** Is the Element still valid for the Datasmith Scene */
	virtual bool IsElementValid() const override;

protected:
	virtual TWeakPtr<IDatasmithElement> GetIDatasmithElement() const override { return BaseMaterialElemement; }

private:
	TWeakPtr<IDatasmithBaseMaterialElement> BaseMaterialElemement;
};

/**
 * UDatasmithMaterialIDElement
 */
UCLASS()
class DATASMITHIMPORTER_API UDatasmithMaterialIDElement : public UDatasmithObjectElement
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	int32 GetId() const;

public:
	TWeakPtr<IDatasmithMaterialIDElement> GetDatasmithMaterialIDElement() const { return MaterialElemement; }
	void SetDatasmithMaterialIDElement(const TSharedPtr<IDatasmithMaterialIDElement>& InElement) { MaterialElemement = InElement; }

	/** Is the Element still valid for the Datasmith Scene */
	virtual bool IsElementValid() const override;

protected:
	virtual TWeakPtr<IDatasmithElement> GetIDatasmithElement() const override { return MaterialElemement; }

private:
	TWeakPtr<IDatasmithMaterialIDElement> MaterialElemement;
};


/*
 * UDatasmithPostProcessElement
 */
UCLASS()
class DATASMITHIMPORTER_API UDatasmithPostProcessElement : public UDatasmithObjectElement
{
	GENERATED_BODY()

public:
	/** Get color filter temperature in Kelvin */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	float GetTemperature() const;

	/** Set color filter temperature in Kelvin */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetTemperature(float Temperature);

	/** Set color filter in linear color scale */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	FLinearColor GetColorFilter() const;

	/** Get color filter in linear color scale */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetColorFilter(FLinearColor ColorFilter);

	/** Get vignette amount */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	float GetVignette() const;

	/** Set vignette amount */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetVignette(float Vignette);

	/** Get depth of field multiplier */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	float GetDof() const;

	/** Set depth of field multiplier */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetDof(float Dof);

	/** Get motion blur multiplier */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	float GetMotionBlur() const;

	/** Set motion blur multiplier */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetMotionBlur(float MotionBlur);

	/** Get color saturation */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	float GetSaturation() const;

	/** Set color saturation */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetSaturation(float Saturation);

	/** Get camera ISO */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	float GetCameraISO() const;

	/** Set camera ISO */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetCameraISO(float CameraISO);

	/** Get camera shutter speed in 1/seconds (ie: 60 = 1/60s) */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	float GetCameraShutterSpeed() const;

	/** Set camera shutter speed in 1/seconds (ie: 60 = 1/60s) */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	void SetCameraShutterSpeed(float CameraShutterSpeed);

public:
	TWeakPtr<IDatasmithPostProcessElement> GetDatasmithPostProcessElement() const { return PostProcessElemement; }
	void SetDatasmithPostProcessElement(const TSharedPtr<IDatasmithPostProcessElement>& InElement) { PostProcessElemement = InElement; }

	/** Is the Element still valid for the Datasmith Scene */
	virtual bool IsElementValid() const override;

protected:
	virtual TWeakPtr<IDatasmithElement> GetIDatasmithElement() const override { return PostProcessElemement; }

private:
	TWeakPtr<IDatasmithPostProcessElement> PostProcessElemement;
};


/*
 * UDatasmithTextureElement
 */
UCLASS()
class DATASMITHIMPORTER_API UDatasmithTextureElement : public UDatasmithObjectElement
{
	GENERATED_BODY()

public:
	/** Get texture filename */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	FString GetFile() const;

	/** Set texture filename */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetFile(const FString& File);

	/** Get texture usage */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	EDatasmithTextureMode GetTextureMode() const;

	/** Set texture usage */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetTextureMode(EDatasmithTextureMode Mode);

	/** Get allow texture resizing */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	bool GetAllowResize() const;

	/** Set allow texture resizing */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetAllowResize(bool bAllowResize);

	/** Get texture gamma <= 0 for auto */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	float GetRGBCurve() const;

	/** Set texture gamma <= 0 for auto */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetRGBCurve(float InRGBCurve);

	/** Gets the color space of the texture */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	EDatasmithColorSpace GetColorSpace() const;

	/** Sets the color space of the texture */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetColorSpace(EDatasmithColorSpace Option);

public:
	TWeakPtr<IDatasmithTextureElement> GetDatasmithTextureElement() const { return TextureElemement; }
	void SetDatasmithTextureElement(const TSharedPtr<IDatasmithTextureElement>& InElement) { TextureElemement = InElement; }

	/** Is the Element still valid for the Datasmith Scene */
	virtual bool IsElementValid() const override;

protected:
	virtual TWeakPtr<IDatasmithElement> GetIDatasmithElement() const override { return TextureElemement; }

private:
	TWeakPtr<IDatasmithTextureElement> TextureElemement;
};

/*
 * UDatasmithMetaDataElement
 */
UCLASS()
class DATASMITHIMPORTER_API UDatasmithMetaDataElement : public UDatasmithObjectElement
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	int32 GetPropertiesCount() const;

	/** Get the property i-th of this meta data */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	UDatasmithKeyValueProperty* GetProperty(int32 i);

	/** Get a property by its name if it exists */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	UDatasmithKeyValueProperty* GetPropertyByName(const FString& InName);

	/** Get the element that is associated with this meta data */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	UDatasmithObjectElement* GetAssociatedElement() const;

	/** Get this metadata element properties as a map of keys and values */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Element")
	void GetProperties(TArray<FString>& OutKeys, TArray<FString>& OutValues);

public:
	TWeakPtr<IDatasmithMetaDataElement> GetDatasmithMetaDataElement() const { return MetaDataElement; }
	void SetDatasmithMetaDataElement(const TSharedPtr<IDatasmithMetaDataElement>& InElement) { MetaDataElement = InElement; }

	/** Is the Element still valid for the Datasmith Scene */
	virtual bool IsElementValid() const override;
	bool IsElementValid(const TWeakPtr<IDatasmithKeyValueProperty>& Element) const;

	UDatasmithKeyValueProperty* FindOrAddElement(const TSharedPtr<IDatasmithKeyValueProperty>& InElement);

protected:
	virtual TWeakPtr<IDatasmithElement> GetIDatasmithElement() const override { return MetaDataElement; }

	TMap< TWeakPtr< IDatasmithKeyValueProperty >, UDatasmithKeyValueProperty* > Properties;

private:
	TWeakPtr<IDatasmithMetaDataElement> MetaDataElement;
};

/*
 * UDatasmithBasePropertyCaptureElement
 */
UCLASS()
class DATASMITHIMPORTER_API UDatasmithBasePropertyCaptureElement : public UDatasmithObjectElement
{
	GENERATED_BODY()

public:
	/** Sets the path used when attempting to capture a generic property */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetPropertyPath(const FString& Path) const;

	/** Gets the path used when attempting to capture a generic property */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	FString GetPropertyPath() const;

	/** Sets the category of this property capture */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetCategory(EDatasmithPropertyCategory Category);

	/** Gets the category of this property capture */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	EDatasmithPropertyCategory GetCategory() const;

public:
	TWeakPtr<IDatasmithBasePropertyCaptureElement> GetBasePropertyCaptureElement() const { return DatasmithElement; }

	/** Is the Element still valid for the Datasmith Scene */
	virtual bool IsElementValid() const override;

protected:
	virtual TWeakPtr<IDatasmithElement> GetIDatasmithElement() const override { return DatasmithElement; }
	TWeakPtr<IDatasmithBasePropertyCaptureElement> DatasmithElement;
};

/*
 * UDatasmithPropertyCaptureElement
 */
UCLASS()
class DATASMITHIMPORTER_API UDatasmithPropertyCaptureElement : public UDatasmithBasePropertyCaptureElement
{
	GENERATED_BODY()

public:
	/**
	 * Get the recorded value for this property as a boolean.
	 * Returned value is meaningless if the property is not of boolean type.
	 */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	bool GetValueBool();

	/** Set the recorded value for this capture */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetValueBool(bool InValue);

	/**
	 * Get the recorded value for this property as an int32.
	 * Returned value is meaningless if the property is not of int32 type.
	 */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	int32 GetValueInt();

	/** Set the recorded value for this capture */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetValueInt(int32 InValue);

	/**
	 * Get the recorded value for this property as a float.
	 * Returned value is meaningless if the property is not of float type.
	 */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	float GetValueFloat();

	/** Set the recorded value for this capture */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetValueFloat(float InValue);

	/**
	 * Get the recorded value for this property as a string.
	 * Returned value is meaningless if the property is not of string type.
	 */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	FString GetValueString();

	/** Set the recorded value for this capture */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetValueString(const FString& InValue);

	/**
	 * Get the recorded value for this property as a rotator.
	 * Returned value is meaningless if the property is not of rotator type.
	 */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	FRotator GetValueRotator();

	/** Set the recorded value for this capture */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetValueRotator(FRotator InValue);

	/**
	 * Get the recorded value for this property as a color.
	 * Returned value is meaningless if the property is not of color type.
	 */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	FColor GetValueColor();

	/** Set the recorded value for this capture */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetValueColor(FColor InValue);

	/**
	 * Get the recorded value for this property as a linear color.
	 * Returned value is meaningless if the property is not of linear color type.
	 */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	FLinearColor GetValueLinearColor();

	/** Set the recorded value for this capture */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetValueLinearColor(FLinearColor InValue);

	/**
	 * Get the recorded value for this property as a vector.
	 * Returned value is meaningless if the property is not of vector type.
	 */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	FVector GetValueVector();

	/** Set the recorded value for this capture */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetValueVector(FVector InValue);

	/**
	 * Get the recorded value for this property as a quat.
	 * Returned value is meaningless if the property is not of quat type.
	 */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	FQuat GetValueQuat();

	/** Set the recorded value for this capture */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetValueQuat(FQuat InValue);

	/**
	 * Get the recorded value for this property as a vector4.
	 * Returned value is meaningless if the property is not of vector4 type.
	 */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	FVector4 GetValueVector4();

	/** Set the recorded value for this capture */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetValueVector4(FVector4 InValue);

	/**
	 * Get the recorded value for this property as a vector2d.
	 * Returned value is meaningless if the property is not of vector2 type.
	 */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	FVector2D GetValueVector2D();

	/** Set the recorded value for this capture */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetValueVector2D(FVector2D InValue);

	/**
	 * Get the recorded value for this property as an int point.
	 * Returned value is meaningless if the property is not of int point type.
	 */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	FIntPoint GetValueIntPoint();

	/** Set the recorded value for this capture */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetValueIntPoint(FIntPoint InValue);

public:
	TWeakPtr<IDatasmithPropertyCaptureElement> GetPropertyCaptureElement() const { return StaticCastSharedPtr<IDatasmithPropertyCaptureElement>(DatasmithElement.Pin()); }
	void SetPropertyCaptureElement(const TSharedPtr<IDatasmithPropertyCaptureElement>& InElement) { DatasmithElement = InElement; }
};

/*
 * UDatasmithObjectPropertyCaptureElement
 */
UCLASS()
class DATASMITHIMPORTER_API UDatasmithObjectPropertyCaptureElement : public UDatasmithBasePropertyCaptureElement
{
	GENERATED_BODY()

public:
	/** Gets the category of this property capture */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetRecordedObject(UDatasmithObjectElement* Object);

	/** Gets the category of this property capture */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	UDatasmithObjectElement* GetRecordedObject() const;

public:
	TWeakPtr<IDatasmithObjectPropertyCaptureElement> GetObjectPropertyCaptureElement() const { return StaticCastSharedPtr<IDatasmithObjectPropertyCaptureElement>(DatasmithElement.Pin()); }
	void SetObjectPropertyCaptureElement(const TSharedPtr<IDatasmithObjectPropertyCaptureElement>& InElement) { DatasmithElement = InElement; }
};

/*
 * UDatasmithActorBindingElement
 */
UCLASS()
class DATASMITHIMPORTER_API UDatasmithActorBindingElement : public UDatasmithObjectElement
{
	GENERATED_BODY()

public:
	/** Sets the actor that this binding will try capturing */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void SetActor(UDatasmithActorElement* Actor);

	/** Gets the actor that this binding will try capturing */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	UDatasmithActorElement* GetActor() const;

	/** Create a new property capture and add it to this binding */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	UDatasmithPropertyCaptureElement* CreatePropertyCapture();

	/** Create a new object property capture and add it to this binding */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	UDatasmithObjectPropertyCaptureElement* CreateObjectPropertyCapture();

	/** Adds an existing property capture to this binding */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void AddPropertyCapture(const UDatasmithBasePropertyCaptureElement* Prop);

	/** Gets how many properties will be captured from the bound actor */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	int32 GetPropertyCapturesCount() const;

	/** Gets property that will be captured from the bound actor */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	UDatasmithBasePropertyCaptureElement* GetPropertyCapture(int32 Index);

	/** Removes one of the properties that will be captured from the bound actor */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void RemovePropertyCapture(const UDatasmithBasePropertyCaptureElement* Prop);

public:
	TWeakPtr<IDatasmithActorBindingElement> GetActorBindingElement() const { return DatasmithElement; }
	void SetActorBindingElement(const TSharedPtr<IDatasmithActorBindingElement>& InElement) { DatasmithElement = InElement; }

	/** Is the Element still valid for the Datasmith Scene */
	virtual bool IsElementValid() const override;

protected:
	virtual TWeakPtr<IDatasmithElement> GetIDatasmithElement() const override { return DatasmithElement; }

private:
	TWeakPtr<IDatasmithActorBindingElement> DatasmithElement;
};

/*
 * UDatasmithVariantElement
 */
UCLASS()
class DATASMITHIMPORTER_API UDatasmithVariantElement : public UDatasmithObjectElement
{
	GENERATED_BODY()

public:
	/** Create a new actor binding and add it to this variant */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	UDatasmithActorBindingElement* CreateActorBinding();

	/** Adds an existing actor binding to this variant */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void AddActorBinding(const UDatasmithActorBindingElement* Binding);

	/** Gets how many actor bindings are in this variant */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	int32 GetActorBindingsCount() const;

	/** Gets an actor binding from this variant */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	UDatasmithActorBindingElement* GetActorBinding(int32 Index);

	/** Removes an actor binding from this variant */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void RemoveActorBinding(const UDatasmithActorBindingElement* Binding);

public:
	TWeakPtr<IDatasmithVariantElement> GetVariantElement() const { return DatasmithElement; }
	void SetVariantElement(const TSharedPtr<IDatasmithVariantElement>& InElement) { DatasmithElement = InElement; }

	/** Is the Element still valid for the Datasmith Scene */
	virtual bool IsElementValid() const override;

protected:
	virtual TWeakPtr<IDatasmithElement> GetIDatasmithElement() const override { return DatasmithElement; }

private:
	TWeakPtr<IDatasmithVariantElement> DatasmithElement;
};

/*
 * UDatasmithVariantSetElement
 */
UCLASS()
class DATASMITHIMPORTER_API UDatasmithVariantSetElement : public UDatasmithObjectElement
{
	GENERATED_BODY()

public:
	/** Create a new variant and add it to the parent variant set */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	UDatasmithVariantElement* CreateVariant(FName InElementName);

	/** Adds an existing variant to this variant set */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void AddVariant(const UDatasmithVariantElement* Variant);

	/** Gets how many variants are in this variant set */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	int32 GetVariantsCount() const;

	/** Gets a variant from this variant set */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	UDatasmithVariantElement* GetVariant(int32 Index);

	/** Removes a variant from this variant set */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void RemoveVariant(const UDatasmithVariantElement* Variant);

public:
	TWeakPtr<IDatasmithVariantSetElement> GetVariantSetElement() const { return DatasmithElement; }
	void SetVariantSetElement(const TSharedPtr<IDatasmithVariantSetElement>& InElement) { DatasmithElement = InElement; }

	/** Is the Element still valid for the Datasmith Scene */
	virtual bool IsElementValid() const override;

protected:
	virtual TWeakPtr<IDatasmithElement> GetIDatasmithElement() const override { return DatasmithElement; }

private:
	TWeakPtr<IDatasmithVariantSetElement> DatasmithElement;
};

/*
 * UDatasmithLevelVariantSetsElement
 */
UCLASS()
class DATASMITHIMPORTER_API UDatasmithLevelVariantSetsElement : public UDatasmithObjectElement
{
	GENERATED_BODY()

public:
	/** Create a new variant set and add it to the parent level variant sets */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	UDatasmithVariantSetElement* CreateVariantSet(FName InElementName);

	/** Adds an existing variant set to this level variant sets */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void AddVariantSet(const UDatasmithVariantSetElement* VariantSet);

	/** Gets how many variant sets are in this level variant sets */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	int32 GetVariantSetsCount() const;

	/** Gets a variant set from this level variant sets */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	UDatasmithVariantSetElement* GetVariantSet(int32 Index);

	/** Removes a variant set from this level variant sets */
	UFUNCTION(BlueprintCallable, Category="Datasmith | Element")
	void RemoveVariantSet(const UDatasmithVariantSetElement* VariantSet);

public:
	TWeakPtr<IDatasmithLevelVariantSetsElement> GetLevelVariantSetsElement() const { return DatasmithElement; }
	void SetLevelVariantSetsElement(const TSharedPtr<IDatasmithLevelVariantSetsElement>& InElement) { DatasmithElement = InElement; }

	/** Is the Element still valid for the Datasmith Scene */
	virtual bool IsElementValid() const override;

protected:
	virtual TWeakPtr<IDatasmithElement> GetIDatasmithElement() const override { return DatasmithElement; }

private:
	TWeakPtr<IDatasmithLevelVariantSetsElement> DatasmithElement;
};