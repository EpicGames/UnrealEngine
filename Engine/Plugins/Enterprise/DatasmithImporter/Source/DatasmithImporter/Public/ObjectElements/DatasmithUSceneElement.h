// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ObjectElements/DatasmithUObjectElements.h"
#include "Templates/SubclassOf.h"
#include "UObject/GCObject.h"

#include "DatasmithUSceneElement.generated.h"

/*
 */
UCLASS(BlueprintType, Transient)
class DATASMITHIMPORTER_API UDatasmithSceneElementBase : public UObject
{
	GENERATED_UCLASS_BODY()

	struct FDatasmithSceneCollector : public FGCObject
	{
		FDatasmithSceneCollector();
		UDatasmithSceneElementBase* DatasmithSceneElement;
		virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		virtual FString GetReferencerName() const override
		{
			return TEXT("UDatasmithSceneElementBase::FDatasmithSceneCollector");
		}
	};
	friend FDatasmithSceneCollector;

public:
	/** Sets the name of the host application which created the scene */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	FString GetHost() const;

	/** Returns the Datasmith version used to export the scene */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	FString GetExporterVersion() const;

	/** Returns the vendor name of the application used to export the scene */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	FString GetVendor() const;

	/** Returns the product name of the application used to export the scene */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	FString GetProductName() const;

	/** Returns the product version of the application used to export the scene */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	FString GetProductVersion() const;

	/** Returns the user identifier who exported the scene */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	FString GetUserID() const;

	/** Returns the OS name used by user who exported the scene */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	FString GetUserOS() const;

	/** Returns the time taken to export the scene */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	int32 GetExportDuration() const;

	/**
	* Physical Sky could be generated in a large amount of modes, like material, lights etc
	* that's why it has been added as static, just enable it and it is done.
	* Notice that if a HDRI environment is used this gets disabled.
	*/
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	bool GetUsePhysicalSky() const;

public:
	/** Create a new Mesh and add it to the Datasmith scene */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	UDatasmithMeshElement* CreateMesh(FName InElementName);

	/** Create an array with all the Mesh in the Datasmith scene. Use CreateMesh -or- RemoveMesh to modify the Datasmith scene. */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	TArray<UDatasmithMeshElement*> GetMeshes();

	/**
	 * Find in the Datasmith scene the MeshElement that correspond to the mesh path name.
	 * The function will return an invalid MeshElement, if the MeshPathName is empty or if it's not relative to the Datasmith scene or if it's not found.
	 */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	UDatasmithMeshElement* GetMeshByPathName(const FString& MeshPathName);

	/** Remove the mesh from the Datasmith scene */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	void RemoveMesh(UDatasmithMeshElement* InMesh);

public:
	/** Create a new MeshActor and add it to the Datasmith scene */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	UDatasmithMeshActorElement* CreateMeshActor(FName InElementName);

	/**
	 * Create an array with the MeshActor in the Datasmith scene that are at the root level of the hierarchy.
	 * Use CreateMeshActor -or- RemoveMeshActor to modify the Datasmith scene.
	 */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	TArray<UDatasmithMeshActorElement*> GetMeshActors();

	/**
	 * Create an array with all the MeshActor in the Datasmith scene without taking into account the hierarchy.
	 * Use CreateMeshActor -or- RemoveMeshActor to modify the Datasmith scene.
	 */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	TArray<UDatasmithMeshActorElement*> GetAllMeshActors();

	/** Remove the MeshActor from the Datasmith scene */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	void RemoveMeshActor(UDatasmithMeshActorElement* InMeshActor, EDatasmithActorRemovalRule RemoveRule = EDatasmithActorRemovalRule::RemoveChildren);

public:
	/**
	 * Create an array with the LightActor in the Datasmith scene that are at the root level of the hierarchy.
	 * Use RemoveLightActor to modify the Datasmith scene.
	 */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	TArray<UDatasmithLightActorElement*> GetLightActors();

	/**
	 * Create an array with all the LightActor in the Datasmith scene without taking into account the hierarchy.
	 * Use RemoveLightActor to modify the Datasmith scene.
	 */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	TArray<UDatasmithLightActorElement*> GetAllLightActors();

	/** Remove the LightActor from the Datasmith scene */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	void RemoveLightActor(UDatasmithLightActorElement* InLightActor, EDatasmithActorRemovalRule RemoveRule = EDatasmithActorRemovalRule::RemoveChildren);

public:
	/** Create a new Camera Actor and add it to the Datasmith scene */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	UDatasmithCameraActorElement* CreateCameraActor(FName InElementName);

	/**
	 * Create an array with the CameraActor in the Datasmith scene that are at the root level of the hierarchy.
	 * Use CreateCameraActor -or- RemoveCameraActor to modify the Datasmith scene.
	 */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	TArray<UDatasmithCameraActorElement*> GetCameraActors();

	/**
	 * Create an array with all the CameraActor in the Datasmith scene without taking into account the hierarchy.
	 * Use CreateCameraActor -or- RemoveCameraActor to modify the Datasmith scene.
	 */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	TArray<UDatasmithCameraActorElement*> GetAllCameraActors();

	/** Remove the Camera actor from the Datasmith scene */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	void RemoveCameraActor(UDatasmithCameraActorElement* InMeshActor, EDatasmithActorRemovalRule RemoveRule = EDatasmithActorRemovalRule::RemoveChildren);

public:
	/**
	 * Create an array with the CustomActor in the Datasmith scene that are at the root level of the hierarchy.
	 * Use RemoveCustomActor to modify the Datasmith scene.
	 */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	TArray<UDatasmithCustomActorElement*> GetCustomActors();

	/**
	 * Create an array with all the CustomActor in the Datasmith scene without taking into account the hierarchy.
	 * Use RemoveCustomActor to modify the Datasmith scene.
	 */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	TArray<UDatasmithCustomActorElement*> GetAllCustomActors();

	/** Remove the LightActor from the Datasmith scene */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	void RemoveCustomActor(UDatasmithCustomActorElement* InCustomActor, EDatasmithActorRemovalRule RemoveRule = EDatasmithActorRemovalRule::RemoveChildren);

public:
	/** Create a new Texture and add it to the Datasmith scene */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	UDatasmithTextureElement* CreateTexture(FName InElementName);

	/** Create an array with all the Textures in the Datasmith scene. Call CreateTexture -or- RemoveTexture to modify the Datasmith scene. */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	TArray<UDatasmithTextureElement*> GetTextures();

	/** Remove the Texture from the Datasmith scene */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	void RemoveTexture(UDatasmithTextureElement* InElement);

public:
	/** Create an array with all the Materials in the Datasmith scene */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	TArray<UDatasmithBaseMaterialElement*> GetAllMaterials();

	/** Remove the material from the Datasmith scene */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	void RemoveMaterial(UDatasmithBaseMaterialElement* InElement);

public:
	/** Get the Postprocess used by the scene. Can be invalid. */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	UDatasmithPostProcessElement* GetPostProcess();

private:
	TArray<UDatasmithMetaDataElement*> GetMetaData();

public:

	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene", meta = (ScriptName = "GetMetadataForObject", DisplayName = "Get Metadata For Object"))
	UDatasmithMetaDataElement* GetMetaDataForObject(UDatasmithObjectElement* Object);

	/**
	 * Get the value associated with the given key of the metadata element associated with the given object.
	 * @param	Object	The Object that is associated with the metadata element of interest.
	 * @param	Key		The key to find in the metadata element.
	 * @return			The string value associated with the given key
	 */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene", meta = (ScriptName = "GetMetadataValueForKey", DisplayName = "Get Metadata Value For Key"))
	FString GetMetaDataValueForKey(UDatasmithObjectElement* Object, const FString& Key);

	/**
	 * Get the keys and values for which the associated value contains the string to match for the metadata element associated with the given object.
	 * @param	Object			The Object that is associated with the metadata element of interest.
	 * @param	StringToMatch	The string to match in the values.
	 * @param	OutKeys			Output array of keys for which the associated values contain the string to match.
	 * @param	OutValues		Output array of values associated to the keys.
	 */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene", meta = (ScriptName = "GetMetadataKeysAndValuesForValue", DisplayName = "Get Metadata Keys And Values For Value"))
	void GetMetaDataKeysAndValuesForValue(UDatasmithObjectElement* Object, const FString& StringToMatch, TArray<FString>& OutKeys, TArray<FString>& OutValues);

	/**
	 *	Find all metadata elements associated with objects of the given type.
	 *	@param	ObjectClass		Class of the object on which to filter, if specificed; otherwise there's no filtering
	 *	@param	OutMetadatas	Output array of metadata elements.
	 */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene", meta = (ScriptName = "GetAllMetadata", DisplayName = "Get All Metadata"))
	void GetAllMetaData(TSubclassOf<UDatasmithObjectElement> ObjectClass, TArray<UDatasmithMetaDataElement*>& OutMetadatas);

	/**
	 *	Find all objects of the given type that have a metadata element that contains the given key and their associated values.
	 *	@param	Key			The key to find in the metadata element.
	 *	@param	ObjectClass	Class of the object on which to filter, if specificed; otherwise there's no filtering
	 *	@param	OutObjects	Output array of objects for which the metadata element contains the given key.
	 *	@param	OutValues	Output array of values associated with each object in OutObjects.
	 */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene", meta = (DeterminesOutputType = "ObjectClass", DynamicOutputParam = "OutObjects"))
	void GetAllObjectsAndValuesForKey(const FString& Key, TSubclassOf<UDatasmithObjectElement> ObjectClass, TArray<UDatasmithObjectElement*>& OutObjects, TArray<FString>& OutValues);

public:
	/** Create a new level variant sets and add it to the Datasmith scene */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	UDatasmithLevelVariantSetsElement* CreateLevelVariantSets(FName InElementName);

	/**
	 * Create an array with all the level variants sets from the Datasmith scene
	 * Use CreateLevelVariantSets -or- RemoveLevelVariantSets to modify the Datasmith scene.
	 */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	TArray<UDatasmithLevelVariantSetsElement*> GetAllLevelVariantSets();

	/** Remove the level variant sets from the Datasmith scene */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	void RemoveLevelVariantSets(UDatasmithLevelVariantSetsElement* InElement);

public:
	/** Attach the actor to its new parent. Detach the actor if it was already attached. */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	void AttachActor(UDatasmithActorElement* NewParent, UDatasmithActorElement* Child, EDatasmithActorAttachmentRule AttachmentRule);

	/** Attach the actor to the scene root. Detach the actor if it was already attached. */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Scene")
	void AttachActorToSceneRoot(UDatasmithActorElement* NewParent, EDatasmithActorAttachmentRule AttachmentRule);

public:
	void SetDatasmithSceneElement(TSharedPtr<class IDatasmithScene> InElement);
	TSharedPtr<class IDatasmithScene> GetSceneElement() { return SceneElement; }

	bool IsElementValid(const TWeakPtr<IDatasmithBaseMaterialElement>& Element) const;
	bool IsElementValid(const TWeakPtr<IDatasmithMaterialIDElement>& Element) const;
	bool IsElementValid(const TWeakPtr<IDatasmithMeshElement>& Element) const;
	bool IsElementValid(const TWeakPtr<IDatasmithMeshActorElement>& Element) const;
	bool IsElementValid(const TWeakPtr<IDatasmithLightActorElement>& Element) const;
	bool IsElementValid(const TWeakPtr<IDatasmithCameraActorElement>& Element) const;
	bool IsElementValid(const TWeakPtr<IDatasmithTextureElement>& Element) const;
	bool IsElementValid(const TWeakPtr<IDatasmithPostProcessElement>& Element) const;
	bool IsElementValid(const TWeakPtr<IDatasmithMetaDataElement>& Element) const;
	bool IsElementValid(const TWeakPtr<IDatasmithCustomActorElement>& Element) const;
	bool IsElementValid(const TWeakPtr<IDatasmithBasePropertyCaptureElement>& Element) const;
	bool IsElementValid(const TWeakPtr<IDatasmithActorBindingElement>& Element) const;
	bool IsElementValid(const TWeakPtr<IDatasmithVariantElement>& Element) const;
	bool IsElementValid(const TWeakPtr<IDatasmithVariantSetElement>& Element) const;
	bool IsElementValid(const TWeakPtr<IDatasmithLevelVariantSetsElement>& Element) const;

	UDatasmithActorElement* FindOrAddActorElement(const TSharedPtr<IDatasmithActorElement>& InElement);

	UDatasmithObjectElement* FindOrAddElement(const TSharedPtr<IDatasmithElement>& InElement);
	UDatasmithBaseMaterialElement* FindOrAddElement(const TSharedPtr<IDatasmithBaseMaterialElement>& InElement);
	UDatasmithMaterialIDElement* FindOrAddElement(const TSharedPtr<IDatasmithMaterialIDElement>& InElement);
	UDatasmithMeshElement* FindOrAddElement(const TSharedPtr<IDatasmithMeshElement>& InElement);
	UDatasmithMeshActorElement* FindOrAddElement(const TSharedPtr<IDatasmithMeshActorElement>& InElement);
	UDatasmithLightActorElement* FindOrAddElement(const TSharedPtr<IDatasmithLightActorElement>& InElement);
	UDatasmithCameraActorElement* FindOrAddElement(const TSharedPtr<IDatasmithCameraActorElement>& InElement);
	UDatasmithPostProcessElement* FindOrAddElement(const TSharedPtr<IDatasmithPostProcessElement>& InElement);
	UDatasmithTextureElement* FindOrAddElement(const TSharedPtr<IDatasmithTextureElement>& InElement);
	UDatasmithMetaDataElement* FindOrAddElement(const TSharedPtr<IDatasmithMetaDataElement>& InElement);
	UDatasmithCustomActorElement* FindOrAddElement(const TSharedPtr<IDatasmithCustomActorElement>& InElement);
	UDatasmithPropertyCaptureElement* FindOrAddElement(const TSharedPtr<IDatasmithPropertyCaptureElement>& InElement);
	UDatasmithObjectPropertyCaptureElement* FindOrAddElement(const TSharedPtr<IDatasmithObjectPropertyCaptureElement>& InElement);
	UDatasmithActorBindingElement* FindOrAddElement(const TSharedPtr<IDatasmithActorBindingElement>& InElement);
	UDatasmithVariantElement* FindOrAddElement(const TSharedPtr<IDatasmithVariantElement>& InElement);
	UDatasmithVariantSetElement* FindOrAddElement(const TSharedPtr<IDatasmithVariantSetElement>& InElement);
	UDatasmithLevelVariantSetsElement* FindOrAddElement(const TSharedPtr<IDatasmithLevelVariantSetsElement>& InElement);

protected:
	virtual void Reset();

private:
	void ExternalAddReferencedObjects(FReferenceCollector& Collector);

	FDatasmithSceneCollector DatasmithSceneCollector;

protected:
	TSharedPtr<class IDatasmithScene> SceneElement;

	TMap<TWeakPtr<IDatasmithBaseMaterialElement>, TObjectPtr<UDatasmithBaseMaterialElement>> Materials;
	TMap<TWeakPtr<IDatasmithMaterialIDElement>, TObjectPtr<UDatasmithMaterialIDElement>> MaterialIDs;
	TMap<TWeakPtr<IDatasmithMeshElement>, TObjectPtr<UDatasmithMeshElement>> Meshes;
	TMap<TWeakPtr<IDatasmithMeshActorElement>, TObjectPtr<UDatasmithMeshActorElement>> MeshActors;
	TMap<TWeakPtr<IDatasmithLightActorElement>, TObjectPtr<UDatasmithLightActorElement>> LightActors;
	TMap<TWeakPtr<IDatasmithCameraActorElement>, TObjectPtr<UDatasmithCameraActorElement>> CameraActors;
	TMap<TWeakPtr<IDatasmithPostProcessElement>, TObjectPtr<UDatasmithPostProcessElement>> PostProcesses;
	TMap<TWeakPtr<IDatasmithTextureElement>, TObjectPtr<UDatasmithTextureElement>> Textures;
	TMap<TWeakPtr<IDatasmithMetaDataElement>, UDatasmithMetaDataElement*> MetaData;
	TMap<TWeakPtr<IDatasmithCustomActorElement>, UDatasmithCustomActorElement*> CustomActors;
	TMap<TWeakPtr<IDatasmithPropertyCaptureElement>, TObjectPtr<UDatasmithPropertyCaptureElement>> PropertyCaptures;
	TMap<TWeakPtr<IDatasmithObjectPropertyCaptureElement>, TObjectPtr<UDatasmithObjectPropertyCaptureElement>> ObjectPropertyCaptures;
	TMap<TWeakPtr<IDatasmithActorBindingElement>, TObjectPtr<UDatasmithActorBindingElement>> ActorBindings;
	TMap<TWeakPtr<IDatasmithVariantElement>, TObjectPtr<UDatasmithVariantElement>> Variants;
	TMap<TWeakPtr<IDatasmithVariantSetElement>, TObjectPtr<UDatasmithVariantSetElement>> VariantSets;
	TMap<TWeakPtr<IDatasmithLevelVariantSetsElement>, TObjectPtr<UDatasmithLevelVariantSetsElement>> LevelVariantSets;
};
