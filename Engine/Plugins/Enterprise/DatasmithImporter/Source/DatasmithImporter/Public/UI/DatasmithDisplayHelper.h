// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "UObject/Object.h"
#include "UObject/StrongObjectPtr.h"

namespace Datasmith
{
	struct FDisplayParameters
	{
		bool bAskForSameOption = false;
		FText WindowTitle;
		FText FileLabel;
		FText FileTooltip; // eg. file full path
		FText PackageLabel;
		FText ProceedButtonLabel;
		FText ProceedButtonTooltip;
		FText CancelButtonLabel;
		FText CancelButtonTooltip;
		float MinDetailHeight = 0;
		float MinDetailWidth = 0;
	};

	struct FDisplayResult
	{
		bool bValidated = true;
		bool bUseSameOption = false;
	};

	/**
	 * Display custom objects as options for user inputs
	 *
	 * @param Options Objects to be displayed for user inputs
	 * @return bool   true if workflow should continue, false on cancellation
	 */
	FDisplayResult DATASMITHIMPORTER_API DisplayOptions(const TArray<TStrongObjectPtr<UObject>>& Options, const FDisplayParameters& Parameters={});
	FDisplayResult DATASMITHIMPORTER_API DisplayOptions(const TArray<UObject*>& Options, const FDisplayParameters& Parameters={});
	FDisplayResult DATASMITHIMPORTER_API DisplayOptions(const TStrongObjectPtr<UObject>& Options, const FDisplayParameters& Parameters={});
}
