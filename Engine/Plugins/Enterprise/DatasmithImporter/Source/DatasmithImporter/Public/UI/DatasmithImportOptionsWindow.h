// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Application/SlateApplication.h"

#include "IDetailsView.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"

class DATASMITHIMPORTER_API SDatasmithOptionsWindow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDatasmithOptionsWindow)
		: _FileFormatVersion(0.f)
		, _bAskForSameOption(false)
	{}

	SLATE_ARGUMENT(TArray<UObject*>, ImportOptions)
	SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
	SLATE_ARGUMENT(FText, FileNameText)
	SLATE_ARGUMENT(FText, FilePathText)
	SLATE_ARGUMENT(float, FileFormatVersion)
	SLATE_ARGUMENT(FText, FileSDKVersion)
	SLATE_ARGUMENT(FText, PackagePathText)
	SLATE_ARGUMENT(bool, bAskForSameOption)
	SLATE_ARGUMENT(FText, ProceedButtonLabel)
	SLATE_ARGUMENT(FText, ProceedButtonTooltip)
	SLATE_ARGUMENT(FText, CancelButtonLabel)
	SLATE_ARGUMENT(FText, CancelButtonTooltip)
	SLATE_ARGUMENT(int32, MinDetailHeight)
	SLATE_ARGUMENT(int32, MinDetailWidth)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);

	virtual bool SupportsKeyboardFocus() const override { return true; }

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		if (InKeyEvent.GetKey() == EKeys::Escape)
		{
			return OnCancel();
		}

		return FReply::Unhandled();
	}

	bool ShouldImport() const
	{
		return bShouldProceed;
	}

	bool UseSameOptions() const
	{
		return bUseSameOptions;
	}

	static const FString& GetDocumentationURL() { return DocumentationURL; }

private:
	FReply OnProceed()
	{
		bShouldProceed = true;
		if (Window.IsValid())
		{
			Window.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	FReply OnCancel()
	{
		bShouldProceed = false;
		bUseSameOptions = false;
		if (Window.IsValid())
		{
			Window.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	FReply OnHelp(const FGeometry& SenderGeometry, const FPointerEvent& MouseEvent);

	void OnSameOptionToggle(ECheckBoxState InCheckState);

	TSharedRef< class SCompoundWidget > ConstructWarningWidget( float FileVersion, FText FileSDKVersion );

private:
	TArray<UObject*> ImportOptions;
	TWeakPtr< SWindow > Window;
	bool bShouldProceed;
	bool bUseSameOptions;
	static FString DocumentationURL;
};
