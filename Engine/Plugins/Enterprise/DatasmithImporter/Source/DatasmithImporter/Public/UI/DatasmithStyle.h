// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"

class FSlateStyleSet;
class ISlateStyle;

/** Manages the style that provides resources for Datasmith plugins widgets. */
class FDatasmithStyle
{
public:
	static void Initialize();
	static void Shutdown();
	static const ISlateStyle& Get();
	static FName GetStyleSetName();
	static FName GetContextName();

	/** Adds an icon (.png in the given ResourcePath) with the given style name to this style set */
	static void SetIcon(const FString& StyleName, const FString& ResourcePath);

private:
	static TUniquePtr< class FSlateStyleSet > Create();
	static TUniquePtr< class FSlateStyleSet > DatasmithStyleInstance;
};