// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "UVGenerationFlattenMapping.generated.h"

class UStaticMesh;
struct FMeshDescription;

UCLASS()
class DATASMITHIMPORTER_API UUVGenerationFlattenMapping : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | StaticMesh", meta = (DisplayName = "Generate Unwrapped UVs"))
	static void GenerateFlattenMappingUVs(UStaticMesh* InStaticMesh, int32 UVChannel, float AngleThreshold = 66.f);

	static void GenerateUVs(FMeshDescription& InMesh, int32 UVChannel, bool bRemoveDegenerate, float AngleThreshold = 66.f);

private:

	static TArray<int32> GetOverlappingCornersRemapping(const FMeshDescription& InMeshDescription, bool bRemoveDegenerates);
};
