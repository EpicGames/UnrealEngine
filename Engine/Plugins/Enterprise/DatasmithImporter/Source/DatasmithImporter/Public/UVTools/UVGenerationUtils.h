// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UStaticMesh;

class DATASMITHIMPORTER_API UVGenerationUtils
{
public:
	/**
	 * Returns the index of the next open UV channel, existing channels filled with default values are considered open.
	 * If no open channel is found, returns -1. 
	 */
	static int32 GetNextOpenUVChannel(UStaticMesh* StaticMesh, int32 LODIndex);

	/*
	 * Setup the a good lightmap minimal resolution for the given static mesh LOD in its build settings.
	 *
	 * @param StaticMesh	The StaticMesh being set up
	 * @param LODIndex		The LOD being set up.
	 */
	static void SetupGeneratedLightmapUVResolution(UStaticMesh* StaticMesh, int32 LODIndex);
};