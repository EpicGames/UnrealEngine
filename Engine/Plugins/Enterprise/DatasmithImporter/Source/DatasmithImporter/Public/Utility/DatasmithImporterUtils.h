// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "DatasmithImportContext.h"
#include "DatasmithScene.h"
#include "IDatasmithSceneElements.h"
#include "InterchangeManager.h"
#include "Materials/MaterialInterface.h"
#include "Misc/PackageName.h"
#include "UObject/UObjectHash.h"
#include "Engine/Texture.h"

class ADatasmithSceneActor;
class IDatasmithScene;
class IDatasmithUEPbrMaterialElement;
class UDatasmithScene;
class UFunction;
class UPackage;
class UWorld;

DECLARE_LOG_CATEGORY_EXTERN( LogDatasmithImport, Log, All );

class DATASMITHIMPORTER_API FDatasmithImporterUtils
{
public:
	enum class EAssetCreationStatus : uint8
	{
		CS_CanCreate,
		CS_HasRedirector,
		CS_ClassMismatch,
		CS_NameTooLong, // asset paths are store in FName, which have an internal limitation.
		CS_NameTooShort,
	};

		/** Loads an IDatasmithScene from a UDatasmithScene */
	static TSharedPtr< IDatasmithScene > LoadDatasmithScene( UDatasmithScene* DatasmithSceneAsset );

	/** Saves an IDatasmithScene into a UDatasmithScene */
	static void SaveDatasmithScene( TSharedRef< IDatasmithScene > DatasmithScene, UDatasmithScene* DatasmithSceneAsset );

	/** Spawns a ADatasmithSceneActor and adds it to the ImportContext */
	static ADatasmithSceneActor* CreateImportSceneActor( FDatasmithImportContext& ImportContext, FTransform WorldTransform );

	/**
	 * Finds all the ADatasmithSceneActor in the world that refers to the given scene
	 * @param World			    Scope of the search
	 * @param DatasmithScene    SceneActor must reference this scene to be included
	 */
	static TArray< ADatasmithSceneActor* > FindSceneActors( UWorld* World, UDatasmithScene* DatasmithScene );

	/**
	 * Delete non imported datasmith elements (actors and components) from a Datasmith Scene Actor hierarchy
	 *
	 * @param SourceSceneActor           The scene on which the actors where imported
	 * @param DestinationSceneActor      The scene on which the actors will be deleted
	 * @param IgnoredDatasmithElements   The Datasmith actors that shouldn't be deleted if they aren't imported
	 */
	static void DeleteNonImportedDatasmithElementFromSceneActor(ADatasmithSceneActor& SourceSceneActor, ADatasmithSceneActor& DestinationSceneActor, const TSet< FName >& IgnoredDatasmithElements);

	/**
	 * Delete an actor
	 * Remove it from the it's level, mark it pending kill and move it the transient package to avoid any potential name collision
	 * @param Actor The actor to delete
	 */
	static void DeleteActor( AActor& Actor );

	/**
	 * Finds a UStaticMesh, UTexture or UMaterialInterface.
	 * Relative paths are resolved based on the AssetsContext.
	 * Absolute paths are sent through FDatasmithImporterUtils::FindObject.
	 */
	template< typename ObjectType >
	static ObjectType* FindAsset( const FDatasmithAssetsImportContext& AssetsContext, const TCHAR* ObjectPathName );

	/**
	 * Find an object with a given name in a package
	 * Use FSoftObjectPath to perform the search
	 * Load the package /ParentPackage/ObjectName if it exists and is not in memory yet
	 *
	 * @param	ParentPackage				Parent package to look in
	 * @param	ObjectName					Name of object to look for
	 */
	template< class ObjectType >
	inline static ObjectType* FindObject( const UPackage* ParentPackage, const FString& ObjectName );

	/**
	 * Add a layer to the world if there is no other layer with the same name
	 *
	 * @param World			The world to which the layers will be added
	 * @param LayerNames	The name of the layers to be added
	 */
	static void AddUniqueLayersToWorld( UWorld* World, const TSet< FName >& LayerNames );

	/**
	 * Try to compute a char budget for asset names, including FNames constraints, OS constraints,
	 * parent package, and user defined limitation.
	 *
	 * @param ParentPackage destination of the asset (Package path consume a part of the budget)
	 * @return An estimation of the budget for asset names
	 */
	static int32 GetAssetNameMaxCharCount(const UPackage* ParentPackage);

	/**
	 * @param Package			Package to create the asset in
	 * @param AssetName			The name of the asset to create
	 * @param OutFailReason		Text containing explanation about failure
	 * Given a package, a name and a class, check if an existing asset with a different class
	 * would not prevent the creation of such asset. A special report is done for object redirector
	 */
	template< class ObjectType >
	static bool CanCreateAsset(const UPackage* Package, const FString& AssetName, FText& OutFailReason);

	/**
	 * @param AssetPathName		Full path name of the asset to create
	 * @param OutFailReason		Text containing explanation about failure
	 * Returns true if the asset can be safely created
	 * Given a path and a class, check if an existing asset with a different class
	 * would not prevent the creation of such asset. A special report is done for object redirector
	 */
	template< class ObjectType >
	static bool CanCreateAsset(const FString& AssetPathName, FText& OutFailReason)
	{
		return CanCreateAsset( AssetPathName, ObjectType::StaticClass(), OutFailReason );
	}

	/**
	 * @param AssetPathName		Full path name of the asset to create
	 * @param AssetClass		Class of the asset to create
	 * @param OutFailReason		Text containing explanation about failure
	 * Returns true if the asset can be safely created
	 * Given a path and a class, check if an existing asset with a different class
	 * would not prevent the creation of such asset. A special report is done for object redirector
	 */
	static bool CanCreateAsset(const FString& AssetPathName, const UClass* AssetClass, FText& OutFailReason);

	/**
	 * @param AssetPathName		Full path name of the asset to create
	 * @param AssetClass		Class of the asset to create
	 * Returns a state from the creation enumeration, EAssetCreationStatus.
	 * Given a path and a class, check if an existing asset with a different class
	 * would not prevent the creation of such asset.
	 */
	static EAssetCreationStatus CanCreateAsset(const FString& AssetPathName, const UClass* AssetClass);

	/**
	 * @param AssetPathName		Full path name of the asset to create
	 * Calls CanCreateAsset(const FString&, const UClass*) with the instantiating class
	 */
	template< class ObjectType >
	static EAssetCreationStatus CanCreateAsset(const FString& AssetPathName)
	{
		return CanCreateAsset(AssetPathName, ObjectType::StaticClass());
	}

	/**
	 * Finds the UDatasmithScene for which the Asset belongs to.
	 */
	static UDatasmithScene* FindDatasmithSceneForAsset( UObject* Asset );

	static FName GetDatasmithElementId( UObject* Object );
	static FString GetDatasmithElementIdString( UObject* Object );

	/**
	 * Converts AActor objects into DatasmithActorElement objects and add them to a DatasmithScene
	 * @param SceneElement		DatasmithScene to populate
	 * @param RootActors		Array of root actors to convert and add to the DatasmithScene
	 */
	static void FillSceneElement( TSharedPtr< IDatasmithScene >& SceneElement, const TArray<AActor*>& RootActors );


	using FFunctionAndMaterialsThatUseIt = TPair< TSharedPtr< IDatasmithUEPbrMaterialElement >, TArray< TSharedPtr< IDatasmithUEPbrMaterialElement > > >;

	/**
	 * Finds all materials that are referenced by other materials in the scene and returns a list ordered
	 * by dependencies, making sure that materials referencing other materials in the list will come after.
	 * The list also include the direct parents (materials or functions) that are using the functions.
	 *
	 * @param SceneElement		DatasmithScene holding the materials.
	 */
	static TArray< FFunctionAndMaterialsThatUseIt > GetOrderedListOfMaterialsReferencedByMaterials(TSharedPtr< IDatasmithScene >& SceneElement);

	class FDatasmithMaterialImportIterator
	{
	public:
		explicit FDatasmithMaterialImportIterator(const FDatasmithImportContext& InImportContext);

		/** Advances the iterator to the next element. */
		FDatasmithMaterialImportIterator& operator++();

		/** conversion to "bool" returning true if the iterator is valid. */
		explicit operator bool() const;

		/** inverse of the "bool" operator */
		bool operator !() const
		{
			return !(bool)*this;
		}

		// Const Accessor.
		const TSharedPtr<IDatasmithBaseMaterialElement>& Value() const;

	private:
		const FDatasmithImportContext& ImportContext;
		int32 CurrentIndex;
		TArray<TSharedPtr<IDatasmithBaseMaterialElement>> SortedMaterials;
	};

	/**
	 * Convenience function duplicating an object specifically optimized for datasmith use cases
	 *
	 * @param SourceObject the object being copied
	 * @param Outer the outer to use for the object
	 * @param Name the optional name of the object
	 *
	 * @return the copied object or null if it failed for some reason
	 */
	static UObject* StaticDuplicateObject(UObject* SourceObject, UObject* Outer, const FName Name = NAME_None);

	/**
	 * Specialization of the duplication of a StaticMesh object specifically optimized for datasmith use case.
	 * This operation invalidate the duplicated SourceStaticMesh and mark it as PendingKill, unless bIgnoreBulkData is True.
	 *
	 * @param SourceStaticMesh	the UStaticMesh being copied
	 * @param Outer				the outer to use for the object
	 * @param Name				the optional name of the object
	 * @param bIgnoreBulkData	if True, the SourceStaticMesh's SourceModels BulkDatas won't be copied and SourceStaticMesh will stay valid after the operation.
	 *
	 * @return the copied StaticMesh or null if it failed for some reason
	 */
	static UStaticMesh* DuplicateStaticMesh(UStaticMesh* SourceStaticMesh, UObject* Outer, const FName Name = NAME_None, bool bIgnoreBulkData = false);

	template< class T >
	static T* DuplicateObject(T* SourceObject, UObject* Outer, const FName Name = NAME_None)
	{
		return (T*)FDatasmithImporterUtils::StaticDuplicateObject(SourceObject, Outer, Name);
	}

	static bool CreatePlmXmlSceneFromCADFiles(FString PlmXmlFileName, const TSet<FString>& FilesToProcess, TArray<FString>& FilesNotProcessed);
};

template< typename ObjectType >
struct FDatasmithFindAssetTypeHelper
{
};

template<>
struct FDatasmithFindAssetTypeHelper< UStaticMesh >
{
	static const TMap< TSharedRef< IDatasmithMeshElement >, UStaticMesh* >& GetImportedAssetsMap( const FDatasmithAssetsImportContext& AssetsContext )
	{
		return ObjectPtrDecay(AssetsContext.GetParentContext().ImportedStaticMeshes);
	}

	static UPackage* GetFinalPackage( const FDatasmithAssetsImportContext& AssetsContext )
	{
		return AssetsContext.StaticMeshesFinalPackage.Get();
	}

	static const TMap< FName, TSoftObjectPtr< UStaticMesh > >* GetAssetsMap( const UDatasmithScene* SceneAsset )
	{
		return SceneAsset ? &SceneAsset->StaticMeshes : nullptr;
	}

	static const TSharedRef<IDatasmithMeshElement>* GetImportedElementByName( const FDatasmithAssetsImportContext& AssetsContext, const TCHAR* ObjectPathName )
	{
		return AssetsContext.GetParentContext().ImportedStaticMeshesByName.Find(ObjectPathName);
	}
};

template<>
struct FDatasmithFindAssetTypeHelper< UTexture >
{
	static const TMap< TSharedRef< IDatasmithTextureElement >, UE::Interchange::FAssetImportResultRef >& GetImportedAssetsMap( const FDatasmithAssetsImportContext& AssetsContext )
	{
		return AssetsContext.GetParentContext().ImportedTextures;
	}

	static UPackage* GetFinalPackage( const FDatasmithAssetsImportContext& AssetsContext )
	{
		return AssetsContext.TexturesFinalPackage.Get();
	}

	static const TMap< FName, TSoftObjectPtr< UTexture > >* GetAssetsMap( const UDatasmithScene* SceneAsset )
	{
		return SceneAsset ? &SceneAsset->Textures : nullptr;
	}

	static const TSharedRef<IDatasmithTextureElement>* GetImportedElementByName( const FDatasmithAssetsImportContext& AssetsContext, const TCHAR* ObjectPathName )
	{
		return nullptr;
	}
};

template<>
struct FDatasmithFindAssetTypeHelper< UMaterialFunction >
{
	static const TMap< TSharedRef< IDatasmithBaseMaterialElement >, UMaterialFunction* >& GetImportedAssetsMap(const FDatasmithAssetsImportContext& AssetsContext)
	{
		return AssetsContext.GetParentContext().ImportedMaterialFunctions;
	}

	static UPackage* GetFinalPackage(const FDatasmithAssetsImportContext& AssetsContext)
	{
		return nullptr;
	}

	static const TMap< FName, TSoftObjectPtr< UMaterialFunction > >* GetAssetsMap(const UDatasmithScene* SceneAsset)
	{
		return SceneAsset ? &SceneAsset->MaterialFunctions : nullptr;
	}

	static const TSharedRef<IDatasmithBaseMaterialElement>* GetImportedElementByName( const FDatasmithAssetsImportContext& AssetsContext, const TCHAR* ObjectPathName )
	{
		return AssetsContext.GetParentContext().ImportedMaterialFunctionsByName.Find(ObjectPathName);
	}
};

template<>
struct FDatasmithFindAssetTypeHelper< UMaterialInterface >
{
	static const TMap< TSharedRef< IDatasmithBaseMaterialElement >, UMaterialInterface* >& GetImportedAssetsMap( const FDatasmithAssetsImportContext& AssetsContext )
	{
		return ObjectPtrDecay(AssetsContext.GetParentContext().ImportedMaterials);
	}

	static UPackage* GetFinalPackage( const FDatasmithAssetsImportContext& AssetsContext )
	{
		return AssetsContext.MaterialsFinalPackage.Get();
	}

	static const TMap< FName, TSoftObjectPtr< UMaterialInterface > >* GetAssetsMap( const UDatasmithScene* SceneAsset )
	{
		return SceneAsset ? &SceneAsset->Materials : nullptr;
	}

	static const TSharedRef<IDatasmithBaseMaterialElement>* GetImportedElementByName( const FDatasmithAssetsImportContext& AssetsContext, const TCHAR* ObjectPathName )
	{
		return nullptr;
	}
};

template<>
struct FDatasmithFindAssetTypeHelper< ULevelSequence >
{
	static UPackage* GetImportPackage( const FDatasmithAssetsImportContext& AssetsContext )
	{
		return AssetsContext.LevelSequencesImportPackage.Get();
	}

	static UPackage* GetFinalPackage( const FDatasmithAssetsImportContext& AssetsContext )
	{
		return AssetsContext.LevelSequencesFinalPackage.Get();
	}

	static const TMap< FName, TSoftObjectPtr< ULevelSequence > >* GetAssetsMap( const UDatasmithScene* SceneAsset )
	{
		return SceneAsset ? &SceneAsset->LevelSequences : nullptr;
	}
};

template<>
struct FDatasmithFindAssetTypeHelper< ULevelVariantSets >
{
	static UPackage* GetImportPackage( const FDatasmithAssetsImportContext& AssetsContext )
	{
		return AssetsContext.LevelVariantSetsImportPackage.Get();
	}

	static UPackage* GetFinalPackage( const FDatasmithAssetsImportContext& AssetsContext )
	{
		return AssetsContext.LevelVariantSetsFinalPackage.Get();
	}

	static const TMap< FName, TSoftObjectPtr< ULevelVariantSets > >* GetAssetsMap( const UDatasmithScene* SceneAsset )
	{
		return SceneAsset ? &SceneAsset->LevelVariantSets : nullptr;
	}
};

template< typename ObjectType >
inline ObjectType* FDatasmithImporterUtils::FindAsset( const FDatasmithAssetsImportContext& AssetsContext, const TCHAR* ObjectPathName )
{
	if ( FCString::Strlen( ObjectPathName ) <= 0 )
	{
		return nullptr;
	}

	if ( FPaths::IsRelative( ObjectPathName ) )
	{
		const auto& ImportedElement   = FDatasmithFindAssetTypeHelper< ObjectType >::GetImportedElementByName( AssetsContext, ObjectPathName );
		const auto& ImportedAssetsMap = FDatasmithFindAssetTypeHelper< ObjectType >::GetImportedAssetsMap( AssetsContext );

		if (ImportedElement)
		{
			auto Result = ImportedAssetsMap.Find(*ImportedElement);
			if (Result)
			{
				return *Result;
			}
		}
		else
		{
			for ( const auto& ImportedAssetPair : ImportedAssetsMap )
			{
				if ( FCString::Stricmp( ImportedAssetPair.Key->GetName(), ObjectPathName ) == 0 )
				{
					return ImportedAssetPair.Value;
				}
			}
		}

		{
			const auto* AssetsMap = FDatasmithFindAssetTypeHelper< ObjectType >::GetAssetsMap( AssetsContext.GetParentContext().SceneAsset );

			// Check if the AssetsMap is already tracking our asset
			if ( AssetsMap && AssetsMap->Contains( ObjectPathName ) )
			{
				return (*AssetsMap)[ FName( ObjectPathName ) ].LoadSynchronous();
			}
			else
			{
				UPackage* FinalPackage = FDatasmithFindAssetTypeHelper< ObjectType >::GetFinalPackage( AssetsContext );
				return FindObject< ObjectType >( FinalPackage, ObjectPathName );
			}
		}
	}
	else
	{
		return FindObject< ObjectType >( nullptr, ObjectPathName );
	}
}

template<>
inline UTexture* FDatasmithImporterUtils::FindAsset< UTexture >( const FDatasmithAssetsImportContext& AssetsContext, const TCHAR* ObjectPathName )
{
	if ( FCString::Strlen( ObjectPathName ) <= 0 )
	{
		return nullptr;
	}

	if ( FPaths::IsRelative( ObjectPathName ) )
	{
		const TSharedRef< IDatasmithTextureElement >* ImportedElement   = FDatasmithFindAssetTypeHelper< UTexture >::GetImportedElementByName( AssetsContext, ObjectPathName );
		const TMap< TSharedRef< IDatasmithTextureElement >, UE::Interchange::FAssetImportResultRef >& ImportedAssetsMap = FDatasmithFindAssetTypeHelper< UTexture >::GetImportedAssetsMap( AssetsContext );

		if (ImportedElement)
		{
			UE::Interchange::FAssetImportResultRef* Result =  const_cast<UE::Interchange::FAssetImportResultRef* >( ImportedAssetsMap.Find(*ImportedElement) );
			ensure(Result);
			if (Result)
			{
				(*Result)->WaitUntilDone();
				return Cast< UTexture >( (*Result)->GetFirstAssetOfClass( UTexture::StaticClass() ) );
			}
		}
		else
		{
			for ( const TPair< TSharedRef< IDatasmithTextureElement >, UE::Interchange::FAssetImportResultRef >& ImportedAssetPair : ImportedAssetsMap )
			{
				if ( FCString::Stricmp( ImportedAssetPair.Key->GetName(), ObjectPathName ) == 0 )
				{
					UE::Interchange::FAssetImportResultRef& Result = const_cast<UE::Interchange::FAssetImportResultRef& >( ImportedAssetPair.Value );
					Result->WaitUntilDone();

					return Cast< UTexture >( Result->GetFirstAssetOfClass( UTexture::StaticClass() ) );
				}
			}
		}

		{
			const TMap< FName, TSoftObjectPtr< UTexture > >* AssetsMap = FDatasmithFindAssetTypeHelper< UTexture >::GetAssetsMap( AssetsContext.ParentContext->SceneAsset );

			// Check if the AssetsMap is already tracking our asset
			if ( AssetsMap && AssetsMap->Contains( ObjectPathName ) )
			{
				return (*AssetsMap)[ FName( ObjectPathName ) ].LoadSynchronous();
			}
			else
			{
				UPackage* FinalPackage = FDatasmithFindAssetTypeHelper< UTexture >::GetFinalPackage( AssetsContext );
				return FindObject< UTexture >( FinalPackage, ObjectPathName );
			}
		}
	}
	else
	{
		return FindObject< UTexture >( nullptr, ObjectPathName );
	}
}

template< class ObjectType >
inline ObjectType* FDatasmithImporterUtils::FindObject( const UPackage* ParentPackage, const FString& ObjectName )
{
	if ( ObjectName.Len() <= 0 )
	{
		return nullptr;
	}

	FString PathName = ObjectName;
	if ( FPaths::IsRelative( PathName ) && ParentPackage )
	{
		PathName = FPaths::Combine( ParentPackage->GetPathName(), ObjectName );
	}

	FSoftObjectPath ObjectPath( PathName );

	// Find the package
	FString LongPackageName = ObjectPath.GetAssetName().IsEmpty() ? ObjectPath.ToString() : ObjectPath.GetLongPackageName();

	// Look for the package in memory
	UPackage* Package = FindPackage( nullptr, *LongPackageName );

	// Look for the package on disk
	if ( !Package && FPackageName::DoesPackageExist( LongPackageName ) )
	{
		Package = LoadPackage( nullptr, *LongPackageName, LOAD_None );
	}

	ObjectType* Object = nullptr;

	if ( Package )
	{
		Package->FullyLoad();

		Object = static_cast< ObjectType* >( FindObjectWithOuter( Package, ObjectType::StaticClass(), FName( *ObjectPath.GetAssetName() ) ) );

		// The object might have been moved away from the ParentPackage but still accessible through an object redirector, so try to load with the SoftObjectPath
		// Note that the object redirector itself is in the Package at the initial location of import
		// No Package means we are trying to find a new object, so don't need to try loading it
		if ( !Object )
		{
			Object = Cast<ObjectType>( ObjectPath.TryLoad() );
		}
	}

	return Object;
}

template< class ObjectType >
bool FDatasmithImporterUtils::CanCreateAsset(const UPackage* Package, const FString& AssetName, FText& OutFailReason)
{
	FString ObjectPath = FPaths::Combine( Package->GetPathName(), AssetName ).AppendChar(L'.').Append(AssetName);
	return CanCreateAsset( ObjectPath, ObjectType::StaticClass(), OutFailReason );
}
