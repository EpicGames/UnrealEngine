// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DatasmithUtils.h"

class DATASMITHIMPORTER_API FDatasmithTextureResize
{
public:
#if WITH_FREEIMAGE_LIB
	/**
	 * Must be called from the main thread before doing any parallel work.
	 */
	static void Initialize();

	/**
	 * Returns true if image extension is supported by Unreal Engine
	 * IMPORTANT: Extension is expected with the starting dot
	 */
	static bool IsSupportedTextureExtension(const FString& Extension);

	/**
	 * Returns true if image extension is supported by Unreal Engine or image can be converted to a supported format
	 * IMPORTANT: Extension is expected with the starting dot
	 */
	static bool GetBestTextureExtension(const TCHAR* Source, FString& Extension);

	/**
	 * Resizes texture according to resizing method
	 * @param Source				The source file of the texture to resize
	 * @param Destination			The path where to save the resized texture file
	 * @param Mode					The resize mode
	 * @param bGenerateNormalMap	Generate or not the normal map
	 */
	static EDSTextureUtilsError ResizeTexture( const TCHAR* Source, const TCHAR* Destination, EDSResizeTextureMode Mode, uint32 MaxSize, bool bGenerateNormalMap );

#else
	/**
	 * Must be called from the main thread before doing any parallel work.
	 */
	static void Initialize()
	{ }

	/**
	* Returns true if image extension is supported by Unreal Engine
	* IMPORTANT: Extension is expected with the starting dot
	*/
	static bool IsSupportedTextureExtension(const FString& Extension)
	{ return false; }

	/**
	* Returns true if image extension is supported by Unreal Engine or image can be converted to a supported format
	* IMPORTANT: Extension is expected with the starting dot
	*/
	static bool GetBestTextureExtension(const TCHAR* Source, FString& Extension)
	{ return false; }

	/**
	* Resizes texture according to resizing method
	* @param Source				The source file of the texture to resize
	* @param Destination			The path where to save the resized texture file
	* @param Mode					The resize mode
	* @param bGenerateNormalMap	Generate or not the normal map
	*/
	static EDSTextureUtilsError ResizeTexture( const TCHAR* Source, const TCHAR* Destination, EDSResizeTextureMode Mode, uint32 MaxSize, bool bGenerateNormalMap )
	{ return EDSTextureUtilsError::ResizeFailed; }
#endif // WITH_FREEIMAGE_LIB
};
