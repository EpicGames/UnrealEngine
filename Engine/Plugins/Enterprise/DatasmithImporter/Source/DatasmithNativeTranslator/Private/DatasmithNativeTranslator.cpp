// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithNativeTranslator.h"

#include "DatasmithAnimationElements.h"
#include "DatasmithAnimationSerializer.h"
#include "DatasmithMeshSerialization.h"
#include "DatasmithSceneSource.h"
#include "DatasmithSceneXmlReader.h"
#include "IDatasmithSceneElements.h"

#include "Misc/Paths.h"
#include "HAL/FileManager.h"


FString FDatasmithNativeTranslator::ResolveFilePath(const FString& FilePath, const TArray<FString>& ResourcePaths)
{
	if (FPaths::IsRelative(FilePath) && !FPaths::FileExists(FilePath))
	{
		for (const FString& ResourcePath : ResourcePaths)
		{
			if (ResourcePath.IsEmpty())
			{
				continue;
			}

			FString NewFilePath = ResourcePath / FilePath;
			if (FPaths::FileExists(NewFilePath))
			{
				return NewFilePath;
			}
		}
	}
	return FilePath;
}

void FDatasmithNativeTranslator::ResolveSceneFilePaths(TSharedRef<IDatasmithScene> Scene, const TArray<FString>& ResourcePaths)
{
	for (int32 Index = 0; Index < Scene->GetMeshesCount(); ++Index)
	{
		const TSharedPtr<IDatasmithMeshElement>& Mesh = Scene->GetMesh(Index);
		const TCHAR* Path = Mesh->GetFile();
		Mesh->SetFile(*ResolveFilePath(Path, ResourcePaths));
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	for (int32 Index = 0; Index < Scene->GetClothesCount(); ++Index)  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	{
		const TSharedPtr<IDatasmithClothElement>& Cloth = Scene->GetCloth(Index);
		const TCHAR* Path = Cloth->GetFile();
		Cloth->SetFile(*ResolveFilePath(Path, ResourcePaths));
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	for (int32 Index = 0; Index < Scene->GetTexturesCount(); ++Index)
	{
		const TSharedPtr<IDatasmithTextureElement>& Tex = Scene->GetTexture(Index);
		const TCHAR* Path = Tex->GetFile();
		Tex->SetFile(*ResolveFilePath(Path, ResourcePaths));
	}

	for (int32 Index = 0; Index < Scene->GetLevelSequencesCount(); ++Index)
	{
		const TSharedPtr<IDatasmithLevelSequenceElement>& Sequence = Scene->GetLevelSequence(Index);
		const TCHAR* Path = Sequence->GetFile();
		Sequence->SetFile(*ResolveFilePath(Path, ResourcePaths));
	}

	TFunction<void (const TSharedPtr<IDatasmithActorElement> Actor)> VisitActorTree;
	VisitActorTree = [&](const TSharedPtr<IDatasmithActorElement> Actor) -> void
	{
		if (Actor->IsA(EDatasmithElementType::Landscape))
		{
			const TSharedPtr<IDatasmithLandscapeElement>& LandscapeActor = StaticCastSharedPtr<IDatasmithLandscapeElement>(Actor);
			const TCHAR* Path = LandscapeActor->GetHeightmap();
			LandscapeActor->SetHeightmap(*ResolveFilePath(Path, ResourcePaths));
		}
		else if (Actor->IsA(EDatasmithElementType::Light))
		{
			const TSharedPtr<IDatasmithLightActorElement>& Light = StaticCastSharedPtr<IDatasmithLightActorElement>(Actor);
			const TCHAR* Path = Light->GetIesFile();
			Light->SetIesFile(*ResolveFilePath(Path, ResourcePaths));
		}

		for (int32 ChildIndex = 0; ChildIndex < Actor->GetChildrenCount(); ++ChildIndex)
		{
			VisitActorTree(Actor->GetChild(ChildIndex));
		}
	};

	for (int32 Index = 0; Index < Scene->GetActorsCount(); ++Index)
	{
		VisitActorTree(Scene->GetActor(Index));
	}
}

void FDatasmithNativeTranslator::Initialize(FDatasmithTranslatorCapabilities& OutCapabilities)
{
	OutCapabilities.SupportedFileFormats.Emplace(TEXT("udatasmith"), TEXT("Datasmith files"));
	OutCapabilities.bParallelLoadStaticMeshSupported = true;
}

bool FDatasmithNativeTranslator::LoadScene(TSharedRef<IDatasmithScene> OutScene)
{
	FDatasmithSceneXmlReader XmlParser;

	bool bParsingResult = XmlParser.ParseFile(GetSource().GetSourceFile(), OutScene);
	if (bParsingResult)
	{
		TArray<FString> ResourcePaths;
		FString ResourcePath = OutScene->GetResourcePath();
		ResourcePath.ParseIntoArray(ResourcePaths, TEXT(";"));

		FString ProjectPath = FPaths::GetPath(GetSource().GetSourceFile());
		ResourcePaths.Insert(ProjectPath, 0);

		ResolveSceneFilePaths(OutScene, ResourcePaths);
	}

	return bParsingResult;
}

bool FDatasmithNativeTranslator::LoadStaticMesh(const TSharedRef<IDatasmithMeshElement> MeshElement, FDatasmithMeshElementPayload& OutMeshPayload)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithNativeTranslator::LoadStaticMesh);

	FString FilePath = MeshElement->GetFile();

	FDatasmithPackedMeshes Pack = GetDatasmithMeshFromFile(FilePath);
	for (FDatasmithMeshModels& DatasmithMesh : Pack.Meshes)
	{
		if (DatasmithMesh.bIsCollisionMesh)
		{
			if (DatasmithMesh.SourceModels.Num() > 0)
			{
				FMeshDescription& MeshDescription = DatasmithMesh.SourceModels[0];
				OutMeshPayload.CollisionMesh = MoveTemp(MeshDescription);
			}
		}
		else
		{
			for (FMeshDescription& SourceModel : DatasmithMesh.SourceModels)
			{
				OutMeshPayload.LodMeshes.Add(MoveTemp(SourceModel));
			}
		}
	}

	return OutMeshPayload.LodMeshes.Num() != 0;
}

bool FDatasmithNativeTranslator::LoadLevelSequence(const TSharedRef<IDatasmithLevelSequenceElement> LevelSequenceElement, FDatasmithLevelSequencePayload& OutLevelSequencePayload)
{
	// #ueent_todo: this totally skip the payload system....
	// Parse the level sequences from file
	FDatasmithAnimationSerializer AnimSerializer;
	if (LevelSequenceElement->GetFile() && IFileManager::Get().FileExists(LevelSequenceElement->GetFile()))
	{
		return AnimSerializer.Deserialize(LevelSequenceElement, LevelSequenceElement->GetFile());
	}
	return false;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool FDatasmithNativeTranslator::LoadCloth(const TSharedRef<IDatasmithClothElement> ClothElement, FDatasmithClothElementPayload& OutClothPayload)  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithNativeTranslator::LoadCloth);

	FString FilePath = ClothElement->GetFile();
	FDatasmithPackedCloths Pack = GetDatasmithClothFromFile(FilePath);

	if (Pack.ClothInfos.IsValidIndex(0))
	{
		OutClothPayload.Cloth = MoveTemp(Pack.ClothInfos[0].Cloth);
		return true;
	}

	return false;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

