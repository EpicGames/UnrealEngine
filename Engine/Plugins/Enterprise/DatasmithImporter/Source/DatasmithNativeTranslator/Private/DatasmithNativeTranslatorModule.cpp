// Copyright Epic Games, Inc. All Rights Reserved.
#include "DatasmithNativeTranslatorModule.h"

#include "DatasmithNativeTranslator.h"
#include "DatasmithTranslator.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FDatasmithNativeTranslatorModule, DatasmithNativeTranslator);

void FDatasmithNativeTranslatorModule::StartupModule()
{
	Datasmith::RegisterTranslator<FDatasmithNativeTranslator>();
}

void FDatasmithNativeTranslatorModule::ShutdownModule()
{
	Datasmith::UnregisterTranslator<FDatasmithNativeTranslator>();
}
