// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DatasmithTranslator.h"


class DATASMITHNATIVETRANSLATOR_API FDatasmithNativeTranslator : public IDatasmithTranslator
{
public:
	virtual FName GetFName() const override { return "DatasmithNativeTranslator"; };

	virtual void Initialize(FDatasmithTranslatorCapabilities& OutCapabilities) override;

	virtual bool LoadScene(TSharedRef<IDatasmithScene> OutScene) override;

	virtual bool LoadStaticMesh(const TSharedRef<IDatasmithMeshElement> MeshElement, FDatasmithMeshElementPayload& OutMeshPayload) override;

	virtual bool LoadLevelSequence(const TSharedRef<IDatasmithLevelSequenceElement> LevelSequenceElement, FDatasmithLevelSequencePayload& OutLevelSequencePayload) override;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	virtual bool LoadCloth(const TSharedRef<IDatasmithClothElement> ClothElement, FDatasmithClothElementPayload& OutClothPayload) override;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

protected:
	static FString ResolveFilePath(const FString& FilePath, const TArray<FString>& ResourcePaths);
	static void ResolveSceneFilePaths(TSharedRef<IDatasmithScene> Scene, const TArray<FString>& ResourcePaths);
};

