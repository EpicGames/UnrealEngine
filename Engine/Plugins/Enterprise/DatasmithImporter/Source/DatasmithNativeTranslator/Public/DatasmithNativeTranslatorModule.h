// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#define DATASMITHNATIVETRANSLATOR_MODULE_NAME TEXT("DatasmithNativeTranslator")

/**
 * Datasmith Translator for .udatasmith files.
 */
class FDatasmithNativeTranslatorModule : public IModuleInterface
{
public:

    static FDatasmithNativeTranslatorModule& Get()
    {
        return FModuleManager::LoadModuleChecked< FDatasmithNativeTranslatorModule >(DATASMITHNATIVETRANSLATOR_MODULE_NAME);
    }

    static bool IsAvailable()
    {
        return FModuleManager::Get().IsModuleLoaded(DATASMITHNATIVETRANSLATOR_MODULE_NAME);
    }

	virtual void StartupModule() override;

	virtual void ShutdownModule() override;
};
