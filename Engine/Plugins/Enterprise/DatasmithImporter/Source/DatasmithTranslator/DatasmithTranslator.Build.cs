// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class DatasmithTranslator : ModuleRules
	{
		public DatasmithTranslator(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"DatasmithCore",
					"DatasmithContent",
					"XmlParser",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Engine",
					"MeshDescription",
					"RawMesh",
					"StaticMeshDescription",
				}
			);
		}
	}
}
