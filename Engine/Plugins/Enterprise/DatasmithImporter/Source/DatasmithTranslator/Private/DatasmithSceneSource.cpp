// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithSceneSource.h"

#include "DatasmithTranslator.h"
#include "DatasmithTranslatorManager.h"
#include "DatasmithUtils.h"

#include "Misc/Paths.h"

#define LOCTEXT_NAMESPACE "DatasmithSceneSource"

void FDatasmithSceneSource::SetSourceFile(const FString& InFilePath)
{
	FilePath = InFilePath;
	FDatasmithUtils::GetCleanFilenameAndExtension(FilePath, SceneDeducedName, FileExtension);
}

void FDatasmithSceneSource::SetSceneName(const FString& InSceneName)
{
	SceneOverrideName = InSceneName;
}

const FString& FDatasmithSceneSource::GetSceneName() const
{
	return SceneOverrideName.IsEmpty() ? SceneDeducedName : SceneOverrideName;
}

#undef LOCTEXT_NAMESPACE
