// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithTranslatableSource.h"
#include "DatasmithTranslator.h"
#include "DatasmithTranslatorManager.h"

#define LOCTEXT_NAMESPACE "DatasmithTranslatableSceneSource"


FDatasmithSceneGuard::FDatasmithSceneGuard(const TSharedPtr<IDatasmithTranslator>& Translator, const TSharedRef<IDatasmithScene>& Scene, bool& bOutLoadOk)
	: Translator(Translator)
{
	bOutLoadOk = Translator.IsValid() && Translator->LoadScene(Scene);
}

FDatasmithSceneGuard::~FDatasmithSceneGuard()
{
	if (Translator.IsValid())
	{
		Translator->UnloadScene();
	}
}

FDatasmithTranslatableSceneSource::FDatasmithTranslatableSceneSource(const FDatasmithSceneSource& Source)
	: Translator(nullptr)
	, SceneGuard(nullptr)
{
	Translator = FDatasmithTranslatorManager::Get().SelectFirstCompatible(Source);
	if (Translator)
	{
		Translator->SetSource(Source);
	}
}

bool FDatasmithTranslatableSceneSource::IsTranslatable() const
{
	return Translator.IsValid();
}

bool FDatasmithTranslatableSceneSource::Translate(TSharedRef< IDatasmithScene > Scene)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithTranslatableSceneSource::Translate);

	bool bLoadedOk = false;
	bool bIsAlreadyLoaded = SceneGuard.IsValid();
	if (IsTranslatable() && !bIsAlreadyLoaded)
	{
		SceneGuard.Reset(new FDatasmithSceneGuard(Translator, Scene, bLoadedOk));
	}
	return bLoadedOk;
}

TSharedPtr<IDatasmithTranslator> FDatasmithTranslatableSceneSource::GetTranslator() const
{
	return IsTranslatable() ? Translator : nullptr;
}

FDatasmithTranslatableSceneSource::~FDatasmithTranslatableSceneSource()
{
	// prevent smart pointers dtr inlining
}

#undef LOCTEXT_NAMESPACE
