// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithTranslator.h"
#include "DatasmithTranslatorManager.h"

#include "XmlFile.h"

#define LOCTEXT_NAMESPACE "DatasmithTranslator"

namespace Datasmith
{

	void Details::RegisterTranslatorImpl(const FTranslatorRegisterInformation& Info)
	{
		FDatasmithTranslatorManager::Get().Register(Info);
	}

	void Details::UnregisterTranslatorImpl(const FTranslatorRegisterInformation& Info)
	{
		FDatasmithTranslatorManager::Get().Unregister(Info.TranslatorName);
	}

	FString GetXMLFileSchema(const FString& XmlFilePath)
	{
		FXmlFile XmlFile;

		if (!XmlFile.LoadFile(XmlFilePath))
		{
			return FString();
		}

		const FXmlNode* RootNode = XmlFile.GetRootNode();
		if (!RootNode)
		{
			return FString();
		}
		FString Tag = RootNode->GetTag();
		return Tag;
	}

} // ns Datasmith


#undef LOCTEXT_NAMESPACE
