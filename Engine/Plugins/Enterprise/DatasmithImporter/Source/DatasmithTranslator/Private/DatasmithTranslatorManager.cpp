// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithTranslatorManager.h"

#include "DatasmithTranslator.h"

#include "Misc/Paths.h"
#include "UObject/Class.h"
#include "UObject/UObjectIterator.h"
#include "DatasmithSceneSource.h"

#define LOCTEXT_NAMESPACE "DatasmithTranslatorManager"


FDatasmithTranslatorManager& FDatasmithTranslatorManager::Get()
{
	static FDatasmithTranslatorManager Instance;
	return Instance;
}

const TArray<FString>& FDatasmithTranslatorManager::GetSupportedFormats() const
{
	if (Formats.Num() == 0)
	{
		for (const Datasmith::FTranslatorRegisterInformation& ImplInfo : RegisteredTranslators)
		{
			if (TSharedPtr<IDatasmithTranslator> Impl = ImplInfo.SpawnFunction ? ImplInfo.SpawnFunction() : nullptr)
			{
				if (ensure(Impl->GetFName() != NAME_None))
				{
					FDatasmithTranslatorCapabilities Capabilities;
					Impl->Initialize(Capabilities);
					if (Capabilities.bIsEnabled)
					{
						for (const FFileFormatInfo& SupportedFormat : Capabilities.SupportedFileFormats)
						{
							FString FormatString = SupportedFormat.Extension + TEXT(';') + SupportedFormat.Description;
							Formats.Add(FormatString);
						}
					}
				}
			}
		}
	}
	return Formats;
}

TSharedPtr<IDatasmithTranslator> FDatasmithTranslatorManager::SelectFirstCompatible(const FDatasmithSceneSource& Source)
{
	FString FileExtension = FPaths::GetExtension(Source.GetSourceFile());
	FString SecondFileExtension = FPaths::GetExtension(Source.GetSourceFile().LeftChop(FileExtension.Len()+1));

	for (const Datasmith::FTranslatorRegisterInformation& ImplInfo : RegisteredTranslators)
	{
		if (TSharedPtr<IDatasmithTranslator> Impl = ImplInfo.SpawnFunction ? ImplInfo.SpawnFunction() : nullptr)
		{
			FDatasmithTranslatorCapabilities Capabilities;
			Impl->Initialize(Capabilities);
			if (Capabilities.bIsEnabled)
			{
				for (const FFileFormatInfo& SupportedFormat : Capabilities.SupportedFileFormats)
				{
					if (SupportedFormat.Extension == FileExtension || (!SecondFileExtension.IsEmpty() && SupportedFormat.Extension.EndsWith(TEXT(".*")) && SupportedFormat.Extension.StartsWith(SecondFileExtension)))
					{
						if (Impl->IsSourceSupported(Source))
						{
							return Impl;
						}
					}
				}
			}
		}
	}

	return nullptr;
}

void FDatasmithTranslatorManager::Register(const Datasmith::FTranslatorRegisterInformation& Info)
{
	RegisteredTranslators.Add(Info);
	InvalidateCache();
}

void FDatasmithTranslatorManager::Unregister(FName TranslatorName)
{
	RegisteredTranslators.RemoveAll(
		[&TranslatorName](const Datasmith::FTranslatorRegisterInformation& Info)
		{
			return Info.TranslatorName == TranslatorName;
		}
	);

	InvalidateCache();
}

void FDatasmithTranslatorManager::InvalidateCache()
{
	Formats.Reset();
}

#undef LOCTEXT_NAMESPACE