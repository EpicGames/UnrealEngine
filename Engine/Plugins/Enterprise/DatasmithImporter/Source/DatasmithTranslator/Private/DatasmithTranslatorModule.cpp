// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithTranslatorModule.h"

#include "ReferenceMaterials/DatasmithC4DImporterMaterialSelector.h"
#include "ReferenceMaterials/DatasmithCityEngineMaterialSelector.h"
#include "ReferenceMaterials/DatasmithReferenceMaterialManager.h"
#include "ReferenceMaterials/DatasmithRevitMaterialSelector.h"
#include "ReferenceMaterials/DatasmithSketchupMaterialSelector.h"
#include "ReferenceMaterials/DatasmithStdMaterialSelector.h"

void IDatasmithTranslatorModule::StartupModule()
{
	FDatasmithReferenceMaterialManager::Create();

	//A minimal set of natively supported reference materials
	FDatasmithReferenceMaterialManager::Get().RegisterSelector(TEXT("C4DTranslator"), MakeShared< FDatasmithC4DImporterMaterialSelector >());
	FDatasmithReferenceMaterialManager::Get().RegisterSelector(TEXT("Revit"), MakeShared< FDatasmithRevitMaterialSelector >());
	FDatasmithReferenceMaterialManager::Get().RegisterSelector(TEXT("SketchUp"), MakeShared< FDatasmithSketchUpMaterialSelector >());
	FDatasmithReferenceMaterialManager::Get().RegisterSelector(TEXT("CityEngine"), MakeShared< FDatasmithCityEngineMaterialSelector >());
	FDatasmithReferenceMaterialManager::Get().RegisterSelector(TEXT("StdMaterial"), MakeShared< FDatasmithStdMaterialSelector >());
}

void IDatasmithTranslatorModule::ShutdownModule()
{
	FDatasmithReferenceMaterialManager::Get().UnregisterSelector(TEXT("C4DTranslator"));
	FDatasmithReferenceMaterialManager::Get().UnregisterSelector(TEXT("Revit"));
	FDatasmithReferenceMaterialManager::Get().UnregisterSelector(TEXT("SketchUp"));
	FDatasmithReferenceMaterialManager::Get().UnregisterSelector(TEXT("CityEngine"));
	FDatasmithReferenceMaterialManager::Get().UnregisterSelector(TEXT("StdMaterial"));

	FDatasmithReferenceMaterialManager::Destroy();
}

IMPLEMENT_MODULE(IDatasmithTranslatorModule, DatasmithTranslator);