// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithC4DImporterMaterialSelector.h"

#include "IDatasmithSceneElements.h"

#include "Materials/Material.h"
#include "Materials/MaterialInstanceConstant.h"
#include "UObject/SoftObjectPath.h"
#include "Templates/Casts.h"

FDatasmithC4DImporterMaterialSelector::FDatasmithC4DImporterMaterialSelector()
{
	ReferenceMaterial.FromSoftObjectPath( FSoftObjectPath("/DatasmithContent/Materials/C4DReference.C4DReference") );
}

bool FDatasmithC4DImporterMaterialSelector::IsValid() const
{
	return ReferenceMaterial.IsValid();
}

const FDatasmithReferenceMaterial& FDatasmithC4DImporterMaterialSelector::GetReferenceMaterial( const TSharedPtr< IDatasmithMaterialInstanceElement >& /*InDatasmithMaterial*/ ) const
{
	return ReferenceMaterial;
}

void FDatasmithC4DImporterMaterialSelector::FinalizeMaterialInstance(const TSharedPtr<IDatasmithMaterialInstanceElement>& InDatasmithMaterial, UMaterialInstanceConstant * MaterialInstance) const
{
	if (!InDatasmithMaterial.IsValid() || MaterialInstance == nullptr)
	{
		return;
	}

	// Set blend mode to translucent if material requires transparency.
	if (InDatasmithMaterial->GetMaterialType() == EDatasmithReferenceMaterialType::Transparent)
	{
		MaterialInstance->BasePropertyOverrides.bOverride_BlendMode = true;
		MaterialInstance->BasePropertyOverrides.BlendMode = BLEND_Translucent;
	}
	// Set blend mode to masked if material has cutouts.
	else if (InDatasmithMaterial->GetMaterialType() == EDatasmithReferenceMaterialType::CutOut)
	{
		MaterialInstance->BasePropertyOverrides.bOverride_BlendMode = true;
		MaterialInstance->BasePropertyOverrides.BlendMode = BLEND_Masked;
	}
}