// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithCityEngineMaterialSelector.h"

#include "IDatasmithSceneElements.h"

#include "Materials/Material.h"
#include "UObject/SoftObjectPath.h"
#include "Templates/Casts.h"

FDatasmithCityEngineMaterialSelector::FDatasmithCityEngineMaterialSelector()
{
	// Reference
	ReferenceMaterial.FromSoftObjectPath( FSoftObjectPath("/DatasmithContent/Materials/CE_OpaqueReference.CE_OpaqueReference") );
	ReferenceMaterialTransparent.FromSoftObjectPath( FSoftObjectPath("/DatasmithContent/Materials/CE_OpacityReference.CE_OpacityReference") );
	ReferenceMaterialTransparentSimple.FromSoftObjectPath( FSoftObjectPath("/DatasmithContent/Materials/CE_OpacitySimpleReference.CE_OpacitySimpleReference") );
}

bool FDatasmithCityEngineMaterialSelector::IsValid() const
{
	return ReferenceMaterial.IsValid() && ReferenceMaterialTransparent.IsValid() && ReferenceMaterialTransparentSimple.IsValid();
}

const FDatasmithReferenceMaterial& FDatasmithCityEngineMaterialSelector::GetReferenceMaterial( const TSharedPtr< IDatasmithMaterialInstanceElement >& InDatasmithMaterial ) const
{
	EDatasmithReferenceMaterialType MaterialType = InDatasmithMaterial->GetMaterialType();

	if ( !IsValidMaterialType( MaterialType ) )
	{
		MaterialType = EDatasmithReferenceMaterialType::Auto;
	}

	bool bIsTransparent = MaterialType == EDatasmithReferenceMaterialType::Transparent;

	if ( MaterialType == EDatasmithReferenceMaterialType::Auto )
	{
		const TSharedPtr< IDatasmithKeyValueProperty > OpacityProperty = InDatasmithMaterial->GetPropertyByName(TEXT("Opacity"));
		const TSharedPtr< IDatasmithKeyValueProperty > OpacityMapProperty = InDatasmithMaterial->GetPropertyByName(TEXT("OpacityMap"));

		float OpacityValue;
		bIsTransparent = ( OpacityProperty.IsValid() && GetFloat( OpacityProperty, OpacityValue ) && OpacityValue < 1.f );

		FString OpacityMap;
		bIsTransparent = bIsTransparent || ( OpacityMapProperty.IsValid() && GetTexture( OpacityMapProperty, OpacityMap ) && !OpacityMap.IsEmpty() );
	}

	if ( bIsTransparent )
	{
		if ( InDatasmithMaterial->GetQuality() == EDatasmithReferenceMaterialQuality::Low )
		{
			return ReferenceMaterialTransparentSimple;
		}
		else
		{
			return ReferenceMaterialTransparent;
		}
	}

	return ReferenceMaterial;
}

bool FDatasmithCityEngineMaterialSelector::IsValidMaterialType( EDatasmithReferenceMaterialType InType ) const
{
	return InType == EDatasmithReferenceMaterialType::Auto || InType == EDatasmithReferenceMaterialType::Opaque || InType == EDatasmithReferenceMaterialType::Transparent;
}
