// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DatasmithDefinitions.h"
#include "ReferenceMaterials/DatasmithReferenceMaterial.h"
#include "ReferenceMaterials/DatasmithReferenceMaterialSelector.h"

#include "Templates/SharedPointer.h"

class IDatasmithMaterialInstanceElement;

class FDatasmithCityEngineMaterialSelector : public FDatasmithReferenceMaterialSelector
{
public:
	FDatasmithCityEngineMaterialSelector();

	virtual bool IsValid() const override;
	virtual const FDatasmithReferenceMaterial& GetReferenceMaterial( const TSharedPtr< IDatasmithMaterialInstanceElement >& InDatasmithMaterial ) const override;

protected:
	bool IsValidMaterialType( EDatasmithReferenceMaterialType InType ) const;

private:
	FDatasmithReferenceMaterial ReferenceMaterial;
	FDatasmithReferenceMaterial ReferenceMaterialTransparent;
	FDatasmithReferenceMaterial ReferenceMaterialTransparentSimple;
};
