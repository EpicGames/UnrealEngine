// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReferenceMaterials/DatasmithReferenceMaterial.h"

#include "Materials/Material.h"

#include "UObject/SoftObjectPath.h"

FDatasmithReferenceMaterial::FDatasmithReferenceMaterial()
	: Material( nullptr )
{
}

void FDatasmithReferenceMaterial::FromMaterial( UMaterialInterface* InMaterial )
{
	if ( InMaterial )
	{
		TArray< FGuid > ParameterIds;

		TArray< FMaterialParameterInfo > VectorParameterInfo;
		InMaterial->GetAllVectorParameterInfo(VectorParameterInfo, ParameterIds);
		for ( const FMaterialParameterInfo& Info : VectorParameterInfo )
		{
			VectorParams.Add( Info.Name.ToString() );
		}

		TArray< FMaterialParameterInfo > ScalarParameterInfo;
		InMaterial->GetAllScalarParameterInfo(ScalarParameterInfo, ParameterIds);
		for (const FMaterialParameterInfo& Info : ScalarParameterInfo)
		{
			ScalarParams.Add(Info.Name.ToString());
		}

		TArray< FMaterialParameterInfo > TextureParameterInfo;
		InMaterial->GetAllTextureParameterInfo(TextureParameterInfo, ParameterIds);
		for (const FMaterialParameterInfo& Info : TextureParameterInfo)
		{
			TextureParams.Add(Info.Name.ToString());
		}

#if WITH_EDITORONLY_DATA
		TArray< FMaterialParameterInfo > BoolParameterInfo;
		InMaterial->GetAllStaticSwitchParameterInfo(BoolParameterInfo, ParameterIds);
		for (const FMaterialParameterInfo& Info : BoolParameterInfo)
		{
			BoolParams.Add(Info.Name.ToString());
		}
#endif
	}

	Material = InMaterial;
}

void FDatasmithReferenceMaterial::FromSoftObjectPath( const FSoftObjectPath& InObjectPath)
{
	FromMaterial( Cast< UMaterialInterface >(InObjectPath.TryLoad() ) );
}
