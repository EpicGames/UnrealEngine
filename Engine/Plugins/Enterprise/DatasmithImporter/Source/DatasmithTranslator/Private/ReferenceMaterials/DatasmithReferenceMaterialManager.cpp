// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReferenceMaterials/DatasmithReferenceMaterialManager.h"

#include "ReferenceMaterials/DatasmithReferenceMaterialSelector.h"

TUniquePtr< FDatasmithReferenceMaterialManager > FDatasmithReferenceMaterialManager::Instance;

FDatasmithReferenceMaterialManager::~FDatasmithReferenceMaterialManager()
{
	Selectors.Empty();
}

void FDatasmithReferenceMaterialManager::Create()
{
	Instance = MakeUnique< FDatasmithReferenceMaterialManager >();
}

void FDatasmithReferenceMaterialManager::Destroy()
{
	Instance.Reset();
}

FDatasmithReferenceMaterialManager& FDatasmithReferenceMaterialManager::Get()
{
	check( Instance.IsValid() );
	return *Instance.Get();
}

FString FDatasmithReferenceMaterialManager::GetHostFromString( const FString& HostString )
{
	if ( HostString.Contains( TEXT("CityEngine") ) )
	{
		return TEXT("CityEngine");
	}
	else if ( HostString.Contains( TEXT("Deltagen") ) )
	{
		return TEXT("Deltagen");
	}
	else if ( HostString.Contains(TEXT("VRED") ) )
	{
		return TEXT("VRED");
	}
	else
	{
		return HostString;
	}
}

void FDatasmithReferenceMaterialManager::RegisterSelector( const TCHAR* Host, TSharedPtr< FDatasmithReferenceMaterialSelector > Selector )
{
	Selectors.FindOrAdd( Host ) = Selector;
}

void FDatasmithReferenceMaterialManager::UnregisterSelector( const TCHAR* Host )
{
	Selectors.Remove( Host );
}

const TSharedPtr< FDatasmithReferenceMaterialSelector > FDatasmithReferenceMaterialManager::GetSelector( const TCHAR* Host ) const
{
	if ( Selectors.Contains( Host ) )
	{
		return Selectors[ Host ];
	}
	else
	{
		return MakeShared< FDatasmithReferenceMaterialSelector >();
	}
}
