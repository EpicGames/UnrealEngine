// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReferenceMaterials/DatasmithReferenceMaterialSelector.h"

#include "ReferenceMaterials/DatasmithReferenceMaterial.h"
#include "IDatasmithSceneElements.h"

FDatasmithReferenceMaterial FDatasmithReferenceMaterialSelector::InvalidReferenceMaterial;

const FDatasmithReferenceMaterial& FDatasmithReferenceMaterialSelector::GetReferenceMaterial( const TSharedPtr< IDatasmithMaterialInstanceElement >& InDatasmithMaterial ) const
{
	return InvalidReferenceMaterial;
}

bool FDatasmithReferenceMaterialSelector::GetColor( const TSharedPtr< IDatasmithKeyValueProperty >& InMaterialProperty, FLinearColor& OutValue ) const
{
	if (!InMaterialProperty.IsValid() || InMaterialProperty->GetPropertyType() != EDatasmithKeyValuePropertyType::Color )
	{
		return false;
	}

	return OutValue.InitFromString( InMaterialProperty->GetValue() ); // TODO: Handle sRGB vs linear RGB properly?
}

bool FDatasmithReferenceMaterialSelector::GetFloat( const TSharedPtr< IDatasmithKeyValueProperty >& InMaterialProperty, float& OutValue ) const
{
	if (!InMaterialProperty.IsValid() || InMaterialProperty->GetPropertyType() != EDatasmithKeyValuePropertyType::Float )
	{
		return false;
	}

	OutValue = FCString::Atof( InMaterialProperty->GetValue() );

	return true;
}

bool FDatasmithReferenceMaterialSelector::GetBool( const TSharedPtr< IDatasmithKeyValueProperty >& InMaterialProperty, bool& OutValue ) const
{
	if (!InMaterialProperty.IsValid() || InMaterialProperty->GetPropertyType() != EDatasmithKeyValuePropertyType::Bool )
	{
		return false;
	}

	OutValue = FString( InMaterialProperty->GetValue() ).ToBool();

	return true;
}

bool FDatasmithReferenceMaterialSelector::GetTexture( const TSharedPtr< IDatasmithKeyValueProperty >& InMaterialProperty, FString& OutValue ) const
{
	if (!InMaterialProperty.IsValid() || InMaterialProperty->GetPropertyType() != EDatasmithKeyValuePropertyType::Texture )
	{
		return false;
	}

	OutValue = InMaterialProperty->GetValue();

	return true;
}

bool FDatasmithReferenceMaterialSelector::GetString(const TSharedPtr< IDatasmithKeyValueProperty >& InMaterialProperty, FString& OutValue) const
{
	if (!InMaterialProperty.IsValid() || InMaterialProperty->GetPropertyType() != EDatasmithKeyValuePropertyType::String )
	{
		return false;
	}

	OutValue = InMaterialProperty->GetValue();

	return true;
}
