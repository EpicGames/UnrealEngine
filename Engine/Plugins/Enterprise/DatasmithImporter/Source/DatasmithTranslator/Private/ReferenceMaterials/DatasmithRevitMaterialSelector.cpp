// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithRevitMaterialSelector.h"

#include "IDatasmithSceneElements.h"

#include "Materials/Material.h"
#include "Materials/MaterialInstanceConstant.h"
#include "UObject/SoftObjectPath.h"
#include "Templates/Casts.h"

FDatasmithRevitMaterialSelector::FDatasmithRevitMaterialSelector()
{
	// Reference
	ReferenceMaterial.FromSoftObjectPath( FSoftObjectPath("/DatasmithContent/Materials/RevitReference.RevitReference") );
	ReferenceMaterialDecal.FromSoftObjectPath(FSoftObjectPath("/DatasmithContent/Materials/StdDecal/M_StdDecal.M_StdDecal"));
}

bool FDatasmithRevitMaterialSelector::IsValid() const
{
	return ReferenceMaterial.IsValid();
}

const FDatasmithReferenceMaterial& FDatasmithRevitMaterialSelector::GetReferenceMaterial( const TSharedPtr< IDatasmithMaterialInstanceElement >& InDatasmithMaterial ) const
{
	if (InDatasmithMaterial->GetMaterialType() == EDatasmithReferenceMaterialType::Decal)
	{
		return ReferenceMaterialDecal;
	}

	return ReferenceMaterial;
}

void FDatasmithRevitMaterialSelector::FinalizeMaterialInstance(const TSharedPtr<IDatasmithMaterialInstanceElement>& InDatasmithMaterial, UMaterialInstanceConstant * MaterialInstance) const
{
	if (!InDatasmithMaterial.IsValid() || MaterialInstance == nullptr)
	{
		return;
	}

	// Set blend mode to translucent if material requires transparency
	if (InDatasmithMaterial->GetMaterialType() == EDatasmithReferenceMaterialType::Transparent)
	{
		MaterialInstance->BasePropertyOverrides.bOverride_BlendMode = true;
		MaterialInstance->BasePropertyOverrides.BlendMode = BLEND_Translucent;
	}
	// Set blend mode to masked if material has cutouts
	else if (InDatasmithMaterial->GetMaterialType() == EDatasmithReferenceMaterialType::CutOut)
	{
		MaterialInstance->BasePropertyOverrides.bOverride_BlendMode = true;
		MaterialInstance->BasePropertyOverrides.BlendMode = BLEND_Masked;
	}
}
