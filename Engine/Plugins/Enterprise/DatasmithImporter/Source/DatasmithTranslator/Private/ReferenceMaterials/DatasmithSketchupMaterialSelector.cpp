// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithSketchupMaterialSelector.h"

#include "IDatasmithSceneElements.h"

#include "Materials/Material.h"
#include "Materials/MaterialInstanceConstant.h"
#include "UObject/SoftObjectPath.h"
#include "Templates/Casts.h"

FDatasmithSketchUpMaterialSelector::FDatasmithSketchUpMaterialSelector()
{
	// Reference
	ReferenceMaterial.FromSoftObjectPath( FSoftObjectPath("/DatasmithContent/Materials/SketchupReference.SketchupReference") );
}

bool FDatasmithSketchUpMaterialSelector::IsValid() const
{
	return ReferenceMaterial.IsValid();
}

const FDatasmithReferenceMaterial& FDatasmithSketchUpMaterialSelector::GetReferenceMaterial( const TSharedPtr< IDatasmithMaterialInstanceElement >& /*InDatasmithMaterial*/ ) const
{
	return ReferenceMaterial;
}

void FDatasmithSketchUpMaterialSelector::FinalizeMaterialInstance(const TSharedPtr<IDatasmithMaterialInstanceElement>& InDatasmithMaterial, UMaterialInstanceConstant * MaterialInstance) const
{
	if (!InDatasmithMaterial.IsValid() || MaterialInstance == nullptr)
	{
		return;
	}

	if (InDatasmithMaterial->GetMaterialType() == EDatasmithReferenceMaterialType::Transparent)
	{
		MaterialInstance->BasePropertyOverrides.bOverride_BlendMode = true;
		MaterialInstance->BasePropertyOverrides.BlendMode = BLEND_Translucent;
	}
}
