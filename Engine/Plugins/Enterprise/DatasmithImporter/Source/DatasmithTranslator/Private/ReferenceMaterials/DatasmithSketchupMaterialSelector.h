// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DatasmithDefinitions.h"
#include "ReferenceMaterials/DatasmithReferenceMaterial.h"
#include "ReferenceMaterials/DatasmithReferenceMaterialSelector.h"

#include "Templates/SharedPointer.h"

class IDatasmithMaterialInstanceElement;

class FDatasmithSketchUpMaterialSelector : public FDatasmithReferenceMaterialSelector
{
public:
	FDatasmithSketchUpMaterialSelector();
	virtual ~FDatasmithSketchUpMaterialSelector() = default;

	virtual bool IsValid() const override;
	virtual const FDatasmithReferenceMaterial& GetReferenceMaterial( const TSharedPtr< IDatasmithMaterialInstanceElement >& /*InDatasmithMaterial*/ ) const override;
	virtual void FinalizeMaterialInstance(const TSharedPtr< IDatasmithMaterialInstanceElement >& InDatasmithMaterial, UMaterialInstanceConstant* MaterialInstance) const override;

private:
	FDatasmithReferenceMaterial ReferenceMaterial;
};
