// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithStdMaterialSelector.h"

#include "IDatasmithSceneElements.h"

#include "Materials/Material.h"
#include "Materials/MaterialInstanceConstant.h"
#include "UObject/SoftObjectPath.h"
#include "Templates/Casts.h"

FDatasmithStdMaterialSelector::FDatasmithStdMaterialSelector()
{
	// Reference
	ReferenceMaterialOpaque.FromSoftObjectPath( FSoftObjectPath("/DatasmithContent/Materials/StdOpaque/M_StdOpaque.M_StdOpaque") );
	ReferenceMaterialTranslucent.FromSoftObjectPath( FSoftObjectPath("/DatasmithContent/Materials/StdTranslucent/M_StdTranslucent.M_StdTranslucent") );
	ReferenceMaterialEmissive.FromSoftObjectPath( FSoftObjectPath("/DatasmithContent/Materials/StdEmissive/M_StdEmissive.M_StdEmissive") );
}

bool FDatasmithStdMaterialSelector::IsValid() const
{
	return ReferenceMaterialOpaque.IsValid() && ReferenceMaterialTranslucent.IsValid() && ReferenceMaterialEmissive.IsValid();
}

const FDatasmithReferenceMaterial& FDatasmithStdMaterialSelector::GetReferenceMaterial( const TSharedPtr< IDatasmithMaterialInstanceElement >& InDatasmithMaterial ) const
{
	switch (InDatasmithMaterial->GetMaterialType())
	{
		case EDatasmithReferenceMaterialType::Transparent:
			return ReferenceMaterialTranslucent;
		case EDatasmithReferenceMaterialType::Emissive:
			return ReferenceMaterialEmissive;
		default:
			return ReferenceMaterialOpaque;
	}
}
