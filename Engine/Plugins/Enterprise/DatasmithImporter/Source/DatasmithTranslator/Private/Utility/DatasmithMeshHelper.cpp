// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utility/DatasmithMeshHelper.h"

#include "StaticMeshAttributes.h"
#include "Engine/StaticMesh.h"
#include "StaticMeshOperations.h"
#include "DatasmithMesh.h"
#include "RawMesh.h"
#include "DatasmithUtils.h"
#include "DatasmithMeshUObject.h"
#include "Misc/SecureHash.h"
#include "UVMapSettings.h"


namespace DatasmithMeshHelper
{

	int32 GetPolygonCount(const FMeshDescription& Mesh)
	{
		return Mesh.Polygons().Num();
	}

	int32 GetTriangleCount(const FMeshDescription& Mesh)
	{
		return Mesh.Triangles().Num();
	}

	void ExtractVertexPositions(const FMeshDescription& Mesh, TArray<FVector3f>& OutPositions)
	{
		FDatasmithMeshUtils::ExtractVertexPositions(Mesh, OutPositions);
	}

	void PrepareAttributeForStaticMesh(FMeshDescription& Mesh)
	{
		FStaticMeshAttributes(Mesh).Register();
	}

	bool IsTriangleDegenerated(const FMeshDescription& Mesh, const FTriangleID TriangleID)
	{
		TVertexAttributesConstRef<FVector3f> VertexPositions = Mesh.GetVertexPositions();
		TArrayView<const FVertexID> TriangleVerts = Mesh.GetTriangleVertices(TriangleID);
		float NormalLengthSquare = ((VertexPositions[TriangleVerts[1]] - VertexPositions[TriangleVerts[2]]) ^ (VertexPositions[TriangleVerts[0]] - VertexPositions[TriangleVerts[2]])).SizeSquared();
		return NormalLengthSquare < SMALL_NUMBER;
	}


	void RemoveEmptyPolygonGroups(FMeshDescription& Mesh)
	{
		bool bRemovedSection = false;
		for (const FPolygonGroupID PolygonGroupID : Mesh.PolygonGroups().GetElementIDs())
		{
			if (Mesh.GetNumPolygonGroupPolygons(PolygonGroupID) == 0)
			{
				Mesh.DeletePolygonGroup(PolygonGroupID);
				bRemovedSection = true;
			}
		}

		if (bRemovedSection)
		{
			FElementIDRemappings Remappings;
			Mesh.Compact(Remappings);
		}
	}

#if WITH_EDITOR
	FMeshDescription* InitMeshDescription(UStaticMesh* StaticMesh, int32 LodIndex)
	{
		if (!ensure(StaticMesh))
		{
			return nullptr;
		}

		while (!StaticMesh->IsSourceModelValid(LodIndex))
		{
			StaticMesh->AddSourceModel();
		}

		return StaticMesh->CreateMeshDescription(LodIndex);
	}

	void FillUStaticMesh(UStaticMesh* StaticMesh, int32 LodIndex, FRawMesh& RawMesh, const TMap<int32, FName>* InMaterialMapInverse)
	{
		FMeshDescription MeshDescription;
		PrepareAttributeForStaticMesh(MeshDescription);
		if (InMaterialMapInverse == nullptr)
		{
			// no explicit mapping, build one from UStaticMesh
			TMap<FName, int32> MaterialMap; // unused
			TMap<int32, FName> MaterialMapInverse;

 			BuildMaterialMappingFromStaticMesh(StaticMesh, MaterialMap, MaterialMapInverse);
			FStaticMeshOperations::ConvertFromRawMesh(RawMesh, MeshDescription, MaterialMapInverse);
		}
		else
		{
			FStaticMeshOperations::ConvertFromRawMesh(RawMesh, MeshDescription, *InMaterialMapInverse);
		}

		FillUStaticMesh(StaticMesh, LodIndex, MoveTemp(MeshDescription));
	}

	void FillUStaticMesh(UStaticMesh* StaticMesh, int32 LodIndex, FDatasmithMesh& DsMesh)
	{
		FMeshDescription MeshDescription;
		PrepareAttributeForStaticMesh(MeshDescription);
		if (FDatasmithMeshUtils::ToMeshDescription(DsMesh, MeshDescription))
		{
			FillUStaticMesh(StaticMesh, LodIndex, MoveTemp(MeshDescription));
		}
	}

	void FillUStaticMesh(UStaticMesh* StaticMesh, int LodIndex, FMeshDescription&& MeshDescription)
	{
		if ( !IsMeshValid( MeshDescription ) )
		{
			return;
		}

		FMeshDescription* OriginalMeshDescription = InitMeshDescription(StaticMesh, LodIndex);
		if (ensure(OriginalMeshDescription))
		{
			// #ueent_todo: validate attributes too make sure we don't push incompatible MeshDescriptions
			*OriginalMeshDescription = MoveTemp(MeshDescription);
		}
	}

	void FillUStaticMeshByCopy(UStaticMesh* StaticMesh, int LodIndex, FMeshDescription MeshDescription)
	{
		FillUStaticMesh(StaticMesh, LodIndex, MoveTemp(MeshDescription));
	}

	void PrepareStaticMaterials(UStaticMesh* StaticMesh, int32 MaterialCount)
	{
		if (!ensure(StaticMesh))
		{
			return;
		}

		MaterialCount = FMath::Max(MaterialCount, 1);

		// Don't touch the StaticMesh if it's not needed (would break the template system on reimport)
		if (MaterialCount != StaticMesh->GetStaticMaterials().Num())
		{
			StaticMesh->GetStaticMaterials().Empty(MaterialCount);
			// Declare materials applied to the StaticMesh according to fit the logic in UDatasmithImportFactory::UpdateMaterials
			for (int MaterialIndex = 0; MaterialIndex < MaterialCount; MaterialIndex++)
			{
				StaticMesh->GetSectionInfoMap().Set(0, MaterialIndex, FMeshSectionInfo(MaterialIndex));
				FName SlotName = DefaultSlotName(MaterialIndex);
				StaticMesh->GetStaticMaterials().Add(FStaticMaterial(nullptr, SlotName, SlotName));
			}
		}
	}

	void BuildMaterialMappingFromStaticMesh(UStaticMesh* StaticMesh, TMap<FName, int32>& MaterialMap, TMap<int32, FName>& MaterialMapInverse)
	{
		if (!ensure(StaticMesh))
		{
			return;
		}

		int32 MatCount = StaticMesh->GetStaticMaterials().Num();
		ensure(MatCount);
		MaterialMap.Empty(MatCount);
		MaterialMapInverse.Empty(MatCount);

		for ( int32 MaterialIndex = 0; MaterialIndex < MatCount; ++MaterialIndex )
		{
			FName& MatName = StaticMesh->GetStaticMaterials()[MaterialIndex].ImportedMaterialSlotName;
			MaterialMap.Add(MatName, MaterialIndex);
			MaterialMapInverse.Add(MaterialIndex, MatName);
		}
	}

	void HashMeshLOD(UStaticMesh* StaticMesh, int32 LodIndex, FMD5& MD5)
	{
		if (ensure(StaticMesh))
		{
			const FMeshDescription* Mesh = StaticMesh->GetMeshDescription(LodIndex);
			if (ensure(Mesh))
			{
				HashMeshDescription(*Mesh, MD5);
			}
		}
	}
#endif //WITH_EDITOR

	void HashMeshDescription(const FMeshDescription& Mesh, FMD5& MD5)
	{
		auto HashAttributeSet = [](FMD5& InMD5, const FAttributesSetBase& AttributeSet)
		{
			TArray<FName> OutAttributeNames;
			AttributeSet.GetAttributeNames(OutAttributeNames);
			for (const FName& AttributeName: OutAttributeNames)
			{
				int32 AttributeHash = AttributeSet.GetHash(AttributeName);
				InMD5.Update((uint8*)&AttributeHash, sizeof(AttributeHash));
				// #ueent_todo better hash function...
				// see FDatasmithFBXSceneMesh::GetHash
			}
		};

		HashAttributeSet(MD5, Mesh.VertexAttributes());
		HashAttributeSet(MD5, Mesh.VertexInstanceAttributes());
		HashAttributeSet(MD5, Mesh.EdgeAttributes());
		HashAttributeSet(MD5, Mesh.PolygonAttributes());
		HashAttributeSet(MD5, Mesh.PolygonGroupAttributes());
	}

	FName DefaultSlotName(int32 MaterialIndex)
	{
		return FName( *FString::FromInt(MaterialIndex) );
	}

	int32 GetNumUVChannel(const FMeshDescription& Mesh)
	{
		TVertexInstanceAttributesConstRef<FVector2f> UVChannels = FStaticMeshConstAttributes(Mesh).GetVertexInstanceUVs();
		return UVChannels.GetNumChannels();
	}

	bool HasUVChannel(const FMeshDescription& Mesh, int32 ChannelIndex)
	{
		return ChannelIndex >= 0 && ChannelIndex < GetNumUVChannel(Mesh);
	}

	bool HasUVData(const FMeshDescription& Mesh, int32 ChannelIndex)
	{
		if (HasUVChannel(Mesh, ChannelIndex))
		{
			TVertexInstanceAttributesConstRef<FVector2f> UVChannels = FStaticMeshConstAttributes(Mesh).GetVertexInstanceUVs();
			const FVector2f DefValue = UVChannels.GetDefaultValue();
			for (FVertexInstanceID InstanceID : Mesh.VertexInstances().GetElementIDs())
			{
				if (UVChannels.Get(InstanceID, ChannelIndex) != DefValue)
				{
					return true;
				}
			}
		}
		return false;
	}

	bool RequireUVChannel(FMeshDescription& Mesh, int32 ChannelIndex)
	{
		if (ChannelIndex < 0 || ChannelIndex >= MAX_MESH_TEXTURE_COORDS_MD)
		{
			return false;
		}

		if (!HasUVChannel(Mesh, ChannelIndex))
		{
			TVertexInstanceAttributesRef<FVector2f> UVChannels = FStaticMeshAttributes(Mesh).GetVertexInstanceUVs();
			UVChannels.SetNumChannels(ChannelIndex + 1);
		}
		return true;
	}

	void CreateDefaultUVs(FDatasmithMesh& DatasmithMesh)
	{
		FDatasmithMeshUtils::CreateDefaultUVsWithLOD(DatasmithMesh);
	}

	void CreateDefaultUVs(FMeshDescription& MeshDescription)
	{
		FBox MeshBoundingBox = MeshDescription.ComputeBoundingBox();
		FUVMapParameters UVParameters(MeshBoundingBox.GetCenter(), FQuat::Identity, MeshBoundingBox.GetSize(), FVector::OneVector, FVector2D::UnitVector);
		TMap<FVertexInstanceID, FVector2D> TexCoords;
		FStaticMeshOperations::GenerateBoxUV(MeshDescription, UVParameters, TexCoords);

		TVertexInstanceAttributesRef<FVector2f> UVs = FStaticMeshAttributes(MeshDescription).GetVertexInstanceUVs();
		if (UVs.GetNumChannels() == 0)
		{
			UVs.SetNumChannels(1);
		}

		for (const FVertexInstanceID VertexInstanceID : MeshDescription.VertexInstances().GetElementIDs())
		{
			if (const FVector2D* UVCoord = TexCoords.Find(VertexInstanceID))
			{
				UVs.Set(VertexInstanceID, 0, FVector2f(*UVCoord));
			}
			else
			{
				ensureMsgf(false, TEXT("Tried to apply UV data that did not match the MeshDescription."));
			}
		}
	}

	bool IsMeshValid(const FMeshDescription& Mesh, FVector3f BuildScale)
	{
		TVertexAttributesConstRef<FVector3f> VertexPositions = Mesh.GetVertexPositions();
		FVector3f RawNormalScale(BuildScale.Y*BuildScale.Z, BuildScale.X*BuildScale.Z, BuildScale.X*BuildScale.Y); // Component-wise scale

		for (const FTriangleID TriangleID : Mesh.Triangles().GetElementIDs())
		{
			TArrayView<const FVertexID> VertexIDs = Mesh.GetTriangleVertices(TriangleID);
			FVector3f Corners[3] =
			{
				VertexPositions[VertexIDs[0]],
				VertexPositions[VertexIDs[1]],
				VertexPositions[VertexIDs[2]]
			};

			FVector3f RawNormal = (Corners[1] - Corners[2]) ^ (Corners[0] - Corners[2]);
			RawNormal *= RawNormalScale;
			float FourSquaredTriangleArea = RawNormal.SizeSquared();

			// We support even small triangles, but this function is still useful to
			// see if we have at least one valid triangle in the mesh
			if (FourSquaredTriangleArea > 0.0f)
			{
				return true;
			}
		}

		// all faces are degenerated, mesh is invalid
		return false;
	}

} // namespace DatasmithMeshHelper
