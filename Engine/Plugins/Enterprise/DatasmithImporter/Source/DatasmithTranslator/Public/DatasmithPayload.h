// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithCloth.h"  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
#include "MeshDescription.h"


struct FDatasmithPayload
{
	TArray<class UDatasmithAdditionalData*> AdditionalData;
};


struct FDatasmithMeshElementPayload : public FDatasmithPayload
{
	TArray<FMeshDescription> LodMeshes;
	FMeshDescription CollisionMesh;
	TArray<FVector3f> CollisionPointCloud; // compatibility, favor the CollisionMesh member
};


/**
 * Describes a Cloth element payload, which is the actual data to be imported.
 */
struct UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.") FDatasmithClothElementPayload : public FDatasmithPayload
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FDatasmithCloth Cloth;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
};


struct FDatasmithLevelSequencePayload : public FDatasmithPayload
{
// #ueent_todo: split element in metadata/payload
};
