// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/StrongObjectPtr.h"


/**
 * Describe a source that can be processed by Datasmith to generate a scene.
 */
class DATASMITHTRANSLATOR_API FDatasmithSceneSource
{
public:
	/**
	 * Set the path of the source
	 * @param InFilename New file path
	 */
	void SetSourceFile(const FString& InFilePath);

	/**
	 * Get the file path for this source
	 * @return The source path
	 */
	const FString& GetSourceFile() const { return FilePath; }

	/**
	 * Get the source file extension
	 * @return const FString&
	 */
	const FString& GetSourceFileExtension() const { return FileExtension; }

	/**
	 * Set a custom name for this scene.
	 * This overrides the default name deduced form the source file
	 * @param InSceneName New name
	 */
	void SetSceneName(const FString& InSceneName);

	/**
	 * @return the scene name
	 */
	const FString& GetSceneName() const ;

private:

	// source path (can be relative)
	FString FilePath;

	// Extracted from the source file path
	FString FileExtension;

	// Default name when not overriden
	FString SceneDeducedName;

	// Explicit name (overrides deduced name)
	FString SceneOverrideName;
};

