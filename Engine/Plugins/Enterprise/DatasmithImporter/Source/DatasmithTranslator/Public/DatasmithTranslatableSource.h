// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithSceneSource.h"

#include "CoreMinimal.h"
#include "UObject/StrongObjectPtr.h"

class IDatasmithScene;
class IDatasmithTranslator;

/**
 * Scopes a translator's loading lifecycle.
 */
class DATASMITHTRANSLATOR_API FDatasmithSceneGuard
{
public:
	FDatasmithSceneGuard(const TSharedPtr<IDatasmithTranslator>& Translator, const TSharedRef<IDatasmithScene>& Scene, bool& bOutLoadOk);

	~FDatasmithSceneGuard();

private:
	TSharedPtr<IDatasmithTranslator> Translator;
};

/**
 * Wrap a source with an adapted translator.
 * This scopes the lifecycle of a translator, 
 */
struct DATASMITHTRANSLATOR_API FDatasmithTranslatableSceneSource
{
	FDatasmithTranslatableSceneSource(const FDatasmithSceneSource& Source);
	~FDatasmithTranslatableSceneSource();

	bool IsTranslatable() const;

	bool Translate(TSharedRef<IDatasmithScene> Scene);

	TSharedPtr<IDatasmithTranslator> GetTranslator() const;

private:
	/** Translator currently in use (null when not importing) */
	TSharedPtr<IDatasmithTranslator> Translator;

	/** internal helper to release scene */
	TUniquePtr<FDatasmithSceneGuard> SceneGuard;
};