// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DatasmithPayload.h"
#include "DatasmithSceneSource.h"
#include "DatasmithImportOptions.h"

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"
#include "UObject/StrongObjectPtr.h"

class FDatasmithSceneSource;
class IDatasmithScene;
class UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.") IDatasmithClothElement;
class IDatasmithMeshElement;
class IDatasmithLevelSequenceElement;

struct FFileFormatInfo
{
	FFileFormatInfo(const TCHAR* InExtension, const TCHAR* InDescription)
		:Extension(InExtension), Description(InDescription)
	{}
	FString Extension;
	FString Description;
};


/**
 * Description of Translator capabilities
 */
struct FDatasmithTranslatorCapabilities
{
	bool bIsEnabled = true;
	bool bParallelLoadStaticMeshSupported = false;
	TArray<FFileFormatInfo> SupportedFileFormats;
};


/**
 * Generic Scene exploration and realization API
 */
class DATASMITHTRANSLATOR_API IDatasmithTranslator
{
public:
	virtual ~IDatasmithTranslator() = default;

	virtual FName GetFName() const = 0;

	/**
	 * Called to initialize this Translator instance
	 * @param OutCapabilities	Declared capabilities for this Translator
	 */
	virtual void Initialize(FDatasmithTranslatorCapabilities& OutCapabilities) {}

	/**
	 * Additional validation step to ensure the given file is supported (Optional)
	 *
	 * @param Source	Candidate Source
	 * @returns			True if the given source is supported by this Translator
	 */
	virtual bool IsSourceSupported(const FDatasmithSceneSource& Source) { return true; }

	/**
	 * Beginning of a translator instance lifecycle: setup of the source
	 *
	 * @param Source    Source of the scene to translate
	 */
	void SetSource(const FDatasmithSceneSource& Source) { SceneSource = Source; }

	/**
	 * @return the initial source
	 */
	const FDatasmithSceneSource& GetSource() const { return SceneSource; }

	/**
	 * Main feature of the translator system, convert source to a datasmith scene
	 *
	 * @param Source	Source to translate
	 * @param OutScene	DatasmithScene to fill with source informations
	 * @returns			Load succeed
	 */
	virtual bool LoadScene(TSharedRef<IDatasmithScene> OutScene) { return false; }

	/**
	 * Called when the scene loading is over. All translator resources can be released.
	 */
	virtual void UnloadScene() {}

	/**
	 * Get payload related to the given Element
	 *
	 * @param MeshElement		Element for which the payload is required
	 * @param OutMeshPayload	Actual mesh data from the source
	 * @returns					Operation succeed
	 */
	virtual bool LoadStaticMesh(const TSharedRef<IDatasmithMeshElement> MeshElement, FDatasmithMeshElementPayload& OutMeshPayload) { return false; }

	/**
	 * Get payload related to the given Element
	 *
	 * @param ClothElement       Element for which the payload is required
	 * @param OutClothPayload    Actual cloth data from the source
	 * @returns                  Operation succeed
	 */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	virtual bool LoadCloth(const TSharedRef<IDatasmithClothElement> ClothElement, FDatasmithClothElementPayload& OutClothPayload) { return false; }
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
	 * Get payload related to the given Element
	 *
	 * @param LevelSequenceElement		Element for which the payload is required
	 * @param OutLevelSequencePayload	Data associated with this element
	 * @returns							Operation succeed
	 */
	virtual bool LoadLevelSequence(const TSharedRef<IDatasmithLevelSequenceElement> LevelSequenceElement, FDatasmithLevelSequencePayload& OutLevelSequencePayload) { return false; }

	/**
	 * Get the additional scene import options.
	 * Implementation can expose additional options to the user here.
	 *
	 * @param OptionClasses list of classes that will be displayed to the user
	 */
	virtual void GetSceneImportOptions(TArray<TObjectPtr<UDatasmithOptionsBase>>& Options) {}
	UE_DEPRECATED(5.1, "Deprecated, please use same method using array of TObjectPtr instead")
	virtual void GetSceneImportOptions(TArray<TStrongObjectPtr<UDatasmithOptionsBase>>& Options) {}

	/**
	 * Values of additional options as entered by the user
	 *
	 * @param Options Actual values for the displayed options.
	 */
	virtual void SetSceneImportOptions(const TArray<TObjectPtr<UDatasmithOptionsBase>>& Options) {}
	UE_DEPRECATED(5.1, "Deprecated, please use same method using array of TObjectPtr instead")
	virtual void SetSceneImportOptions(TArray<TStrongObjectPtr<UDatasmithOptionsBase>>& Options) {}

private:
	FDatasmithSceneSource SceneSource;
};

namespace Datasmith
{
	struct FTranslatorRegisterInformation
	{
		FName TranslatorName = NAME_None;
		TFunction<TSharedPtr<IDatasmithTranslator>()> SpawnFunction;
	};

	namespace Details
	{
		void DATASMITHTRANSLATOR_API RegisterTranslatorImpl(const FTranslatorRegisterInformation& Info);
		void DATASMITHTRANSLATOR_API UnregisterTranslatorImpl(const FTranslatorRegisterInformation& Info);
	};


	template<typename ImplType>
	void RegisterTranslator()
	{
		FTranslatorRegisterInformation RegInfo;
		ImplType Instance;
		RegInfo.TranslatorName = Instance.GetFName();
		RegInfo.SpawnFunction = []{ return MakeShared<ImplType>(); };
		Details::RegisterTranslatorImpl(RegInfo);
	}

	template<typename ImplType>
	void UnregisterTranslator()
	{
		FTranslatorRegisterInformation RegInfo;
		ImplType Instance;
		RegInfo.TranslatorName = Instance.GetFName();
		Details::UnregisterTranslatorImpl(RegInfo);
	}

	template<class UOptionClass>
	inline UOptionClass* MakeOptionsPtr()
	{
		return NewObject<UOptionClass>(GetTransientPackage(), UOptionClass::StaticClass());
	}

	template<class UOptionClass>
	inline TStrongObjectPtr<UOptionClass> MakeOptions()
	{
		TStrongObjectPtr<UOptionClass> Option(MakeOptionsPtr<UOptionClass>());
		return Option;
	}

	template<class UOptionClass>
	inline TObjectPtr<UOptionClass> MakeOptionsObjectPtr()
	{
		return NewObject<UOptionClass>(GetTransientPackage(), UOptionClass::StaticClass());
	}

	FString DATASMITHTRANSLATOR_API GetXMLFileSchema(const FString& XmlFilePath);

	template<typename ValueType>
	bool CheckXMLSchema(const FString& FileXmlSchema, ValueType Format)
	{
		return (FileXmlSchema == Format);
	}

	template<typename ValueType, class... ValueTypes>
	bool CheckXMLSchema(const FString& FileXmlSchema, ValueType Format, ValueTypes... OtherFormats)
	{
		if (FileXmlSchema == Format)
		{
			return true;
		}
		return CheckXMLSchema(FileXmlSchema, OtherFormats...);
	}

	/**
	 * Check if the xml file schema is in the list of supported schema
	 */
	template<class... ValueTypes>
	bool CheckXMLFileSchema(const FString& XmlFilePath, ValueTypes&... SupportedSchema)
	{
		FString FileXmlSchema = GetXMLFileSchema(XmlFilePath);
		return CheckXMLSchema(FileXmlSchema, SupportedSchema...);
	}
}
