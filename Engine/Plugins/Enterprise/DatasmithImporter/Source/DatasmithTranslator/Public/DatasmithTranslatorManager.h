// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithTranslator.h"


/**
 * Responsible of the Translators lifecycle.
 */
class DATASMITHTRANSLATOR_API FDatasmithTranslatorManager
{
	FDatasmithTranslatorManager() = default;
public:
	/** Gather all viable translators */
	static FDatasmithTranslatorManager& Get();

	/** Get the list of "ext;description" formats */
	const TArray<FString>& GetSupportedFormats() const;

	/**
	 * Based on source file extension, select a suitable Translator
	 * @param Source	import source
	 * @return			First Translator capable of handling the given file
	 */
	TSharedPtr<IDatasmithTranslator> SelectFirstCompatible(const FDatasmithSceneSource& Source);

	/**
	 * Register an implementation.(Only registered Implementations are usable.)
	 * @note: Do not use directly, use Datasmith::RegisterTranslator<>() function instead
	 * @param Info required informations to dynamically handle various Translators
	 */
	void Register(const Datasmith::FTranslatorRegisterInformation& Info);

	/**
	 * Revert a call to Register.
	 * @note: Do not use directly, use Datasmith::UnregisterTranslator<>() function instead
	 * @param TranslatorName Name of the registered Translator to remove.
	 */
	void Unregister(FName TranslatorName);

private:
	void InvalidateCache();

private:
	TArray<Datasmith::FTranslatorRegisterInformation> RegisteredTranslators;
	mutable TArray<FString> Formats;
};
