// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#define DATASMITHTRANSLATOR_MODULE_NAME TEXT("DatasmithTranslator")

class IDatasmithTranslatorModule : public IModuleInterface
{
public:

	static IDatasmithTranslatorModule& Get()
	{
		return FModuleManager::LoadModuleChecked< IDatasmithTranslatorModule >(DATASMITHTRANSLATOR_MODULE_NAME);
	}

	static bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(DATASMITHTRANSLATOR_MODULE_NAME);
	}

	virtual void StartupModule() override;

	virtual void ShutdownModule() override;
};