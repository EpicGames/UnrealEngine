// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Set.h"
#include "Containers/UnrealString.h"

struct FSoftObjectPath;
class UMaterialInterface;

/**
 * Represents a material that exposes named parameters for override purpose.
 * Note that this can be a Material or a MaterialInstance.
 */
class DATASMITHTRANSLATOR_API FDatasmithReferenceMaterial
{
public:
	FDatasmithReferenceMaterial();

	void FromMaterial( UMaterialInterface* InMaterial );
	void FromSoftObjectPath( const FSoftObjectPath& InObjectPath );

	UMaterialInterface* GetMaterial() const { return Material; }

	bool IsValid() const { return Material != nullptr; }

	TSet< FString > VectorParams;
	TSet< FString > ScalarParams;
	TSet< FString > TextureParams;
#if WITH_EDITORONLY_DATA
	TSet< FString > BoolParams;
#endif

private:
	UMaterialInterface* Material;
};
