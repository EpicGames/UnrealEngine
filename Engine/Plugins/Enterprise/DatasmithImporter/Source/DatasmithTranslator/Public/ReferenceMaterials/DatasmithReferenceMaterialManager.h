// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/ContainersFwd.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

class FDatasmithReferenceMaterialSelector;

class DATASMITHTRANSLATOR_API FDatasmithReferenceMaterialManager
{
public:
	FDatasmithReferenceMaterialManager() = default;
	~FDatasmithReferenceMaterialManager();

	static void Create();
	static void Destroy();
	static FDatasmithReferenceMaterialManager& Get();

	FString GetHostFromString( const FString& HostString );

	void RegisterSelector( const TCHAR* Host, TSharedPtr< FDatasmithReferenceMaterialSelector > Selector );
	void UnregisterSelector( const TCHAR* Host );

	const TSharedPtr< FDatasmithReferenceMaterialSelector > GetSelector( const TCHAR* Host ) const;

private:
	static TUniquePtr< FDatasmithReferenceMaterialManager > Instance;

	TMap< FString, TSharedPtr< FDatasmithReferenceMaterialSelector > > Selectors;
};