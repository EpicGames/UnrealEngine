// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Math/Color.h"
#include "Templates/SharedPointer.h"

class FDatasmithReferenceMaterial;
class IDatasmithMaterialInstanceElement;
class IDatasmithKeyValueProperty;
class UMaterialInstanceConstant;

class DATASMITHTRANSLATOR_API FDatasmithReferenceMaterialSelector
{
public:
	virtual ~FDatasmithReferenceMaterialSelector() = default;

	virtual bool IsValid() const { return false; }
	virtual const FDatasmithReferenceMaterial& GetReferenceMaterial( const TSharedPtr< IDatasmithMaterialInstanceElement >& InDatasmithMaterial ) const;
	virtual void FinalizeMaterialInstance(const TSharedPtr< IDatasmithMaterialInstanceElement >& InDatasmithMaterial, UMaterialInstanceConstant* MaterialInstance) const {}

	virtual bool GetColor( const TSharedPtr< IDatasmithKeyValueProperty >& InMaterialProperty, FLinearColor& OutValue ) const;
	virtual bool GetFloat( const TSharedPtr< IDatasmithKeyValueProperty >& InMaterialProperty, float& OutValue ) const;
	virtual bool GetBool( const TSharedPtr< IDatasmithKeyValueProperty >& InMaterialProperty, bool& OutValue ) const;
	virtual bool GetTexture( const TSharedPtr< IDatasmithKeyValueProperty >& InMaterialProperty, FString& OutValue ) const;
	virtual bool GetString( const TSharedPtr< IDatasmithKeyValueProperty >& InMaterialProperty, FString& OutValue ) const;

protected:
	static FDatasmithReferenceMaterial InvalidReferenceMaterial;

};

