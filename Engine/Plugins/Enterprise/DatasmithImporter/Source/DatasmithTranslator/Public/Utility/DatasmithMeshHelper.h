// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/ContainersFwd.h"
#include "Math/Vector.h"

class FDatasmithMesh;
class FMD5;
class FName;
class UStaticMesh;
struct FMeshDescription;
struct FMeshTriangle;
struct FRawMesh;

namespace DatasmithMeshHelper
{
	/**
	 * Number of polygon in the given Mesh
	 *
	 * @param Mesh      Source mesh
	 * @return int32    number of polys
	 */
	DATASMITHTRANSLATOR_API int32 GetPolygonCount(const FMeshDescription& Mesh);

	/**
	 * Number of triangles in the given Mesh
	 *
	 * @param Mesh      Source mesh
	 * @return int32    number of triangles
	 */
	DATASMITHTRANSLATOR_API int32 GetTriangleCount(const FMeshDescription& Mesh);

	/**
	 * Build a copy of all positions.
	 * Can be used to build a convexHull, a bounding box, etc...
	 *
	 * @param Mesh
	 * @param Positions
	 * @return DATASMITHIMPORTER_API void
	 */
	DATASMITHTRANSLATOR_API void ExtractVertexPositions(const FMeshDescription& Mesh, TArray<FVector3f>& OutPositions);

	/**
	 * Register all attributes required to correctly interact with StaticMesh.
	 * @note: Delegates to FStaticMesh the registering of standard attributes
	 *
	 * @param Mesh   MeshDescription to prepare for StaticMesh usage
	 */
	DATASMITHTRANSLATOR_API void PrepareAttributeForStaticMesh(FMeshDescription& Mesh);

	/**
	 * Test if a triangle is extremely small or if its corners are aligned.
	 *
	 * @param Mesh		    Source mesh that contains the tested triangle
	 * @param MeshTriangle	Triangle in that mesh
	 * @return bool         True for degenerated triangles, false otherwise
	 */
	DATASMITHTRANSLATOR_API bool IsTriangleDegenerated(const FMeshDescription& Mesh, const FMeshTriangle& MeshTriangle);

	/**
	 * Remove any empty polygon groups in the mesh description and compact it if needed.
	 *
	 * @param Mesh		    The source mesh
	 */
	DATASMITHTRANSLATOR_API void RemoveEmptyPolygonGroups(FMeshDescription& Mesh);

#if WITH_EDITOR
	/**
	 * Write mesh data on a UStaticMesh.
	 * RawMesh will be converted to MeshDescription beforehand.
	 *
	 * @param StaticMesh    Destination StaticMesh
	 * @param LodIndex      Destination index in the StaticMesh
	 * @param Mesh          Source data to write into the StaticMesh
	 */
	DATASMITHTRANSLATOR_API void FillUStaticMesh(UStaticMesh* StaticMesh, int32 LodIndex, FRawMesh& RawMesh, const TMap<int32, FName>* InMaterialMapInverse = nullptr);
	DATASMITHTRANSLATOR_API void FillUStaticMesh(UStaticMesh* StaticMesh, int32 LodIndex, FDatasmithMesh& DSMesh);
	DATASMITHTRANSLATOR_API void FillUStaticMeshByCopy(UStaticMesh* StaticMesh, int LodIndex, FMeshDescription MeshDescription);
	DATASMITHTRANSLATOR_API void FillUStaticMesh(UStaticMesh* StaticMesh, int LodIndex, FMeshDescription&& MeshDescription);
	/**
	 * Make sure the StaticMesh can host all used materials
	 *
	 * @param				    StaticMesh Mesh to prepare
	 * @param MaterialCount     Number of slot required
	 */
	DATASMITHTRANSLATOR_API void PrepareStaticMaterials(UStaticMesh* StaticMesh, int32 MaterialCount);

	/**
	 * Used in FRawMesh <-> FMeshDescription context, create the SlotName <-> MaterialIndex mapping
	 * MaterialIndex in range [0, StaticMaterials.Num()-1]
	 * SlotName extracted from, StaticMaterials.ImportedMaterialSlotName
	 *
	 * @param StaticMesh            Source of the mapping information
	 * @param MaterialMap	        map SlotName to MaterialIndex
	 * @param MaterialMapInverse    map a MaterialIndex to SlotName
	 */
	DATASMITHTRANSLATOR_API void BuildMaterialMappingFromStaticMesh(UStaticMesh* StaticMesh, TMap<FName, int32>& MaterialMap, TMap<int32, FName>& MaterialMapInverse);

	/**
	 * Hash a Mesh at specified LOD index in a StaticMesh
	 *
	 * @param StaticMesh    Source Mesh set
	 * @param LodIndex      LOD index of the Mesh to hash
	 * @param MD5           Hash to update
	 * @return DATASMITHIMPORTER_API void
	 */
	DATASMITHTRANSLATOR_API void HashMeshLOD(UStaticMesh* StaticMesh, int32 LodIndex, FMD5& MD5);
#endif //WITH_EDITOR

	/**
	 * Hash a MeshDescription
	 *
	 * @param Mesh  MeshDescription to hash
	 * @param MD5   Hash to update
	 */
	DATASMITHTRANSLATOR_API void HashMeshDescription(const FMeshDescription& Mesh, FMD5& MD5);

	/**
	 * Generate a default FName to name a material slot based on the index
	 *
	 * @param MaterialIndex index of the slot to name
	 */
	DATASMITHTRANSLATOR_API FName DefaultSlotName(int32 MaterialIndex);

	DATASMITHTRANSLATOR_API int32 GetNumUVChannel(const FMeshDescription& Mesh);
	DATASMITHTRANSLATOR_API bool HasUVChannel(const FMeshDescription& Mesh, int32 ChannelIndex);

	DATASMITHTRANSLATOR_API bool HasUVData(const FMeshDescription& Mesh, int32 ChannelIndex);

	DATASMITHTRANSLATOR_API bool RequireUVChannel(FMeshDescription& Mesh, int32 ChannelIndex);

	/**
	 * Generate simple UV data at channel 0.
	 *
	 * @param DatasmithMesh		The DatasmithMesh in which the UV data will be created.
	 */
	DATASMITHTRANSLATOR_API void CreateDefaultUVs(FDatasmithMesh& DatasmithMesh);

	/**
	 * Generate simple UV data at channel 0.
	 *
	 * @param MeshDescription	The MeshDescription in which the UV data will be created.
	 */
	DATASMITHTRANSLATOR_API void CreateDefaultUVs(FMeshDescription& MeshDescription);

	/**
	 * Verify that at least one triangle in that mesh is not degenerated
	 *
	 * @param Mesh
	 * @param BuildScale
	 * @return DATASMITHIMPORTER_API bool
	 */
	DATASMITHTRANSLATOR_API bool IsMeshValid(const FMeshDescription& Mesh, FVector3f BuildScale=FVector3f::OneVector);
}
