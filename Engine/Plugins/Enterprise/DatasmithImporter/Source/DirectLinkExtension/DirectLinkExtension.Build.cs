// Copyright Epic Games, Inc. All Rights Reserved.
using System;

namespace UnrealBuildTool.Rules
{
	public class DirectLinkExtension : ModuleRules
	{
		public DirectLinkExtension(ReadOnlyTargetRules Target)
			: base(Target)
		{

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Engine",
					"InputCore",
					"UdpMessaging", // required for DirectLink networking
				}
			);

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"DirectLink",
					"ExternalSource",
				}
			);

			if (Target.Type == TargetType.Editor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"UnrealEd",
					}
				);
			}
		}
	}
}