// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoReimportManager.h"

#include "AssetRegistry/AssetData.h"
#include "Async/Async.h"
#include "ExternalSource.h"
#include "SourceUri.h"
#if WITH_EDITOR
#include "EditorReimportHandler.h"
#endif
#include "ExternalSourceModule.h"
#include "Stats/Stats.h"
#include "Misc/App.h"

namespace UE::DatasmithImporter
{
	struct FAutoReimportInfo
	{
		FAutoReimportInfo(UObject* InTargetObject, const TSharedRef<FExternalSource>& InExternalSource, FDelegateHandle InImportDelegateHandle)
			: TargetObject(InTargetObject)
			, ExternalSource(InExternalSource)
			, ImportDelegateHandle(InImportDelegateHandle)
			, bChangedDuringPIE(false)
		{}

		TSoftObjectPtr<UObject> TargetObject;
		TSharedRef<FExternalSource> ExternalSource;
		FDelegateHandle ImportDelegateHandle;
		bool bChangedDuringPIE;
	};

	void FAutoReimportManager::Tick(float DeltaTime)
	{
		if (!CanTriggerReimport())
		{
			return;
		}

		// Trigger Reimport for modified ExternalSources.
		{
			// Using a TSet here so that multiple reimport request for the same external source will only be processed once. 
			// Doing this allows us to skip redundant reimports, as the reimport already uses the latest data from the ExternalSource.
			TSet<TSharedRef<FExternalSource>> PendingReimportSet;
			TSharedPtr<FExternalSource> EnqueuedSourceToReimport;
			while (PendingReimportQueue.Dequeue(EnqueuedSourceToReimport))
			{
				PendingReimportSet.Add(EnqueuedSourceToReimport.ToSharedRef());
			}

			for (const TSharedRef<FExternalSource>& ExternalSourceToReimport : PendingReimportSet)
			{
				TriggerAutoReimportOnExternalSource(ExternalSourceToReimport);
			}
		}

		TSharedPtr<FExternalSource> InvalidedSourcePtr;
		while (PendingInvalidateQueue.Dequeue(InvalidedSourcePtr))
		{
			TSharedRef<FExternalSource> InvalidedSource(InvalidedSourcePtr.ToSharedRef());
			TArray<TSharedRef<FAutoReimportInfo>> AutoReimportInfoList;
			AutoReimportExternalSourcesMap.MultiFind(InvalidedSource, AutoReimportInfoList);
			AutoReimportExternalSourcesMap.Remove(InvalidedSource);

			for (const TSharedRef<FAutoReimportInfo>& AutoReimportInfo : AutoReimportInfoList)
			{
				UObject* Asset = AutoReimportInfo->TargetObject.Get();

				// Assets that were registered for auto-reimport are unregistered from the source
				// and put back in the "PendingRegistration" list. That way if a compatible source appear they will register to it.
				AutoReimportObjectsMap.Remove(Asset);
				PendingAutoReimportObjects.Add(Asset);
			}
		}

		// Try to re-enable auto-reimport for assets who have lost their connection.
		{
			TArray<TSoftObjectPtr<UObject>> AutoReimportObjectListCopy(PendingAutoReimportObjects.Array());
			for (const TSoftObjectPtr<UObject>& Object : AutoReimportObjectListCopy)
			{
				if (Object.IsValid())
				{
					EnableAssetAutoReimport(Object.Get());
				}
			}
		}
	}

	TStatId FAutoReimportManager::GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FAutoReimportManager, STATGROUP_Tickables);
	}


	bool FAutoReimportManager::CanTriggerReimport() const
	{
		if (!ensure(IsInGameThread())
			|| UE::IsSavingPackage(nullptr)
			|| IsGarbageCollecting()
			|| (GIsEditor && FApp::IsGame())) // Is in PIE mode
		{
			return false;
		}

		return true;
	}

	bool FAutoReimportManager::IsAssetAutoReimportEnabled(UObject* InAsset) const
	{
		return AutoReimportObjectsMap.Contains(InAsset) || PendingAutoReimportObjects.Contains(InAsset);
	}

	bool FAutoReimportManager::SetAssetAutoReimport(UObject* InAsset, bool bEnabled)
	{
		return bEnabled ? EnableAssetAutoReimport(InAsset) : DisableAssetAutoReimport(InAsset);
	}

	void FAutoReimportManager::OnExternalSourceInvalidated(const TSharedRef<FExternalSource>& ExternalSource)
	{
		PendingInvalidateQueue.Enqueue(ExternalSource);
	}

	bool FAutoReimportManager::EnableAssetAutoReimport(UObject* InAsset)
	{
		// Enable auto-reimport for InAsset.
		FAssetData AssetData(InAsset);
		const FSourceUri Uri = FSourceUri::FromAssetData(AssetData);

		if (Uri.IsValid() && !AutoReimportObjectsMap.Contains(InAsset))
		{
			if (TSharedPtr<FExternalSource> ExternalSource = IExternalSourceModule::GetOrCreateExternalSource(Uri))
			{
				// Register a delegate triggering a reimport task on the external source snapshotupdate event.
				// That way the asset will be auto-reimported and kept up-to-date.
				FDelegateHandle DelegateHandle = ExternalSource->OnExternalSourceChanged.AddRaw(this, &FAutoReimportManager::OnExternalSourceChanged);

				TSharedRef<FExternalSource> ExternalSourceRef = ExternalSource.ToSharedRef();
				TSharedRef<FAutoReimportInfo> AutoReimportInfo = MakeShared<FAutoReimportInfo>(InAsset, ExternalSourceRef, DelegateHandle);

				AutoReimportObjectsMap.Add(InAsset, AutoReimportInfo);
				AutoReimportExternalSourcesMap.Add(ExternalSourceRef, AutoReimportInfo);
				//TODO, we can't just async load here, we must add some sort of "auto-reimport capability" to the external source.
				ExternalSource->AsyncLoad();
				PendingAutoReimportObjects.Remove(InAsset);
				return true;
			}
			else
			{
				bool bIsAlreadySet = false;
				PendingAutoReimportObjects.Add(InAsset, &bIsAlreadySet);
				if (!bIsAlreadySet)
				{
					return true;
				}
			}
		}

		return false;
	}

	bool FAutoReimportManager::DisableAssetAutoReimport(UObject* InAsset)
	{
		// Disable auto-reimport for InAsset.
		if (const TSharedRef<FAutoReimportInfo>* AutoReimportInfoPtr = AutoReimportObjectsMap.Find(InAsset))
		{
			// Holding a local reference to the FAutoReimportInfo to ensure its lifetime while we are cleaning up.
			const TSharedRef<FAutoReimportInfo> AutoReimportInfo(*AutoReimportInfoPtr);

			AutoReimportInfo->ExternalSource->OnExternalSourceChanged.Remove(AutoReimportInfo->ImportDelegateHandle);
			AutoReimportObjectsMap.Remove(InAsset);
			AutoReimportExternalSourcesMap.RemoveSingle(AutoReimportInfo->ExternalSource, AutoReimportInfo);
		}
		else
		{
			PendingAutoReimportObjects.Remove(InAsset);
		}

		return true;
	}

	void FAutoReimportManager::UpdateModifiedRegisteredAsset(UObject* InAsset)
	{
		if (!IsAssetAutoReimportEnabled(InAsset))
		{
			// Asset is not registered, nothing to update.
			return;
		}

		const FAssetData AssetData(InAsset);
		const FSourceUri Uri = FSourceUri::FromAssetData(AssetData);
		const TSharedPtr<FExternalSource> UpdatedExternalSource = Uri.IsValid() ? IExternalSourceModule::GetOrCreateExternalSource(Uri) : nullptr;
		if (!UpdatedExternalSource)
		{
			// Asset was registered for auto reimport but no longer has a DirectLink source, disable auto reimport.
			DisableAssetAutoReimport(InAsset);
			return;
		}

		const bool bHasDirectLinkSourceChanged = AutoReimportObjectsMap.FindChecked(InAsset)->ExternalSource != UpdatedExternalSource;
		if (bHasDirectLinkSourceChanged)
		{
			// The source changed but is still a DirectLink source.
			// Since the auto-reimport is asset-driven and not source-driven, keep the auto reimport active with the new source.	
			DisableAssetAutoReimport(InAsset);
			EnableAssetAutoReimport(InAsset);
		}
	}

	void FAutoReimportManager::OnExternalSourceChanged(const TSharedRef<FExternalSource>& ExternalSource)
	{
		// Put the reimport request in a thread-safe queue that will be processed in the main thread.
		PendingReimportQueue.Enqueue(ExternalSource);
	}

	void FAutoReimportManager::TriggerAutoReimportOnExternalSource(const TSharedRef<FExternalSource>& ExternalSource)
	{
		TArray<TSharedRef<FAutoReimportInfo>> AutoReimportInfos;
		AutoReimportExternalSourcesMap.MultiFind(ExternalSource, AutoReimportInfos);
		if (AutoReimportInfos.Num() == 0)
		{
			return;
		}

		for (const TSharedRef<FAutoReimportInfo>& AutoReimportInfo : AutoReimportInfos)
		{
			if (UObject* Asset = AutoReimportInfo->TargetObject.Get())
			{
				TriggerAutoReimportOnAsset(AutoReimportInfo->TargetObject.Get());
			}
		}
	}

	void FAutoReimportManager::TriggerAutoReimportOnAsset(UObject* Asset)
	{
		const FAssetData AssetData(Asset);
		const FSourceUri Uri = FSourceUri::FromAssetData(AssetData);

		// Make sure we are not triggering a reimport on an asset that doesn't have a DirectLink source.
		if (Uri.IsValid())
		{
#if WITH_EDITOR
			FReimportManager::Instance()->Reimport(Asset, /*bAskForNewFileIfMissing*/ false, /*bShowNotification*/ true, /*PreferredReimportFile*/ TEXT(""), /*SpecifiedReimportHandler */ nullptr, /*SourceFileIndex*/ INDEX_NONE, /*bForceNewFile*/ false, /*bAutomated*/ true);
#else
			// TODO: Find a way to trigger re-import at runtime
#endif
		}
		else
		{
			DisableAssetAutoReimport(Asset);
		}
	}
}