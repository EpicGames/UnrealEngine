// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Queue.h"
#include "Containers/Map.h"
#include "Delegates/IDelegateInstance.h"
#include "Logging/LogMacros.h"
#include "Templates/SharedPointer.h"
#include "Tickable.h"
#include "UObject/SoftObjectPtr.h"

class UObject;

namespace UE::DatasmithImporter
{
	class FExternalSource;
	struct FAutoReimportInfo;

	class FAutoReimportManager : public FTickableGameObject, public TSharedFromThis<FAutoReimportManager>
	{

	public:
		/* Begin FTickableGameObject Interface */
		virtual void Tick(float DeltaTime) override;
		virtual bool IsTickable() const override { return PendingAutoReimportObjects.Num() > 0 || !PendingReimportQueue.IsEmpty() || !PendingInvalidateQueue.IsEmpty(); }
		virtual TStatId GetStatId() const override;
		virtual bool IsTickableWhenPaused() const override { return true; }
		virtual bool IsTickableInEditor() const override { return true; }
		/* End FTickableGameObject Interface */

		bool IsAssetAutoReimportEnabled(UObject* InAsset) const;

		bool SetAssetAutoReimport(UObject* InAsset, bool bEnabled);

		void OnExternalSourceInvalidated(const TSharedRef<FExternalSource>& ExternalSource);

		/**
		 * Update the internal registration a given asset registered for auto-reimport.
		 * Modified assets may no longer have a DirectLink source and we must keep track of such changes.
		 */
		void UpdateModifiedRegisteredAsset(UObject* InAsset);

	private:

		bool CanTriggerReimport() const;

		bool EnableAssetAutoReimport(UObject* InAsset);

		bool DisableAssetAutoReimport(UObject* InAsset);

		void OnExternalSourceChanged(const TSharedRef<FExternalSource>& ExternalSource);

		void TriggerAutoReimportOnExternalSource(const TSharedRef<FExternalSource>& ExternalSource);

		void TriggerAutoReimportOnAsset(UObject* Asset);

	private:

		TSet<TSoftObjectPtr<UObject>> PendingAutoReimportObjects;

		TMap<UObject*, TSharedRef<FAutoReimportInfo>> AutoReimportObjectsMap;

		TMultiMap<TSharedRef<FExternalSource>, TSharedRef<FAutoReimportInfo>> AutoReimportExternalSourcesMap;

		TQueue<TSharedPtr<FExternalSource>> PendingReimportQueue;

		TQueue<TSharedPtr<FExternalSource>> PendingInvalidateQueue;
	};
}