// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectLinkAssetObserver.h"

#include "AssetRegistry/AssetData.h"
#include "DirectLinkManager.h"
#include "DirectLinkUriResolver.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "SourceUri.h"
#include "UObject/UObjectGlobals.h"

namespace UE::DatasmithImporter
{
	FDirectLinkAssetObserver::FDirectLinkAssetObserver(FDirectLinkManager& InManager)
		: Manager(InManager)
	{
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();
		AssetRegistry.OnAssetRemoved().AddRaw(this, &FDirectLinkAssetObserver::AssetRemovedEvent);
		AssetRegistry.OnAssetUpdated().AddRaw(this, &FDirectLinkAssetObserver::AssetUpdatedEvent);

#if WITH_EDITOR
		FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(this, &FDirectLinkAssetObserver::OnObjectPropertyChanged);
#endif
	}

	FDirectLinkAssetObserver::~FDirectLinkAssetObserver()
	{
		if (FModuleManager::Get().IsModuleLoaded(AssetRegistryConstants::ModuleName))
		{
			IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();
			AssetRegistry.OnAssetRemoved().RemoveAll(this);
			AssetRegistry.OnAssetUpdated().RemoveAll(this);
		}

#if WITH_EDITOR
		FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
#endif
	}

	void FDirectLinkAssetObserver::AssetRemovedEvent(const FAssetData& AssetData)
	{
		const FAssetTagValueRef ValueRef = AssetData.TagsAndValues.FindTag(FSourceUri::GetAssetDataTag());

		if (ValueRef.IsSet() && ValueRef.GetValue().StartsWith(FDirectLinkUriResolver::GetDirectLinkScheme()))
		{
			UObject* DirectLinkAsset = AssetData.GetAsset();
			if (DirectLinkAsset)
			{
				Manager.SetAssetAutoReimport(DirectLinkAsset, false);
			}
		}
	}

	void FDirectLinkAssetObserver::AssetUpdatedEvent(const FAssetData& AssetData)
	{
		if (AssetData.IsAssetLoaded())
		{
			if (UObject* UpdatedAsset = AssetData.GetAsset())
			{
				Manager.UpdateModifiedRegisteredAsset(UpdatedAsset);
			}
		}
	}

	void FDirectLinkAssetObserver::OnObjectPropertyChanged(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent)
	{
		// The source URI may have changed.
		Manager.UpdateModifiedRegisteredAsset(ObjectBeingModified);
	}

}