// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

struct FAssetData;
struct FPropertyChangedEvent;
class UObject;

namespace UE::DatasmithImporter
{
	class FDirectLinkExternalSource;
	class FDirectLinkManager;
	struct FAutoReimportInfo;

	class FDirectLinkAssetObserver
	{
	public:
		FDirectLinkAssetObserver(FDirectLinkManager& Manager);

		~FDirectLinkAssetObserver();

	private:
		void AssetRemovedEvent(const FAssetData& AssetData);

		void AssetUpdatedEvent(const FAssetData& AssetData);

		void OnObjectPropertyChanged(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent);

	private:
		FDirectLinkManager& Manager;
	};
}