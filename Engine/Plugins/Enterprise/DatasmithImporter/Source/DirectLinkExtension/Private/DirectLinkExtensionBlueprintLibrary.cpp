// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectLinkExtensionBlueprintLibrary.h"

#include "DirectLinkExtensionModule.h"
#include "DirectLinkExternalSource.h"
#include "DirectLinkUriResolver.h"
#include "IDirectLinkManager.h"
#include "SourceUri.h"

TArray<FString> UDirectLinkExtensionBlueprintLibrary::GetAvailableDirectLinkSourcesUri()
{
	using namespace UE::DatasmithImporter;

	TArray<FString> SourceUris;
	const IDirectLinkManager& Manager = IDirectLinkExtensionModule::Get().GetManager();
	const TArray<TSharedRef<FDirectLinkExternalSource>> ExternalSources = Manager.GetExternalSourceList();

	for (const TSharedRef<FDirectLinkExternalSource>& ExternalSource : ExternalSources)
	{
		SourceUris.Add(ExternalSource->GetSourceUri().ToString());
	}

	return SourceUris;
}

bool UDirectLinkExtensionBlueprintLibrary::ParseDirectLinkSourceUri(const FString& SourceUriString, FString& OutComputerName, FString& OutEndpointName, FString& OutExecutableName, FString& OutSourceName)
{
	using namespace UE::DatasmithImporter;

	const FSourceUri SourceUri(SourceUriString);
	if (TOptional<FDirectLinkSourceDescription> SourceDescription = FDirectLinkUriResolver::TryParseDirectLinkUri(SourceUri))
	{
		OutComputerName = SourceDescription->ComputerName;
		OutEndpointName = SourceDescription->EndpointName;
		OutExecutableName = SourceDescription->ExecutableName;
		OutSourceName = SourceDescription->SourceName;

		return true;
	}

	return false;
}
