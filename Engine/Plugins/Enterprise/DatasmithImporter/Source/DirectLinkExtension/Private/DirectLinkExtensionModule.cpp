// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectLinkExtensionModule.h"

#include "DirectLinkExternalSource.h"
#include "DirectLinkManager.h"
#include "DirectLinkUriResolver.h"

#include "ExternalSourceModule.h"
#include "IUriManager.h"

#define LOCTEXT_NAMESPACE "DirectLinkEditorModule"

namespace UE::DatasmithImporter
{
	class FDirectLinkExtensionModule : public IDirectLinkExtensionModule
	{
		const FName DirectLinkUriResolverName = TEXT("DirectLinkUriResolverName");

	public:
		virtual void StartupModule() override
		{
			// This will instantiate the DirectLinkManager.
			FDirectLinkManager::GetInstance();
			IUriManager& UriManager = IExternalSourceModule::Get().GetManager();
			UriManager.RegisterResolver(DirectLinkUriResolverName, MakeShared<FDirectLinkUriResolver>());
		}

		virtual void ShutdownModule() override
		{
			if (IExternalSourceModule::IsAvailable())
			{
				IUriManager& UriManager = IExternalSourceModule::Get().GetManager();
				UriManager.UnregisterResolver(DirectLinkUriResolverName);
			}

			FDirectLinkManager::ResetInstance();
		}

		virtual IDirectLinkManager& GetManager() const override
		{
			return FDirectLinkManager::GetInstance();
		}

		virtual void OverwriteUriResolver(const TSharedRef<UE::DatasmithImporter::IUriResolver>& UriResolver)
		{
			IUriManager& UriManager = IExternalSourceModule::Get().GetManager();

			// Unregister last registered URI resolver
			UriManager.UnregisterResolver(DirectLinkUriResolverName);

			UriManager.RegisterResolver(DirectLinkUriResolverName, UriResolver);
		}

	};
}

IMPLEMENT_MODULE(UE::DatasmithImporter::FDirectLinkExtensionModule, DirectLinkExtension);

#undef LOCTEXT_NAMESPACE
