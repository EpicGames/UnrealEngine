// Copyright Epic Games, Inc. All Rights Reserved.


#include "DirectLinkExternalSource.h"

#include "DirectLinkExtensionModule.h"
#include "DirectLinkManager.h"

#include "Async/Async.h"
#include "DirectLinkEndpoint.h"
#include "DirectLinkDeltaConsumer.h"
#include "DirectLinkElementSnapshot.h"
#include "DirectLinkMisc.h"
#include "DirectLinkSceneSnapshot.h"
#include "Misc/AsyncTaskNotification.h"

#define LOCTEXT_NAMESPACE "DirectLinkExternalSource"

namespace UE::DatasmithImporter
{
	/**
	 * Wrapper DirectLink SceneReceiver, used to notify the FExternalSource when the DatasmithScene is loaded.
	 */
	class FInternalDirectLinkSceneReceiverWrapper : public DirectLink::ISceneReceiver
	{
	public:
		FInternalDirectLinkSceneReceiverWrapper(const TSharedRef<DirectLink::ISceneReceiver>& InSceneReceiver, const TSharedRef<FDirectLinkExternalSource>& InDirectLinkExternalSource)
			: SceneReceiver(InSceneReceiver)
			, DirectLinkExternalSource(InDirectLinkExternalSource)
		{}

		virtual void FinalSnapshot(const DirectLink::FSceneSnapshot& SceneSnapshot) override
		{
			SceneReceiver->FinalSnapshot(SceneSnapshot);
			if (TSharedPtr<FDirectLinkExternalSource> PinnedExternalSource = DirectLinkExternalSource.Pin())
			{
				if (TSharedPtr<IDatasmithScene> Scene = PinnedExternalSource->GetDatasmithScene())
				{
					// Even though the scene is already loaded, it needs to be loaded inside the translator before we can load additional payload with it.
					PinnedExternalSource->TranslatorLoadScene(Scene.ToSharedRef());
				}

				PinnedExternalSource->CachedHash = GenerateSceneSnapshotHash(SceneSnapshot);
				PinnedExternalSource->TriggerOnExternalSourceChanged();
			}
		}

	private:

		FMD5Hash GenerateSceneSnapshotHash(const DirectLink::FSceneSnapshot& SceneSnapshot)
		{
			using namespace DirectLink;
			FMD5Hash Hash;
			FMD5 SceneMD5Hash;

			SceneMD5Hash.Update((uint8*)&SceneSnapshot.SceneId, sizeof(FSceneGraphId));
			for (const TPair<FSceneGraphId, TSharedRef<FElementSnapshot>>& ElementPair : SceneSnapshot.Elements)
			{
				FElementHash ElementHash(ElementPair.Value->GetHash());
				SceneMD5Hash.Update((uint8*)&ElementPair.Key, sizeof(FSceneGraphId));
				SceneMD5Hash.Update((uint8*)&ElementHash, sizeof(FElementHash));
			}

			Hash.Set(SceneMD5Hash);
			return Hash;
		}

		TSharedRef<DirectLink::ISceneReceiver> SceneReceiver;
		TWeakPtr<FDirectLinkExternalSource> DirectLinkExternalSource;
	};

	FDirectLinkExternalSource::~FDirectLinkExternalSource()
	{
		ensureMsgf(!DestinationHandle.IsValid(), TEXT("Invalidate() was not called before the FDirectLinkExternalSource destruction. This may have left dangling pointers in the Endpoint."));
	}

	FExternalSourceCapabilities FDirectLinkExternalSource::GetCapabilities() const
	{ 
		FExternalSourceCapabilities Capabilities;
		Capabilities.bSupportAsynchronousLoading = true;

		return Capabilities;
	}

	void FDirectLinkExternalSource::Initialize(const FString& InSourceName, const FGuid& InSourceHandle, const FGuid& InDestinationHandle)
	{
		SourceName = InSourceName;
		SourceHandle = InSourceHandle;
		DestinationHandle = InDestinationHandle;
	}

	bool FDirectLinkExternalSource::OpenStream()
	{
		using namespace DirectLink;

		if (SourceHandle.IsValid() && DestinationHandle.IsValid())
		{
			const FEndpoint::EOpenStreamResult Result = FDirectLinkManager::GetInstance().GetEndpoint().OpenStream(SourceHandle, DestinationHandle);

			bIsStreamOpen = Result == FEndpoint::EOpenStreamResult::Opened || Result == FEndpoint::EOpenStreamResult::AlreadyOpened;

			return bIsStreamOpen;			
		}

		return false;
	}

	void FDirectLinkExternalSource::CloseStream()
	{
		if (SourceHandle.IsValid() && DestinationHandle.IsValid())
		{
			FDirectLinkManager::GetInstance().GetEndpoint().CloseStream(SourceHandle, DestinationHandle);
			bIsStreamOpen = false;
		}
	}

	void FDirectLinkExternalSource::Invalidate()
	{
		CloseStream();
		FDirectLinkManager::GetInstance().GetEndpoint().RemoveDestination(DestinationHandle);

		ClearOnExternalSourceLoadedDelegates();
		DestinationHandle.Invalidate();
	}

	TSharedPtr<DirectLink::ISceneReceiver> FDirectLinkExternalSource::GetSceneReceiver(const DirectLink::IConnectionRequestHandler::FSourceInformation& Source)
	{
		if (!InternalSceneReceiver)
		{
			if (TSharedPtr<DirectLink::ISceneReceiver> SceneReceiver = GetSceneReceiverInternal(Source))
			{
				InternalSceneReceiver = MakeShared<FInternalDirectLinkSceneReceiverWrapper>(SceneReceiver.ToSharedRef(), StaticCastSharedRef<FDirectLinkExternalSource>(AsShared()));
			}
		}

		return InternalSceneReceiver;
	}

	bool FDirectLinkExternalSource::StartAsyncLoad()
	{
		if (!IsAvailable())
		{
			return false;
		}
		
		if (!IsStreamOpen())
		{
			return OpenStream();
		}

		// No need to do anything, the stream is already opened and the scene will be loaded on the next DirectLink sync.
		return true;
	}
}
#undef LOCTEXT_NAMESPACE