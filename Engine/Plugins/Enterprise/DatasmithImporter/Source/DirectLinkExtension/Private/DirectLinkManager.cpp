// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectLinkManager.h"

#include "DirectLinkAssetObserver.h"
#include "DirectLinkExtensionSettings.h"
#include "DirectLinkExternalSource.h"
#include "DirectLinkUriResolver.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Async/Async.h"
#include "Misc/CoreDelegates.h"
#include "UObject/UObjectGlobals.h"

#define LOCTEXT_NAMESPACE "DirectLinkManager"

DEFINE_LOG_CATEGORY(LogDirectLinkManager);


namespace UE::DatasmithImporter
{
	TUniquePtr<FDirectLinkManager> FDirectLinkManager::Instance = nullptr;

	FDirectLinkAutoReconnectManager::FDirectLinkAutoReconnectManager(FDirectLinkManager& InManager)
		: Manager(InManager)
		, bShouldRun(true)
	{
		if (const UDirectLinkExtensionSettings* DefaultSettings = GetDefault<UDirectLinkExtensionSettings>())
		{
			bAutoReconnectEnabled = DefaultSettings->bAutoReconnect;
			ReconnectionDelayInSeconds = DefaultSettings->ReconnectionDelayInSeconds;
		}

		FCoreDelegates::OnPreExit.AddRaw(this, &FDirectLinkAutoReconnectManager::Stop);
	}

	FDirectLinkAutoReconnectManager::~FDirectLinkAutoReconnectManager()
	{
		FCoreDelegates::OnPreExit.RemoveAll(this);
		if (CompletedFuture.IsValid())
		{
			Stop();
			CompletedFuture.Wait();
		}
	}

	bool FDirectLinkAutoReconnectManager::Start()
	{
		if (bAutoReconnectEnabled && (!CompletedFuture.IsValid() || CompletedFuture.IsReady()))
		{
			bShouldRun = true;
			CompletedFuture = Async(EAsyncExecution::ThreadPool, [this]() {	Run(); });

			return true;
		}

		return false;
	}

	void FDirectLinkAutoReconnectManager::Stop()
	{
		bShouldRun = false;
	}

	void FDirectLinkAutoReconnectManager::Run()
	{
		const float CurrentTime = FPlatformTime::Seconds();
		const float TimeSinceLastTry = CurrentTime - LastTryTime;

		if (TimeSinceLastTry < ReconnectionDelayInSeconds)
		{
			FPlatformProcess::Sleep(ReconnectionDelayInSeconds - TimeSinceLastTry);
		}

		bool bHasSourceToReconnect = false;
		{
			FWriteScopeLock ReconnectionScopeLock(Manager.ReconnectionListLock);
			for (int32 Index = Manager.ExternalSourcesToReconnect.Num() - 1; Index >= 0; --Index)
			{
				if (Manager.ExternalSourcesToReconnect[Index]->OpenStream())
				{
					Manager.ExternalSourcesToReconnect.RemoveAtSwap(Index);
				}
			}
			bHasSourceToReconnect = Manager.ExternalSourcesToReconnect.Num() > 0;
		}

		LastTryTime = FPlatformTime::Seconds();

		if (bShouldRun && bHasSourceToReconnect)
		{
			// Could not reconnect, go back to the ThreadPool and try again later.
			CompletedFuture = Async(EAsyncExecution::ThreadPool, [this]() {	Run(); });
		}
	}

	FDirectLinkManager::FDirectLinkManager()
		: Endpoint(MakeUnique<DirectLink::FEndpoint>(TEXT("UE5-Editor")))
		, AssetObserver(MakeUnique<FDirectLinkAssetObserver>(*this))
		, ReconnectionManager(MakeUnique<FDirectLinkAutoReconnectManager>(*this))
		, AutoReimportManger(MakeShared<FAutoReimportManager>())
	{
		Endpoint->AddEndpointObserver(this);

		ensureMsgf(!Instance, TEXT("There can only be one instance of FDirectLinkManager."));
	}

	FDirectLinkManager::~FDirectLinkManager() = default;

	void FDirectLinkManager::Clear()
	{
		Endpoint->RemoveEndpointObserver(this);

		// Make sure all DirectLink external source become stales and their delegates stripped.
		for (const TPair<DirectLink::FSourceHandle, TSharedRef<FDirectLinkExternalSource>>& UriExternalSourcePair : DirectLinkSourceToExternalSourceMap)
		{
			UriExternalSourcePair.Value->Invalidate();
		}
	}

	TSharedPtr<FDirectLinkExternalSource> FDirectLinkManager::GetOrCreateExternalSource(const DirectLink::FSourceHandle& SourceHandle)
	{
		if (TSharedRef<FDirectLinkExternalSource>* ExternalSourceEntry = DirectLinkSourceToExternalSourceMap.Find(SourceHandle))
		{
			// A DirectLinkExternalSource already exists for this SourceHandle.
			return *ExternalSourceEntry;
		}
		else if (RegisteredExternalSourcesInfo.Num() > 0)
		{
			DirectLink::FRawInfo::FDataPointInfo* SourceDataPointInfo;
			{
				FRWScopeLock ScopeLock(RawInfoLock, FRWScopeLockType::SLT_ReadOnly);
				SourceDataPointInfo = RawInfoCache.DataPointsInfo.Find(SourceHandle);
			}
			FSourceUri ExternalSourceUri(GetUriFromSourceHandle(SourceHandle));

			if (SourceDataPointInfo && ExternalSourceUri.IsValid())
			{
				const FString& SourceName = SourceDataPointInfo->Name;
				const FString ExternalSourceName = FString::Printf(TEXT("%s_%s_ExternalSource"), *SourceName, *SourceHandle.ToString());
				const DirectLink::IConnectionRequestHandler::FSourceInformation SourceInfo{ SourceHandle };

				for (const FDirectLinkExternalSourceRegisterInformation& RegisteredInfo : RegisteredExternalSourcesInfo)
				{
					TSharedRef<FDirectLinkExternalSource> SpawnedExternalSource = RegisteredInfo.SpawnFunction(ExternalSourceUri);

					if (SpawnedExternalSource->CanOpenNewConnection(SourceInfo))
					{
						const FGuid DestinationHandle = Endpoint->AddDestination(ExternalSourceName, DirectLink::EVisibility::Private, SpawnedExternalSource);
						SpawnedExternalSource->Initialize(SourceName, SourceHandle, DestinationHandle);

						DirectLinkSourceToExternalSourceMap.Add(SourceHandle, SpawnedExternalSource);
						
						return SpawnedExternalSource;
					}
				}
			}
		}

		return nullptr;
	}

	TSharedPtr<FDirectLinkExternalSource> FDirectLinkManager::GetOrCreateExternalSource(const FSourceUri& Uri)
	{
		DirectLink::FSourceHandle SourceHandle;
		TSharedPtr<FDirectLinkExternalSource> ExternalSource;

		if (TOptional<FDirectLinkSourceDescription> SourceDescription = FDirectLinkUriResolver::TryParseDirectLinkUri(Uri))
		{
			// Try getting the external source with the explicit id first.
			if (SourceDescription->SourceId)
			{
				SourceHandle = SourceDescription->SourceId.GetValue();
				if (SourceHandle.IsValid())
				{
					ExternalSource = GetOrCreateExternalSource(SourceHandle);
				}
			}
			
			// Could not retrieve the external source from the id, fall back on the first source matching the source description.
			if (!ExternalSource)
			{
				SourceHandle = ResolveSourceHandleFromDescription(SourceDescription.GetValue());
				if (SourceHandle.IsValid())
				{
					ExternalSource = GetOrCreateExternalSource(SourceHandle);
				}
			}
		}

		return ExternalSource;
	}

	DirectLink::FSourceHandle FDirectLinkManager::ResolveSourceHandleFromDescription(const FDirectLinkSourceDescription& SourceDescription) const
	{
		FRWScopeLock ScopeLock(RawInfoLock, FRWScopeLockType::SLT_ReadOnly);

		for (const auto& EndpointKeyValue : RawInfoCache.EndpointsInfo)
		{
			// Try to find a matching DirectLink source.
			if (EndpointKeyValue.Value.ComputerName == SourceDescription.ComputerName
				&& EndpointKeyValue.Value.ExecutableName == SourceDescription.ExecutableName
				&& EndpointKeyValue.Value.Name == SourceDescription.EndpointName)
			{
				for (const DirectLink::FRawInfo::FDataPointId& SourceInfo : EndpointKeyValue.Value.Sources)
				{
					if (SourceInfo.Name == SourceDescription.SourceName)
					{
						// Source found, returning the handle.
						return SourceInfo.Id;
					}
				}
			}
		}

		//Returning default invalid handle.
		return DirectLink::FSourceHandle();
	}

	void FDirectLinkManager::OnStateChanged(const DirectLink::FRawInfo& RawInfo)
	{
		{
			FRWScopeLock ScopeLock(RawInfoLock, FRWScopeLockType::SLT_Write);
			RawInfoCache = RawInfo;
		}

		UpdateSourceCache();

		CancelEmptySourcesLoading();
	}

	void FDirectLinkManager::UpdateSourceCache()
	{
		FRWScopeLock ScopeLock(RawInfoLock, FRWScopeLockType::SLT_ReadOnly);

		// List the source Id of all current external source. This is used to determine which ones are no longer valid.
		TSet<DirectLink::FSourceHandle> InvalidExternalSourceIds;
		DirectLinkSourceToExternalSourceMap.GetKeys(InvalidExternalSourceIds);

		for (const TPair<FMessageAddress, DirectLink::FRawInfo::FEndpointInfo>& EndpointInfoPair : RawInfoCache.EndpointsInfo)
		{
			if (!EndpointInfoPair.Value.bIsLocal)
			{
				continue;
			}

			for (const DirectLink::FRawInfo::FDataPointId& DataPointId : EndpointInfoPair.Value.Sources)
			{
				if (DataPointId.bIsPublic)
				{
					if (GetOrCreateExternalSource(DataPointId.Id))
					{
						// This source is still valid.
						InvalidExternalSourceIds.Remove(DataPointId.Id);
					}
				}
			}
		}


		TSet<DirectLink::FDestinationHandle> ActiveStreamsSources;
		for (const DirectLink::FRawInfo::FStreamInfo& StreamInfo : RawInfoCache.StreamsInfo)
		{
			if (!(StreamInfo.ConnectionState == DirectLink::EStreamConnectionState::Active
				|| StreamInfo.ConnectionState == DirectLink::EStreamConnectionState::RequestSent))
			{
				continue;
			}

			if (TSharedRef<FDirectLinkExternalSource>* ExternalSource = DirectLinkSourceToExternalSourceMap.Find(StreamInfo.Source))
			{
				ActiveStreamsSources.Add(StreamInfo.Source);
			}
		}

		{
			FWriteScopeLock ReconnectionScopeLock(ReconnectionListLock);

			for (int32 SourceIndex = ExternalSourcesToReconnect.Num() - 1; SourceIndex >= 0; --SourceIndex)
			{
				// If the source no longer exists, then there is no point in trying to reconnect.
				if (InvalidExternalSourceIds.Contains(ExternalSourcesToReconnect[SourceIndex]->GetSourceHandle()))
				{
					ExternalSourcesToReconnect.RemoveAtSwap(SourceIndex);
				}
			}

			for (const TPair<FGuid, TSharedRef<FDirectLinkExternalSource>>& ExternalSourceKeyValue : DirectLinkSourceToExternalSourceMap)
			{
				const TSharedRef<FDirectLinkExternalSource>& ExternalSource = ExternalSourceKeyValue.Value;

				if (ExternalSource->IsStreamOpen() && !ActiveStreamsSources.Contains(ExternalSourceKeyValue.Key))
				{
					// Lost connection, update the external source state and try to reconnect.
					ExternalSource->CloseStream();
					
					if (!ExternalSource->OpenStream())
					{
						// Could not reopen the stream, retry later.
						ExternalSourcesToReconnect.Add(ExternalSource);
						ReconnectionManager->Start();
					}
				}
			}
		}

		// Remove all external sources that are no longer valid.
		for (const DirectLink::FSourceHandle& SourceHandle : InvalidExternalSourceIds)
		{
			InvalidateSource(SourceHandle);
		}
	}

	void FDirectLinkManager::CancelEmptySourcesLoading() const
	{
		FRWScopeLock ScopeLock(RawInfoLock, FRWScopeLockType::SLT_ReadOnly);

		for (const DirectLink::FRawInfo::FStreamInfo& StreamInfo : RawInfoCache.StreamsInfo)
		{
			if (const TSharedRef<FDirectLinkExternalSource>* ExternalSourcePtr = DirectLinkSourceToExternalSourceMap.Find(StreamInfo.Source))
			{
				const TSharedRef<FDirectLinkExternalSource>& ExternalSource = *ExternalSourcePtr;
				
				// We can infer that a DirectLink source is empty (no scene synced) by looking at if its stream is planning to send any data.
				// #ueent_todo: Ideally it would be better to not allow an AsyncLoad to take place in the first time, but we can't know a source is empty 
				//				before actually connecting to it, so this is the best we can do at the current time.
				const bool bStreamIsEmpty = StreamInfo.ConnectionState == DirectLink::EStreamConnectionState::Active
					&& !StreamInfo.CommunicationStatus.IsTransmitting()
					&& StreamInfo.CommunicationStatus.TaskTotal == 0;

				if (bStreamIsEmpty
					&& ExternalSource->IsAsyncLoading()
					&& !ExternalSource->GetDatasmithScene().IsValid())
				{
					ExternalSource->CancelAsyncLoad();
					UE_LOGF(LogDirectLinkManager, Warning, "The DirectLink source \"%ls\" could not be loaded: Nothing to synchronize. Make sure to do a DirectLink sync in your exporter.", *ExternalSource->GetSourceName());
				}
			}
		}
	}

	void FDirectLinkManager::RegisterDirectLinkExternalSource(FDirectLinkExternalSourceRegisterInformation&& RegisterInformation)
	{
		RegisteredExternalSourcesInfo.Add(MoveTemp(RegisterInformation));
	}

	void FDirectLinkManager::UnregisterDirectLinkExternalSource(FName InName)
	{
		for (int32 InfoIndex = RegisteredExternalSourcesInfo.Num() - 1; InfoIndex >= 0; --InfoIndex)
		{
			if (RegisteredExternalSourcesInfo[InfoIndex].Name == InName)
			{
				RegisteredExternalSourcesInfo.RemoveAtSwap(InfoIndex);
				break;
			}
		}
	}

	DirectLink::FEndpoint& FDirectLinkManager::GetEndpoint()
	{
		return *Endpoint.Get();
	}

	void FDirectLinkManager::InvalidateSource(const DirectLink::FSourceHandle& InvalidSourceHandle)
	{
		TSharedRef<FDirectLinkExternalSource> DirectLinkExternalSource = DirectLinkSourceToExternalSourceMap.FindAndRemoveChecked(InvalidSourceHandle);

		AutoReimportManger->OnExternalSourceInvalidated(DirectLinkExternalSource);

		DirectLinkExternalSource->Invalidate();
	}

	TArray<TSharedRef<FDirectLinkExternalSource>> FDirectLinkManager::GetExternalSourceList() const
	{
		TArray<TSharedRef<FDirectLinkExternalSource>> ExternalSources;
		DirectLinkSourceToExternalSourceMap.GenerateValueArray(ExternalSources);
		return ExternalSources;
	}

	FSourceUri FDirectLinkManager::GetUriFromSourceHandle(const DirectLink::FSourceHandle& SourceHandle)
	{
		FRWScopeLock ScopeLock(RawInfoLock, FRWScopeLockType::SLT_ReadOnly);

		if (const DirectLink::FRawInfo::FDataPointInfo* SourceInfo = RawInfoCache.DataPointsInfo.Find(SourceHandle))
		{
			const FString SourceName(SourceInfo->Name);
			if (DirectLink::FRawInfo::FEndpointInfo* EndpointInfo = RawInfoCache.EndpointsInfo.Find(SourceInfo->EndpointAddress))
			{
				const FString EndpointName(EndpointInfo->Name);
				const FString UriPath(EndpointInfo->ComputerName / EndpointInfo->ExecutableName / EndpointInfo->Name / SourceName);
				TMap<FString, FString> UriQuery = { {FDirectLinkUriResolver::GetSourceIdPropertyName(), LexToString(SourceHandle)} };

				return FSourceUri(FDirectLinkUriResolver::GetDirectLinkScheme(), UriPath, UriQuery);
			}
		}

		return FSourceUri();
	}
}

#undef LOCTEXT_NAMESPACE
