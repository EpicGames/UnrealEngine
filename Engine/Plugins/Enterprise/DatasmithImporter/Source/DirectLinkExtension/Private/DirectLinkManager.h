// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDirectLinkManager.h"

#include "AutoReimportManager.h"
#include "SourceUri.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Containers/Queue.h"
#include "Delegates/DelegateCombinations.h"
#include "DirectLinkEndpoint.h"
#include "HAL/CriticalSection.h"
#include "IUriResolver.h"
#include "UObject/SoftObjectPtr.h"

struct FAssetData;

DECLARE_LOG_CATEGORY_EXTERN(LogDirectLinkManager, Log, All);

namespace UE::DatasmithImporter
{
	class FDirectLinkAssetObserver;
	class FDirectLinkExternalSource;
	class FDirectLinkManager;
	struct FDirectLinkSourceDescription;

	class FDirectLinkAutoReconnectManager
	{
	public:
		FDirectLinkAutoReconnectManager(FDirectLinkManager& InManager);

		~FDirectLinkAutoReconnectManager();

		bool Start();

		void Stop();

	private:
		void Run();

		FDirectLinkManager& Manager;

		TAtomic<bool> bShouldRun;

		TFuture<void> CompletedFuture;

		float LastTryTime = 0;

		bool bAutoReconnectEnabled = false;

		float ReconnectionDelayInSeconds = 1;
	};

	class FDirectLinkManager: public IDirectLinkManager, public DirectLink::IEndpointObserver
	{
	public:

		virtual ~FDirectLinkManager();

		/**
		 * Static instance that can be used internally to avoid relying on IDirectLinkExtensionModule to get the DirectLink manager.
		 */
		static FDirectLinkManager& GetInstance()
		{
			if (!Instance)
			{
				Instance = TUniquePtr<FDirectLinkManager>(new FDirectLinkManager());;
				check(Instance);
			}
			return *Instance;
		}
		
		static void ResetInstance()
		{
			if (Instance)
			{
				Instance->Clear();
			}
			Instance.Reset();
		}

		// IEndpointObserver interface begin
		virtual void OnStateChanged(const DirectLink::FRawInfo& RawInfo) override;
		// IEndpointObserver interface end

		// IDirectLinkManager interface begin
		virtual TSharedPtr<FDirectLinkExternalSource> GetOrCreateExternalSource(const DirectLink::FSourceHandle& SourceHandle) override;
		virtual TSharedPtr<FDirectLinkExternalSource> GetOrCreateExternalSource(const FSourceUri& Uri) override;
		virtual DirectLink::FEndpoint& GetEndpoint() override;
		virtual FSourceUri GetUriFromSourceHandle(const DirectLink::FSourceHandle& SourceHandle) override;
#if WITH_EDITOR
		virtual bool IsAssetAutoReimportEnabled(UObject* InAsset) const override { return AutoReimportManger->IsAssetAutoReimportEnabled(InAsset); }
		virtual bool SetAssetAutoReimport(UObject* InAsset, bool bEnabled) override { return AutoReimportManger->SetAssetAutoReimport(InAsset, bEnabled); }
#else
		virtual bool IsAssetAutoReimportEnabled(UObject* InAsset) const override { return false; }
		virtual bool SetAssetAutoReimport(UObject* InAsset, bool bEnabled) override { return false; }
#endif //WITH_EDITOR
		virtual TArray<TSharedRef<FDirectLinkExternalSource>> GetExternalSourceList() const override;
		virtual void UnregisterDirectLinkExternalSource(FName InName) override;
	protected:
		virtual void RegisterDirectLinkExternalSource(FDirectLinkExternalSourceRegisterInformation&& ExternalSourceClass) override;
		// IDirectLinkManager interface end

	public:

		/**
		 * Update the internal registration a given asset registered for auto-reimport.
		 * Modified assets may no longer have a DirectLink source and we must keep track of such changes.
		 */
		void UpdateModifiedRegisteredAsset(UObject* InAsset) { AutoReimportManger->UpdateModifiedRegisteredAsset(InAsset); }

	private:

		FDirectLinkManager();

		/**
		 * Should be called before destructing the FDirectLinkManager.
		 */
		void Clear();

		/**
		 * Remove a DirectLink source from cache and invalidate its associated DirectLinkExternalSource object.
		 * @param InvalidSourceId	The SourceHandle of the invalid DirectLink source.
		 */
		void InvalidateSource(const DirectLink::FSourceHandle& InvalidSourceHandle);

		/**
		 * Return the first SourceHandle matching the description of the source without. Does not use the source id in the description.
		 * @param SourceDescription	The Description of the DirectLink source.
		 * @return	The DirectLink SourceHandle corresponding to the description. If no match was found the FSourceHandle is invalid.
		 */
		DirectLink::FSourceHandle ResolveSourceHandleFromDescription(const FDirectLinkSourceDescription& SourceDescription) const;

		/**
		 * Update internal cache. Create FDirectLinkExternalSource for new DirectLink source and remove expired ones.
		 */
		void UpdateSourceCache();

		void CancelEmptySourcesLoading() const;

	private:

		static TUniquePtr<FDirectLinkManager> Instance;

		/**
		 * Cached DirectLink state.
		 */
		DirectLink::FRawInfo RawInfoCache;

		/**
		 * Lock used to guard RawInfoCache, as the cache is updated from an async thread.
		 */
		mutable FRWLock RawInfoLock;

		TUniquePtr<DirectLink::FEndpoint> Endpoint;

		TUniquePtr<FDirectLinkAssetObserver> AssetObserver;

		TArray<FDirectLinkExternalSourceRegisterInformation> RegisteredExternalSourcesInfo;
		
		TMap<DirectLink::FSourceHandle, TSharedRef<FDirectLinkExternalSource>> DirectLinkSourceToExternalSourceMap;

		FRWLock ReconnectionListLock;
		TArray<TSharedRef<FDirectLinkExternalSource>> ExternalSourcesToReconnect;

		TUniquePtr<FDirectLinkAutoReconnectManager> ReconnectionManager;
		
		TSharedRef<FAutoReimportManager> AutoReimportManger;
		
		friend FDirectLinkAutoReconnectManager;
	};
}