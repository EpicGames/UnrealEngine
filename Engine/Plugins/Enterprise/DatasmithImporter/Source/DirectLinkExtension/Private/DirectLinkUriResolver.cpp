// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectLinkUriResolver.h"

#include "DirectLinkExtensionModule.h"
#include "DirectLinkExternalSource.h"
#include "IDirectLinkManager.h"

#include "Misc/AutomationTest.h"

namespace UE::DatasmithImporter
{
	TSharedPtr<FExternalSource> FDirectLinkUriResolver::GetOrCreateExternalSource(const FSourceUri& Uri) const
	{
		IDirectLinkManager& DirectLinkManager = IDirectLinkExtensionModule::Get().GetManager();
		return DirectLinkManager.GetOrCreateExternalSource(Uri);
	}

	bool FDirectLinkUriResolver::CanResolveUri(const FSourceUri& Uri) const
	{
		return Uri.HasScheme(GetDirectLinkScheme());
	}

	TOptional<FDirectLinkSourceDescription> FDirectLinkUriResolver::TryParseDirectLinkUri(const FSourceUri& Uri)
	{
		if (Uri.HasScheme(GetDirectLinkScheme()))
		{
			const FString UriPath(Uri.GetPath());
			TArray<FString> PathStrings;

			// Try to split the URI path into 4 parts, those parts should correspond to the DirectLink source info.
			if (UriPath.ParseIntoArray(PathStrings, TEXT("/")) == 4)
			{
				FDirectLinkSourceDescription SourceDescription;
				SourceDescription.ComputerName = MoveTemp(PathStrings[0]);
				SourceDescription.ExecutableName = MoveTemp(PathStrings[1]);
				SourceDescription.EndpointName = MoveTemp(PathStrings[2]);
				SourceDescription.SourceName = MoveTemp(PathStrings[3]);

				TMap<FString, FString> QueryKeyValues = Uri.GetQueryMap();
				if (const FString* SourceIdString = QueryKeyValues.Find(GetSourceIdPropertyName()))
				{
					FGuid SourceId;
					LexFromString(SourceId, **SourceIdString);
					SourceDescription.SourceId.Emplace(MoveTemp(SourceId));
				}

				return TOptional<FDirectLinkSourceDescription>(MoveTemp(SourceDescription));
			}
		}

		return TOptional<FDirectLinkSourceDescription>();
	}

	const FString& FDirectLinkUriResolver::GetDirectLinkScheme()
	{
		static FString Scheme(TEXT("directlink"));
		return Scheme;
	}

	const FString& FDirectLinkUriResolver::GetSourceIdPropertyName()
	{
		static FString SourceIdName(TEXT("SourceId"));
		return SourceIdName;
	}

	/**
	 * Automated test to validate DirectLink URI parsing.
	 */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FDirectLinkUriResolverTests, "Editor.Import.Datasmith.ExternalSource.DirectLink URI Parsing", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FDirectLinkUriResolverTests::RunTest(const FString& Parameters)
	{
		const FString ComputerName = TEXT("FooComputer");
		const FString ExecutableName = TEXT("BarDCC");
		const FString EndpointName = TEXT("DatasmithExporter");
		const FString SourceName = TEXT("DummySource");
		const FGuid RandomGuid = FGuid::NewGuid();

		{
			const FSourceUri ValidUri = FSourceUri(FDirectLinkUriResolver::GetDirectLinkScheme(), ComputerName / ExecutableName / EndpointName / SourceName);
			TOptional<FDirectLinkSourceDescription> ParsedSourceDescription = FDirectLinkUriResolver::TryParseDirectLinkUri(ValidUri);
			if (!ParsedSourceDescription.IsSet())
			{
				AddError(TEXT("Could not parse valid directlink source URI"));
				return false;
			}
			else if (ParsedSourceDescription->ComputerName != ComputerName
				|| ParsedSourceDescription->ExecutableName != ExecutableName
				|| ParsedSourceDescription->EndpointName != EndpointName
				|| ParsedSourceDescription->SourceName != SourceName)
			{
				AddError(TEXT("Could not parse valid directlink source URI path"));
				return false;
			}
			else if (ParsedSourceDescription->SourceId.IsSet())
			{
				AddError(TEXT("Parsed a SourceId when there was none"));
				return false;
			}
		}

		{
			TMap<FString, FString> UriQuery{ {FDirectLinkUriResolver::GetSourceIdPropertyName(), LexToString(RandomGuid)} };
			const FSourceUri ValidUriWithSourceId = FSourceUri(FDirectLinkUriResolver::GetDirectLinkScheme(), ComputerName / ExecutableName / EndpointName / SourceName, UriQuery);
			TOptional<FDirectLinkSourceDescription> ParsedSourceDescription = FDirectLinkUriResolver::TryParseDirectLinkUri(ValidUriWithSourceId);
			if (!ParsedSourceDescription.IsSet())
			{
				AddError(TEXT("Could not parse valid directlink source URI with SourceId"));
				return false;
			}
			else if (!ParsedSourceDescription->SourceId.IsSet()
				|| ParsedSourceDescription->SourceId.GetValue() != RandomGuid)
			{
				AddError(TEXT("Could not parse source id"));
				return false;
			}
		}
		
		{
			const FSourceUri InvalidUriInvalidPath = FSourceUri(FDirectLinkUriResolver::GetDirectLinkScheme(), FString() / FString() / FString() / FString());
			if (TOptional<FDirectLinkSourceDescription> ParsedSourceDescription = FDirectLinkUriResolver::TryParseDirectLinkUri(InvalidUriInvalidPath))
			{
				AddError(TEXT("DirectLink URI parsing did not fail on invalid path."));
				return false;
			}
			
			const FSourceUri InvalidUriEmptyPath = FSourceUri(FDirectLinkUriResolver::GetDirectLinkScheme(), FString());
			if (TOptional<FDirectLinkSourceDescription> ParsedSourceDescription = FDirectLinkUriResolver::TryParseDirectLinkUri(InvalidUriEmptyPath))
			{
				AddError(TEXT("DirectLink URI parsing did not fail on empty path."));
				return false;
			}
		}

		return true;
	}
}