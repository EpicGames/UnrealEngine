// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "DirectLinkExtensionBlueprintLibrary.generated.h"

UCLASS(MinimalAPI, meta = (ScriptName = "DirectLinkExtensionLibrary"))
class UDirectLinkExtensionBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | DirectLink")
	static TArray<FString> GetAvailableDirectLinkSourcesUri();

	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | DirectLink")
	static bool ParseDirectLinkSourceUri(const FString& SourceUriString, FString& OutComputerName, FString& OutEndpointName, FString& OutExecutableName, FString& OutSourceName);
};