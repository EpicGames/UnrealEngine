// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDirectLinkManager.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"		// For inline LoadModuleChecked()

#define DIRECTLINKEXTENSION_MODULE_NAME TEXT("DirectLinkExtension")

namespace UE::DatasmithImporter
{
	class IUriResolver;
}

namespace DirectLink
{
	class FEndpoint;
}

class IDirectLinkExtensionModule : public IModuleInterface
{
public:
	/**
	 * Singleton-like access to IDirectLinkEditorModule
	 *
	 * @return Returns IDirectLinkEditorModule singleton instance, loading the module on demand if needed
	 */
	static inline IDirectLinkExtensionModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IDirectLinkExtensionModule>(DIRECTLINKEXTENSION_MODULE_NAME);
	}

	/**
	 * Checks to see if this module is loaded and ready.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(DIRECTLINKEXTENSION_MODULE_NAME);
	}

	static DirectLink::FEndpoint& GetEndpoint()
	{
		return Get().GetManager().GetEndpoint();
	}

	/**
	 * Overwrite the UriResolver of the DirectLinkExtension module.
	 */
	virtual void OverwriteUriResolver(const TSharedRef<UE::DatasmithImporter::IUriResolver>& UriResolver) = 0;

	/**
	 * Return the DirectLinkManager singleton.
	 */
	virtual UE::DatasmithImporter::IDirectLinkManager& GetManager() const = 0;
};

