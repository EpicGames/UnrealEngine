// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"

#include "DirectLinkExtensionSettings.generated.h"

UCLASS(config=Editor)
class DIRECTLINKEXTENSION_API UDirectLinkExtensionSettings : public UObject
{
	GENERATED_BODY()

public:
	/**
	 * Whether or not to attempt reconnecting lost connection.
	 */
	UPROPERTY(config)
	bool bAutoReconnect = true;

	/**
	 * Time in seconds to wait between reconnection attempts.
	 */
	UPROPERTY(config, meta = (ClampMin="0.1"))
	float ReconnectionDelayInSeconds = 5.f;
};