// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ExternalSource.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "DirectLinkConnectionRequestHandler.h"
#include "DirectLinkDeltaConsumer.h"

namespace UE::DatasmithImporter
{
	class FDirectLinkExternalSource;
}

DECLARE_MULTICAST_DELEGATE_OneParam(OnSnapshotUpdateDelegate, const TSharedRef<UE::DatasmithImporter::FDirectLinkExternalSource>&)

namespace UE::DatasmithImporter
{
	/**
	 * DirectLinkExternalSouce used for referencing DirectLink source at import.
	 */
	class DIRECTLINKEXTENSION_API FDirectLinkExternalSource : public FExternalSource, public DirectLink::IConnectionRequestHandler
	{
	public:
		explicit FDirectLinkExternalSource(const FSourceUri& InSourceUri)
			: FExternalSource(InSourceUri)
		{}

		virtual ~FDirectLinkExternalSource();

		// FExternalSource interface begin
		virtual FString GetSourceName() const override { return SourceName; }
		virtual bool IsAvailable() const override { return SourceHandle.IsValid(); };
		virtual bool IsOutOfSync() const override { return !(IsStreamOpen() && GetDatasmithScene()); }
		virtual FMD5Hash GetSourceHash() const override { return CachedHash; }
		virtual FExternalSourceCapabilities GetCapabilities() const override;
	protected:
		virtual TSharedPtr<IDatasmithScene> LoadImpl() override { return nullptr; }
		virtual bool StartAsyncLoad() override;
		// FExternalSource interface end

	public:
		// DirectLink::IConnectionRequestHandler interface begin.
		virtual bool CanOpenNewConnection(const DirectLink::IConnectionRequestHandler::FSourceInformation& Source) = 0;
		virtual TSharedPtr<DirectLink::ISceneReceiver> GetSceneReceiver(const DirectLink::IConnectionRequestHandler::FSourceInformation& Source) override final;
		// DirectLink::IConnectionRequestHandler interface end

		/**
		 * Initialize the FDirectLinkExternalSource after its creation.
		 */
		void Initialize(const FString& InSourceName, const FGuid& InSourceHandle, const FGuid& InDestinationHandle);

		/**
		 * Try to open a DirectLink stream to the source.
		 * @return True if the operation was successful.
		 */
		bool OpenStream();

		void CloseStream();

		bool IsStreamOpen() const { return bIsStreamOpen; }

		/**
		 * Clear all delegates and make this FDirectLinkExternalSource stale and unavailable.
		 */
		void Invalidate();

		const FGuid& GetSourceHandle() const { return SourceHandle; }

		const FGuid& GetDestinationHandle() const { return DestinationHandle; }

	protected:
		/**
		 * Used by GetSceneReceiver() to create a wrapper around the ISceneReceiver returned by this function.
		 */
		virtual TSharedPtr<DirectLink::ISceneReceiver> GetSceneReceiverInternal(const DirectLink::IConnectionRequestHandler::FSourceInformation& Source) = 0;

	private:

		FGuid SourceHandle;

		FGuid DestinationHandle;

		FString SourceName;

		bool bIsStreamOpen = false;

		TSharedPtr<class FInternalDirectLinkSceneReceiverWrapper> InternalSceneReceiver;

		FMD5Hash CachedHash;

		friend class FInternalDirectLinkSceneReceiverWrapper;
	};
}