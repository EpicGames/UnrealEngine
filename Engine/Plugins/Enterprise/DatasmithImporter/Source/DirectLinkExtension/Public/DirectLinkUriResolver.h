// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "IUriResolver.h"
#include "Misc/Guid.h"

namespace DirectLink
{
	using FSourceHandle = FGuid;
}

namespace UE::DatasmithImporter
{
	class FDirectLinkExternalSource;

	struct FDirectLinkSourceDescription
	{
		FString ComputerName;
		FString ExecutableName;
		FString EndpointName;
		FString SourceName;
		TOptional<FGuid> SourceId;
	};

	class DIRECTLINKEXTENSION_API FDirectLinkUriResolver : public IUriResolver
	{
	public:
		// IUriResolver interface begin
		virtual TSharedPtr<FExternalSource> GetOrCreateExternalSource(const FSourceUri& Uri) const override;
		virtual bool CanResolveUri(const FSourceUri& Uri) const override;
		virtual FName GetScheme() const override { return FName(GetDirectLinkScheme()); }
#if WITH_EDITOR
		virtual TSharedPtr<FExternalSource> BrowseExternalSource(const FSourceUri& DefaultSourceUri) const override
		{
			return {};
		}
#endif
		// IUriResolver interface end

		/**
		 * Try to parse the DirectLink source description components from an URI. 
		 * Return an unset TOptional if the parsing failed.
		 */
		static TOptional<FDirectLinkSourceDescription> TryParseDirectLinkUri(const FSourceUri& Uri);

		/**
		 * Return the scheme used for DirectLink URIs : "directlink"
		 */
		static const FString& GetDirectLinkScheme();

		/**
		 * Return the string "SourceId" used in URI queries for specifying the handle of the directlink source.
		 */
		static const FString& GetSourceIdPropertyName();
	};
}