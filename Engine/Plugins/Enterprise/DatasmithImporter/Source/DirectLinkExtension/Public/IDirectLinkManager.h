// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SourceUri.h"
#include "CoreMinimal.h"
#include "Delegates/DelegateCombinations.h"
#include "Templates/UnrealTemplate.h"

struct FAssetData;

struct FGuid;
namespace DirectLink
{
	using FSourceHandle = FGuid;
	class FEndpoint;
}

namespace UE::DatasmithImporter
{
	class FDirectLinkExternalSource;

	struct FDirectLinkExternalSourceRegisterInformation
	{
		FName Name;
		TFunction<TSharedRef<FDirectLinkExternalSource>(const FSourceUri&)> SpawnFunction;
	};

	/**
	 * Interface used to manage the creation and lifetime of FDirectLinkExternalSource objects.
	 */
	class DIRECTLINKEXTENSION_API IDirectLinkManager : public FNoncopyable
	{
	public:
		virtual ~IDirectLinkManager() = default;

		/**
		 * Get the FDirectLinkExternalSource associated to the given DirectLink::FSourceHandle, either by creating it or returning a cached value.
		 * The FDirectLinkExternalSource is in a disconnected state by default.
		 * Return nullptr if the FSourceHandle is not recognized.
		 * @param SourceHandle	The handle of the DirectLink source.
		 * @return	The FDirectLinkExternalSource associated to the DirectLink source.
		 */
		virtual TSharedPtr<FDirectLinkExternalSource> GetOrCreateExternalSource(const DirectLink::FSourceHandle& SourceHandle) = 0;

		/**
		 * Get the FDirectLinkExternalSource associated to the given Uri, either by creating it or returning a cached value.
		 * The FDirectLinkExternalSource is in a disconnected state by default.
		 * Return nullptr if the Uri is not compatible.
		 * @param Uri	The Uri describing a DirectLink source.
		 * @return	The FDirectLinkExternalSource associated to the DirectLink source.
		 */
		virtual TSharedPtr<FDirectLinkExternalSource> GetOrCreateExternalSource(const FSourceUri& Uri) = 0;

		/**
		 * Return the DirectLink Endpoint managed by the DirectLinkManager.
		 */
		virtual DirectLink::FEndpoint& GetEndpoint() = 0;

		/**
		 * Get the SourceUri referencing the DirectLink source associated to the passed SourceHandle.
		 * @param SourceHandle	The DirectLink source handle for which we want the Uri.
		 * @return	The SourceUri associated to the DirectLink source. 
		 *			If the SourceHandle didn't match a source, the returned FSourceUri is invalid.
		 */
		virtual FSourceUri GetUriFromSourceHandle(const DirectLink::FSourceHandle& SourceHandle) = 0;

		/**
		 * Get the list of all FDirectLinkExternalSource currently available.
		 */
		virtual TArray<TSharedRef<FDirectLinkExternalSource>> GetExternalSourceList() const = 0;

		/**
		 * Template function used to register a derived type of FDirectLinkExternalSource.
		 * Registered types are used to establish DirectLink streams with compatible sources.
		 */
		template<typename ExternalSourceType>
		void RegisterDirectLinkExternalSource(FName InName)
		{
			static_assert(TPointerIsConvertibleFromTo<ExternalSourceType, const FDirectLinkExternalSource>::Value, "RegisterDirectLinkExternalSource: ExternalSourceType must be a type derived from FDirectLinkExternalSource");

			FDirectLinkExternalSourceRegisterInformation RegisterInformation;
			RegisterInformation.Name = InName;
			RegisterInformation.SpawnFunction = [](const FSourceUri& InSourceUri)
				{
					return MakeShared<ExternalSourceType>(InSourceUri);
				};

			RegisterDirectLinkExternalSource(MoveTemp(RegisterInformation));
		}

		/**
		 * Used to unregister a type of FDirectLinkExternalSource.
		 */
		virtual void UnregisterDirectLinkExternalSource(FName InName) = 0;

		/**
		 * Enable or disable auto-reimport on the given UObject asset. The asset must have a DirectLink source.
		 * This effectively trigger a reimport operation on the asset every time its associated ExternalSource receives an update.
		 * @param Asset					The asset we want to 
		 * @param bEnableAutoReimport	
		 * @return	True if the operation was successful.
		 */
		virtual bool SetAssetAutoReimport(UObject* Asset, bool bEnableAutoReimport) = 0;

		virtual bool IsAssetAutoReimportEnabled(UObject* InAsset) const = 0;

	protected:
		/**
		 * Internal function used to register a derived type.
		 * Type checking is done by the RegisterDirectLinkExternalSource<T>() function
		 */
		virtual void RegisterDirectLinkExternalSource(FDirectLinkExternalSourceRegisterInformation&& ExternalSourceClass) = 0;
	};
}