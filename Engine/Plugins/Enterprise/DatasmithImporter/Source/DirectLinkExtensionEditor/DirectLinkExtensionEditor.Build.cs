// Copyright Epic Games, Inc. All Rights Reserved.
using System;

namespace UnrealBuildTool.Rules
{
	public class DirectLinkExtensionEditor : ModuleRules
	{
		public DirectLinkExtensionEditor(ReadOnlyTargetRules Target)
			: base(Target)
		{

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Engine",
					"InputCore",
				}
			);

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"ContentBrowser",
					"Core",
					"CoreUObject",
					"DirectLink",
					"DirectLinkExtension",
					"ExternalSource",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"ContentBrowserData",
					"EditorStyle",
					"EditorWidgets",
					"MainFrame",
					"Projects",
					"PropertyEditor",
					"Slate",
					"SlateCore",
					"ToolMenus",
					"UnrealEd",
				}
			);
		}
	}
}