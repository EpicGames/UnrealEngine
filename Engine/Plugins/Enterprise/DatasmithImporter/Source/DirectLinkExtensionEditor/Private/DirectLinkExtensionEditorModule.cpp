// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectLinkExtensionEditorModule.h"

#include "DirectLinkUriResolverInEditor.h"
#include "DirectLinkExtensionUI.h"
#include "SDirectLinkAvailableSource.h"

#include "ContentBrowserModule.h"
#include "Editor.h"
#include "Interfaces/IMainFrameModule.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "DirectLinkExtensionEditorModule"

namespace UE::DatasmithImporter
{
	class FDirectLinkExtensionEditorModule : public IDirectLinkExtensionEditorModule
	{
		const FName DirectLinkUriResolverInEditorName = TEXT("DirectLinkUriResolverInEditorName");

	public:
		virtual void StartupModule() override
		{
			IDirectLinkExtensionModule::Get().OverwriteUriResolver(MakeShared<FDirectLinkUriResolverInEditor>());

			DirectLinkExtensionUI = MakeUnique<FDirectLinkExtensionUI>();
		}

		virtual void ShutdownModule() override
		{
			DirectLinkExtensionUI.Reset();
		}

		virtual TSharedPtr<FDirectLinkExternalSource> DisplayDirectLinkSourcesDialog() override;

	private:
		TUniquePtr<FDirectLinkExtensionUI> DirectLinkExtensionUI;
	};

	TSharedPtr<FDirectLinkExternalSource> FDirectLinkExtensionEditorModule::DisplayDirectLinkSourcesDialog()
	{
		TSharedPtr<SWindow> ParentWindow;

		if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
		{
			IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
			ParentWindow = MainFrame.GetParentWindow();
		}

		TSharedRef<SWindow> Window = SNew(SWindow)
			.Title(LOCTEXT("DirectLinkEditorAvailableSourcesTitle", "DirectLink Available Sources"))
			.SizingRule(ESizingRule::UserSized)
			.AutoCenter(EAutoCenter::PreferredWorkArea)
			.ClientSize(FVector2D(600, 200))
			.SupportsMinimize(false);

		TSharedPtr<SDirectLinkAvailableSource> AvailableSourceWindow;
		Window->SetContent
		(
			SAssignNew(AvailableSourceWindow, SDirectLinkAvailableSource)
			.WidgetWindow(Window)
			.ProceedButtonLabel(FText(LOCTEXT("SelectLabel", "Select")))
			.ProceedButtonTooltip(FText::GetEmpty())
		);

		FSlateApplication::Get().AddModalWindow(Window, ParentWindow);

		return AvailableSourceWindow->GetShouldProceed() ? AvailableSourceWindow->GetSelectedSource() : nullptr;
	}
}

IMPLEMENT_MODULE(UE::DatasmithImporter::FDirectLinkExtensionEditorModule, DirectLinkExtensionEditor)

#undef LOCTEXT_NAMESPACE
