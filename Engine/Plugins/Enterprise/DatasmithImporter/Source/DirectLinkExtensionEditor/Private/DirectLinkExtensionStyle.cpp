// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectLinkExtensionStyle.h"

#include "Styling/SlateStyleRegistry.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/AppStyle.h"
#include "Interfaces/IPluginManager.h"

#define IMAGE_PLUGIN_BRUSH( RelativePath, ... ) FSlateImageBrush( FDirectLinkExtensionStyle::InContent( RelativePath, TEXT(".png") ), __VA_ARGS__ )
#define IMAGE_PLUGIN_BRUSH_SVG(RelativePath, ...) FSlateVectorImageBrush(FDirectLinkExtensionStyle::InContent(RelativePath, TEXT(".svg")), __VA_ARGS__)

namespace UE::DatasmithImporter
{
	TUniquePtr<FSlateStyleSet> FDirectLinkExtensionStyle::StyleSet;

	FString FDirectLinkExtensionStyle::InContent(const FString& RelativePath, const TCHAR* Extension)
	{
		static FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("DatasmithImporter"))->GetContentDir();
		return (ContentDir / RelativePath) + Extension;
	}

	FName FDirectLinkExtensionStyle::GetStyleSetName()
	{
		return FName(TEXT("DirectLinkExtensionStyle"));
	}

	void FDirectLinkExtensionStyle::Initialize()
	{
		// Only register once
		if (StyleSet.IsValid())
		{
			return;
		}

		StyleSet = MakeUnique<FSlateStyleSet>(GetStyleSetName());
		StyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
		StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

		// Const icon sizes
		const FVector2D Icon16x16(16.0f, 16.0f);

		// 16x16
		StyleSet->Set("DirectLinkExtension.NotAvailable", new IMAGE_PLUGIN_BRUSH_SVG(TEXT("Icons/DirectLinkBadgeNotAvailable"), Icon16x16));
		StyleSet->Set("DirectLinkExtension.OutOfSync", new IMAGE_PLUGIN_BRUSH_SVG(TEXT("Icons/DirectLinkBadgeOutOfSync"), Icon16x16));
		StyleSet->Set("DirectLinkExtension.UpToDate", new IMAGE_PLUGIN_BRUSH_SVG(TEXT("Icons/DirectLinkBadgeUpToDate"), Icon16x16));
		StyleSet->Set("DirectLinkExtension.AutoReimport", new IMAGE_PLUGIN_BRUSH_SVG(TEXT("Icons/DirectLinkBadgeAutoReimport"), Icon16x16));
		StyleSet->Set("DirectLinkExtension.AutoReimportGrayscale", new IMAGE_PLUGIN_BRUSH_SVG(TEXT("Icons/DirectLinkAutoReimport_16"), Icon16x16));

		FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
	};

	void FDirectLinkExtensionStyle::Shutdown()
	{
		if (StyleSet.IsValid())
		{
			FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
			StyleSet.Reset();
		}
	}

	const ISlateStyle& FDirectLinkExtensionStyle::Get()
	{
		return *StyleSet.Get();
	}
}

#undef IMAGE_PLUGIN_BRUSH
#undef IMAGE_PLUGIN_BRUSH_SVG
