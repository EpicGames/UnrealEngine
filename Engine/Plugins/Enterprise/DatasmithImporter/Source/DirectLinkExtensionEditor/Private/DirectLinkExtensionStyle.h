// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateStyle.h"

class ISlateStyle;
class FSlateStyleSet;

namespace UE::DatasmithImporter
{
	class FDirectLinkExtensionStyle
	{
	public:
		static void Initialize();

		static void Shutdown();

		static const ISlateStyle& Get();

		static FName GetStyleSetName();
	private:
		static FString InContent(const FString& RelativePath, const TCHAR* Extension);

	private:
		static TUniquePtr<FSlateStyleSet> StyleSet;
	};
}
