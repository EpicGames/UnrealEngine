// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DirectLinkExtensionStyle.h"
#include "SDirectLinkAvailableSource.h"
#include "SDirectLinkAssetThumbnailIndicator.h"

#include "ContentBrowserModule.h"
#include "CoreMinimal.h"
#include "Interfaces/IMainFrameModule.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "DirectLinkExtensionUI"

namespace UE::DatasmithImporter
{
	class FDirectLinkExtensionUI
	{
	public:
		FDirectLinkExtensionUI()
		{
			FDirectLinkExtensionStyle::Initialize();

			if (FContentBrowserModule* ContentBrowserModule = FModuleManager::Get().GetModulePtr<FContentBrowserModule>(TEXT("ContentBrowser")))
			{
				AssetViewExtraStateGeneratorHandle = ContentBrowserModule->AddAssetViewExtraStateGenerator(FAssetViewExtraStateGenerator(
					FOnGenerateAssetViewExtraStateIndicators::CreateRaw(this, &FDirectLinkExtensionUI::OnGenerateAssetViewLockStateIcons),
					FOnGenerateAssetViewExtraStateIndicators::CreateRaw(this, &FDirectLinkExtensionUI::OnGenerateAssetViewLockStateTooltip)
				));
			}
		}

		~FDirectLinkExtensionUI()
		{
			if (FContentBrowserModule* ContentBrowserModule = FModuleManager::Get().GetModulePtr<FContentBrowserModule>(TEXT("ContentBrowser")))
			{
				if (AssetViewExtraStateGeneratorHandle.IsValid())
				{
					ContentBrowserModule->RemoveAssetViewExtraStateGenerator(AssetViewExtraStateGeneratorHandle);
				}
			}

			FDirectLinkExtensionStyle::Shutdown();
		}

	private:

		TSharedRef<SWidget> OnGenerateAssetViewLockStateIcons(const FAssetData& AssetData)
		{
			return SNew(SDirectLinkAssetThumbnailIndicator)
				.AssetData(AssetData);
		}

		TSharedRef<SWidget> OnGenerateAssetViewLockStateTooltip(const FAssetData& AssetData)
		{
			return SNew(SDirectLinkAssetThumbnailTooltip)
				.AssetData(AssetData);
		}

		/* Delegate handle for asset direct link connection state indicator extensions.  */
		FDelegateHandle AssetViewExtraStateGeneratorHandle;

		TWeakPtr<SWindow> DirectLinkWindow;
	};
}

#undef LOCTEXT_NAMESPACE