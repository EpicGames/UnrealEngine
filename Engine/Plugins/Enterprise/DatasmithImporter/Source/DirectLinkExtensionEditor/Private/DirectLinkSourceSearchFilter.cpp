// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectLinkSourceSearchFilter.h"

#include "DirectLinkUriResolver.h"

#include "FrontendFilterBase.h"
#include "SourceUri.h"

#define LOCTEXT_NAMESPACE "UDirectLinkSourceSearchFilter"


/** A filter that search for assets imported via DirectLink */
class FFrontendFilter_DirectLinkSource : public FFrontendFilter
{
public:
	FFrontendFilter_DirectLinkSource(TSharedPtr<FFrontendFilterCategory> InCategory)
		: FFrontendFilter(InCategory)
	{}

	// FFrontendFilter implementation
	virtual FLinearColor GetColor() const override { return FLinearColor::Green; }
	virtual FString GetName() const override { return TEXT("DirectLinkSourceFilter"); }
	virtual FText GetDisplayName() const override { return LOCTEXT("FrontendFilter_DirectLinkSource", "DirectLink Asset"); }
	virtual FText GetToolTipText() const override { return LOCTEXT("FrontendFilter_DirectLinkSourceTooltip", "Show only assets have been imported with DirectLink."); }
	// End of FFrontendFilter implementation

	// IFilter implementation
	virtual bool PassesFilter(FAssetFilterType InItem) const override;
	// End of IFilter implementation
};

bool FFrontendFilter_DirectLinkSource::PassesFilter(FAssetFilterType InItem) const
{
	using namespace UE::DatasmithImporter;
	FContentBrowserItemDataAttributeValue SourceUriAttribute = InItem.GetItemAttribute(FSourceUri::GetAssetDataTag());
	if (SourceUriAttribute.IsValid())
	{
		FSourceUri SourceUri(SourceUriAttribute.GetValueString());
		return SourceUri.HasScheme(FDirectLinkUriResolver::GetDirectLinkScheme());
	}

	return false;
}

void UDirectLinkSourceSearchFilter::AddFrontEndFilterExtensions(TSharedPtr<FFrontendFilterCategory> DefaultCategory, TArray< TSharedRef<class FFrontendFilter> >& InOutFilterList) const
{
	InOutFilterList.Add(MakeShareable(new FFrontendFilter_DirectLinkSource(DefaultCategory)));
}

#undef LOCTEXT_NAMESPACE
