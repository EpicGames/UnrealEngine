// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DirectLinkUriResolver.h"

#include "DirectLinkExtensionEditorModule.h"
#include "DirectLinkExternalSource.h"

namespace UE::DatasmithImporter
{
	class FDirectLinkUriResolverInEditor : public FDirectLinkUriResolver
	{
	public:
		// IUriResolver interface begin
		virtual TSharedPtr<FExternalSource> BrowseExternalSource(const FSourceUri& DefaultSourceUri) const override
		{
			return IDirectLinkExtensionEditorModule::Get().DisplayDirectLinkSourcesDialog();
		}
		// IUriResolver interface end
	};
}