// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDirectLinkAssetThumbnailIndicator.h"

#include "DirectLinkExtensionModule.h"
#include "IDirectLinkManager.h"
#include "DirectLinkExtensionStyle.h"

#include "ExternalSource.h"
#include "ExternalSourceModule.h"
#include "SourceUri.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SOverlay.h"

#define LOCTEXT_NAMESPACE "DirectLinkAssetThumbnailIndicator"

template <>
struct TWidgetTypeTraits<class UE::DatasmithImporter::SDirectLinkAssetThumbnailIndicator>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

namespace UE::DatasmithImporter
{
	namespace ThumbnailIndicatorImpl
	{
		enum class EDirectLinkSourceStatus
		{
			None,
			NotAvailable,
			OutOfSync,
			UpToDate,
			AutoReimport,
		};

		FSourceUri TryGetSourceUri(const FAssetData& AssetData) 
		{
			const FAssetTagValueRef SourceUriTag = AssetData.TagsAndValues.FindTag(FSourceUri::GetAssetDataTag());
			if (SourceUriTag.IsSet())
			{
				return FSourceUri(SourceUriTag.GetValue());
			}

			return FSourceUri();
		}

		EVisibility GetVisibility(const FAssetData& AssetData)
		{
			// If the asset has a DirectLink source, make the icon visible, collapsed/hidden otherwise.
			const FSourceUri SourceUri = TryGetSourceUri(AssetData);
			if (SourceUri.IsValid() && SourceUri.HasScheme(FDirectLinkUriResolver::GetDirectLinkScheme()))
			{
				return EVisibility::Visible;
			}

			return EVisibility::Collapsed;
		}

		FMD5Hash TryGetSourceHash(const FAssetData& AssetData)
		{
			FAssetTagValueRef SourceHashTag = AssetData.TagsAndValues.FindTag(FName(TEXT("SourceHash")));
			if (SourceHashTag.IsSet())
			{
				FMD5Hash ParsedHash;
				const FString SourceHashString(SourceHashTag.GetValue());
				LexFromString(ParsedHash, *SourceHashString);

				return ParsedHash;
			}

			return FMD5Hash();
		}

		const EDirectLinkSourceStatus GetSourceStatus(const FAssetData& AssetData)
		{
			const FSourceUri SourceUri = TryGetSourceUri(AssetData);
			if (!SourceUri.IsValid() || !SourceUri.HasScheme(FDirectLinkUriResolver::GetDirectLinkScheme()))
			{
				return EDirectLinkSourceStatus::None;
			}

			// Only loaded asset can possibly be registered for auto-reimport.
			const bool bLoadAsset = false;
			if (UObject* Asset = AssetData.FastGetAsset(bLoadAsset))
			{
				if (IDirectLinkExtensionModule::Get().GetManager().IsAssetAutoReimportEnabled(Asset))
				{
					return EDirectLinkSourceStatus::AutoReimport;
				}
			}

			const TSharedPtr<FExternalSource> ExternalSource = IExternalSourceModule::GetOrCreateExternalSource(SourceUri);
			if (!ExternalSource.IsValid() || !ExternalSource->IsAvailable())
			{
				return EDirectLinkSourceStatus::NotAvailable;
			}
			else if (ExternalSource->IsOutOfSync())
			{
				// The scene loaded in ExternalSource is out of sync.
				return EDirectLinkSourceStatus::OutOfSync;
			}

			const FMD5Hash SourceHash = TryGetSourceHash(AssetData);
			const FMD5Hash ExternalSourceHash = ExternalSource->GetSourceHash();
			if (!SourceHash.IsValid() || SourceHash != ExternalSourceHash)
			{
				//The source hash in the asset is different from the one in the external source, the asset is out of sync.
				return EDirectLinkSourceStatus::OutOfSync;
			}

			return EDirectLinkSourceStatus::UpToDate;
		}
	}

	const FSlateBrush* SDirectLinkAssetThumbnailIndicator::NotAvailableBrush = nullptr;
	const FSlateBrush* SDirectLinkAssetThumbnailIndicator::OutOfSyncBrush = nullptr;
	const FSlateBrush* SDirectLinkAssetThumbnailIndicator::UpToDateBrush = nullptr;
	const FSlateBrush* SDirectLinkAssetThumbnailIndicator::AutoReimportBrush = nullptr;

	void SDirectLinkAssetThumbnailIndicator::Construct(const FArguments& InArgs)
	{
		ObjectPropertyChangedHandle = FCoreUObjectDelegates::OnObjectPropertyChanged.AddSP(this, &SDirectLinkAssetThumbnailIndicator::OnObjectPropertyChanged);

		CacheBrushes();
		AssetData = InArgs._AssetData;
		SetVisibility(MakeAttributeSP(this, &SDirectLinkAssetThumbnailIndicator::GetVisibility));

		ChildSlot
		[
			SNew(SOverlay)
			+ SOverlay::Slot() // The DirectLink image on top.
			[
				SNew(SImage)
				.Image(this, &SDirectLinkAssetThumbnailIndicator::GetImageBrush)
			]
		];
	}

	SDirectLinkAssetThumbnailIndicator::~SDirectLinkAssetThumbnailIndicator()
	{
		FCoreUObjectDelegates::OnObjectPropertyChanged.Remove(ObjectPropertyChangedHandle);
	}

	void SDirectLinkAssetThumbnailIndicator::CacheBrushes()
	{
		if (NotAvailableBrush == nullptr)
		{
			NotAvailableBrush = FDirectLinkExtensionStyle::Get().GetBrush(TEXT("DirectLinkExtension.NotAvailable"));
			OutOfSyncBrush = FDirectLinkExtensionStyle::Get().GetBrush(TEXT("DirectLinkExtension.OutOfSync"));
			UpToDateBrush = FDirectLinkExtensionStyle::Get().GetBrush(TEXT("DirectLinkExtension.UpToDate"));
			AutoReimportBrush = FDirectLinkExtensionStyle::Get().GetBrush(TEXT("DirectLinkExtension.AutoReimport"));
		}
	}

	EVisibility SDirectLinkAssetThumbnailIndicator::GetVisibility() const
	{
		return ThumbnailIndicatorImpl::GetVisibility(AssetData);
	}

	const FSlateBrush* SDirectLinkAssetThumbnailIndicator::GetImageBrush() const
	{
		switch (ThumbnailIndicatorImpl::GetSourceStatus(AssetData))
		{
		case ThumbnailIndicatorImpl::EDirectLinkSourceStatus::NotAvailable:
			return NotAvailableBrush;
		case ThumbnailIndicatorImpl::EDirectLinkSourceStatus::OutOfSync:
			return OutOfSyncBrush;
		case ThumbnailIndicatorImpl::EDirectLinkSourceStatus::UpToDate:
			return UpToDateBrush;
		case ThumbnailIndicatorImpl::EDirectLinkSourceStatus::AutoReimport:
			return AutoReimportBrush;
		case ThumbnailIndicatorImpl::EDirectLinkSourceStatus::None:
		default:
			return nullptr;
		}
	}

	void SDirectLinkAssetThumbnailIndicator::OnObjectPropertyChanged(UObject* ModifiedObject, struct FPropertyChangedEvent&)
	{
		if (ModifiedObject->IsA(UAssetImportData::StaticClass()))
		{
			ModifiedObject = ModifiedObject->GetOuter();
		}

		UObject* Asset = AssetData.FastGetAsset();
		if (Asset && Asset == ModifiedObject)
		{
			// Refresh the asset data as it might have changed.
			AssetData = FAssetData(Asset);
		}
	}

	void SDirectLinkAssetThumbnailTooltip::Construct(const FArguments& InArgs)
	{
		AssetData = InArgs._AssetData;
		SetVisibility(MakeAttributeSP(this, &SDirectLinkAssetThumbnailTooltip::GetTooltipVisibility));

		ChildSlot
		[
			SNew(STextBlock)
			.Text(this, &SDirectLinkAssetThumbnailTooltip::GetTooltipText)
		];
	}

	EVisibility SDirectLinkAssetThumbnailTooltip::GetTooltipVisibility() const
	{
		return ThumbnailIndicatorImpl::GetVisibility(AssetData);
	}

	FText SDirectLinkAssetThumbnailTooltip::GetTooltipText() const
	{
		switch (ThumbnailIndicatorImpl::GetSourceStatus(AssetData))
		{
		case ThumbnailIndicatorImpl::EDirectLinkSourceStatus::NotAvailable:
			return LOCTEXT("NotAvailableTooltip", "Direct Link source is not available.");
		case ThumbnailIndicatorImpl::EDirectLinkSourceStatus::OutOfSync:
			return LOCTEXT("OutOfSyncTooltip", "Asset is not synced to Direct Link source.");
		case ThumbnailIndicatorImpl::EDirectLinkSourceStatus::UpToDate:
			return LOCTEXT("UpToDateTooltip", "Asset is up-to-date with Direct Link source.");
		case ThumbnailIndicatorImpl::EDirectLinkSourceStatus::AutoReimport:
			return LOCTEXT("AutoReimportTooltip", "Asset will auto-reimport on Direct Link source change.");
		case ThumbnailIndicatorImpl::EDirectLinkSourceStatus::None:
		default:
			return FText();
		}
	}

}

#undef LOCTEXT_NAMESPACE
