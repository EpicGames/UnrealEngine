// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DirectLinkUriResolver.h"

#include "DirectLinkUriResolver.h"

#include "AssetRegistry/AssetData.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/SCompoundWidget.h"

struct FSlateBrush;

namespace UE::DatasmithImporter
{
	class SDirectLinkAssetThumbnailIndicator : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SDirectLinkAssetThumbnailIndicator) {}
			SLATE_ARGUMENT(FAssetData, AssetData)
		SLATE_END_ARGS();

		void Construct(const FArguments& InArgs);

		virtual ~SDirectLinkAssetThumbnailIndicator();

	private:
		EVisibility GetVisibility() const;

		const FSlateBrush* GetImageBrush() const;

		void OnObjectPropertyChanged(UObject* ModifiedObject, struct FPropertyChangedEvent&);

		static void CacheBrushes();

		// Brushes used to render DirectLink icons.
		static const FSlateBrush* NotAvailableBrush;
		static const FSlateBrush* OutOfSyncBrush;
		static const FSlateBrush* UpToDateBrush;
		static const FSlateBrush* AutoReimportBrush;

		/** Asset path for this indicator widget.*/
		FAssetData AssetData;

		FDelegateHandle ObjectPropertyChangedHandle;
	};

	class SDirectLinkAssetThumbnailTooltip : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SDirectLinkAssetThumbnailTooltip) {}
			SLATE_ARGUMENT(FAssetData, AssetData)
		SLATE_END_ARGS();

		/**
		 * Construct this widget.
		 * @param InArgs Slate arguments
		 */
		void Construct(const FArguments& InArgs);
		
	private:
		EVisibility GetTooltipVisibility() const;

		FText GetTooltipText() const;

		mutable FAssetData AssetData;
	};
}
