// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDirectLinkAvailableSource.h"
#include "DirectLinkExternalSource.h"
#include "DirectLinkUriResolver.h"
#include "DirectLinkExtensionModule.h"
#include "IDirectLinkManager.h"

#include "Styling/AppStyle.h"
#include "Styling/SlateBrush.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "DirectLinkAvailableSourceWindow"

namespace UE::DatasmithImporter
{
	namespace SDirectLinkSourceWindowUtils
	{
		const FName ComputerColumnId("ComputerName");
		const FName ProcessColumnId("ProcessName");
		const FName EndpointColumnId("EndpointName");
		const FName SourceColumnId("SourceName");
	}

	struct FDirectLinkExternalSourceInfo
	{
		FDirectLinkExternalSourceInfo(const TSharedRef<FDirectLinkExternalSource>& InExternalSource)
			: ExternalSource(InExternalSource)
		{
			if (TOptional<FDirectLinkSourceDescription> ParsedSourceDescription = FDirectLinkUriResolver::TryParseDirectLinkUri(ExternalSource->GetSourceUri()))
			{
				SourceDescription = MoveTemp(ParsedSourceDescription.GetValue());
			}
		}

		TSharedRef<FDirectLinkExternalSource> ExternalSource;
		FDirectLinkSourceDescription SourceDescription;
	};

	class SDirectLinkExternalSourceInfoRow : public SMultiColumnTableRow<TSharedRef<FDirectLinkExternalSourceInfo>>
	{
	public:
		SLATE_BEGIN_ARGS(SDirectLinkExternalSourceInfoRow)
		{}
		SLATE_END_ARGS()

			void Construct(const FArguments& InArgs, TSharedRef<FDirectLinkExternalSourceInfo> InItem, const TSharedRef<STableViewBase>& InOwner)
		{
			ExternalSourceInfo = InItem;

			FSuperRowType::FArguments Args = FSuperRowType::FArguments();
			FSuperRowType::Construct(Args, InOwner);
		}

		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
		{
			const FDirectLinkSourceDescription& SourceDescription = ExternalSourceInfo->SourceDescription;

			if (ColumnName == SDirectLinkSourceWindowUtils::ComputerColumnId)
			{
				return SNew(STextBlock)
					.Text(FText::FromString(SourceDescription.ComputerName));
			}
			else if (ColumnName == SDirectLinkSourceWindowUtils::ProcessColumnId)
			{
				return SNew(STextBlock)
					.Text(FText::FromString(SourceDescription.ExecutableName));
			}
			else if (ColumnName == SDirectLinkSourceWindowUtils::EndpointColumnId)
			{
				return SNew(STextBlock)
					.Text(FText::FromString(SourceDescription.EndpointName));

			}
			else if (ColumnName == SDirectLinkSourceWindowUtils::SourceColumnId)
			{
				return SNew(STextBlock)
					.Text(FText::FromString(SourceDescription.SourceName));
			}

			checkNoEntry();
			return SNullWidget::NullWidget;
		}

	private:
		TSharedPtr<FDirectLinkExternalSourceInfo> ExternalSourceInfo;
	};

	void SDirectLinkAvailableSource::Construct(const FArguments& InArgs)
	{
		Window = InArgs._WidgetWindow;

		GenerateDirectLinkExternalSourceInfos();

		TSharedRef<SHeaderRow> HeaderRow = SNew(SHeaderRow)
			// Computer
			+ SHeaderRow::Column(SDirectLinkSourceWindowUtils::ComputerColumnId)
			.DefaultLabel(LOCTEXT("ComputerColumnLabel", "Computer"))
			// Process
			+SHeaderRow::Column(SDirectLinkSourceWindowUtils::ProcessColumnId)
			.DefaultLabel(LOCTEXT("ProcessColumnLabel", "Process"))
			// Endpoint
			+ SHeaderRow::Column(SDirectLinkSourceWindowUtils::EndpointColumnId)
			.DefaultLabel(LOCTEXT("EndpointColumnLabel", "Endpoint"))
			// Source
			+SHeaderRow::Column(SDirectLinkSourceWindowUtils::SourceColumnId)
			.DefaultLabel(LOCTEXT("SourceColumnLabel", "Source"));

		ChildSlot
		[
			SNew(SVerticalBox)
			// No connection hint
			+ SVerticalBox::Slot()
			.Padding(2.f)
			.FillHeight(TAttribute<float>(this, &SDirectLinkAvailableSource::GetNoConnectionHintFillHeight))
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Visibility(this, &SDirectLinkAvailableSource::GetNoConnectionHintVisibility)
				.Text(LOCTEXT("ConnectionHintText", "No DirectLink source available."))
				.AutoWrapText(true)
				.Justification(ETextJustify::Center)
				.Font(FCoreStyle::GetDefaultFontStyle("Regular", 13))
			]

			// Connection view
			+ SVerticalBox::Slot()
			.Padding(2.f)
			.VAlign(VAlign_Fill)
			[
				SAssignNew(SourceListView, SListView< TSharedRef<FDirectLinkExternalSourceInfo>>)
				.Visibility(this, &SDirectLinkAvailableSource::GetConnectionViewVisibility)
				.ListItemsSource(&DirectLinkExternalSourceInfos)
				.OnGenerateRow(this, &SDirectLinkAvailableSource::OnGenerateRow)
				.OnSelectionChanged(this, &SDirectLinkAvailableSource::OnSelectionChanged)
				.SelectionMode(ESelectionMode::Single)
				.HeaderRow(HeaderRow)
			]

			+ SVerticalBox::Slot()
			.Padding(2.f)
			.AutoHeight()
			.VAlign(VAlign_Bottom)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.ContentPadding(1)
					.ToolTipText(LOCTEXT("RefreshSourceListButtonTooltip", "Refresh the list of Direct Link sources."))
					.OnClicked_Lambda([this]() { GenerateDirectLinkExternalSourceInfos(); SourceListView->RequestListRefresh(); return FReply::Handled(); })
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("Icons.Refresh"))
					]
				]

				+SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SNew(SSpacer)
				]

				+ SHorizontalBox::Slot()
				.Padding(5, 0)
				.AutoWidth()
				.HAlign(HAlign_Right)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Text(InArgs._ProceedButtonLabel)
					.ToolTipText(InArgs._ProceedButtonTooltip)
					.IsEnabled_Lambda([this]() { return SelectedSource.IsValid(); })
					.OnClicked(this, &SDirectLinkAvailableSource::OnProceed)
				]
				+ SHorizontalBox::Slot()
				.Padding(5, 0)
				.AutoWidth()
				.HAlign(HAlign_Right)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Text(FText(LOCTEXT("CancelButton", "Cancel")))
					.OnClicked(this, &SDirectLinkAvailableSource::OnCancel)
				]
			]
		];
	}


	float SDirectLinkAvailableSource::GetNoConnectionHintFillHeight() const
	{
		return DirectLinkExternalSourceInfos.Num() == 0 ? 1.f : 0.f;
	}

	::EVisibility SDirectLinkAvailableSource::GetNoConnectionHintVisibility() const
	{
		return DirectLinkExternalSourceInfos.Num() ? ::EVisibility::Hidden : ::EVisibility::Visible;
	}

	::EVisibility SDirectLinkAvailableSource::GetConnectionViewVisibility() const
	{
		return DirectLinkExternalSourceInfos.Num() ? ::EVisibility::Visible : ::EVisibility::Hidden;
	}

	void SDirectLinkAvailableSource::GenerateDirectLinkExternalSourceInfos()
	{
		DirectLinkExternalSourceInfos.Empty(DirectLinkExternalSourceInfos.Num());
		for (const TSharedRef<FDirectLinkExternalSource>& ExternalSource : IDirectLinkExtensionModule::Get().GetManager().GetExternalSourceList())
		{
			DirectLinkExternalSourceInfos.Add(MakeShared<FDirectLinkExternalSourceInfo>(ExternalSource));
		}
	}

	TSharedRef<ITableRow> SDirectLinkAvailableSource::OnGenerateRow(TSharedRef<FDirectLinkExternalSourceInfo> Item, const TSharedRef<STableViewBase>& Owner) const
	{
		return SNew(SDirectLinkExternalSourceInfoRow, Item, Owner);
	}

	void SDirectLinkAvailableSource::OnSelectionChanged(TSharedPtr<FDirectLinkExternalSourceInfo> SelectedValue, ESelectInfo::Type SelectionType)
	{
		if (SelectedValue)
		{
			SelectedSource = SelectedValue->ExternalSource;
		}
		else
		{
			SelectedSource = nullptr;
		}
	}
}

#undef LOCTEXT_NAMESPACE
