// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDirectLinkManager.h"

#include "Framework/Application/SlateApplication.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

namespace UE::DatasmithImporter
{
	class FExternalSource;

	/**
	 * Dialog used to make a selection of one available FDirectLinkExternalSource.
	 */
	class SDirectLinkAvailableSource : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(SDirectLinkAvailableSource)
		{}
			SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
			SLATE_ARGUMENT(FText, ProceedButtonLabel)
			SLATE_ARGUMENT(FText, ProceedButtonTooltip)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);

		virtual ~SDirectLinkAvailableSource() = default;

		virtual bool SupportsKeyboardFocus() const override { return true; }

		virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
		{
			if (InKeyEvent.GetKey() == EKeys::Escape)
			{
				return OnCancel();
			}

			return FReply::Unhandled();
		}

		bool GetShouldProceed() const { return bShouldProceed; }

		TSharedPtr<FDirectLinkExternalSource> GetSelectedSource() const { return SelectedSource; }

	private:

		void GenerateDirectLinkExternalSourceInfos();

		TSharedRef<ITableRow> OnGenerateRow(TSharedRef<struct FDirectLinkExternalSourceInfo> Item, const TSharedRef<STableViewBase>& Owner) const;

		void OnSelectionChanged(TSharedPtr<struct FDirectLinkExternalSourceInfo>, ESelectInfo::Type);

		FReply OnCancel()
		{
			bShouldProceed = false;
			if (Window.IsValid())
			{
				Window.Pin()->RequestDestroyWindow();
			}
			return FReply::Handled();
		}

		FReply OnProceed()
		{
			if (SelectedSource.IsValid() && Window.IsValid())
			{
				bShouldProceed = true;
				Window.Pin()->RequestDestroyWindow();
			}

			return FReply::Handled();
		}

		float GetNoConnectionHintFillHeight() const;

		::EVisibility GetNoConnectionHintVisibility() const;

		::EVisibility GetConnectionViewVisibility() const;

		bool bShouldProceed = false;

		TWeakPtr< SWindow > Window;

		TSharedPtr<FDirectLinkExternalSource> SelectedSource;

		TSharedPtr<SListView<TSharedRef<struct FDirectLinkExternalSourceInfo>>> SourceListView;

		TArray<TSharedRef<struct FDirectLinkExternalSourceInfo>> DirectLinkExternalSourceInfos;
	};
}
