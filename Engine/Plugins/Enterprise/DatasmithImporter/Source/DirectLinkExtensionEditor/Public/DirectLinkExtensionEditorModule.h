// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DirectLinkExtensionModule.h"

namespace UE::DatasmithImporter
{
	class FDirectLinkExternalSource;
}

#define DIRECTLINKEXTENSIONEDITOR_MODULE_NAME TEXT("DirectLinkExtensionEditor")

class IDirectLinkExtensionEditorModule : public IDirectLinkExtensionModule
{
public:
	/**
	 * Singleton-like access to IDirectLinkExtensionEditorModule
	 *
	 * @return Returns IDirectLinkExtensionEditorModule singleton instance, loading the module on demand if needed
	 */
	static inline IDirectLinkExtensionEditorModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IDirectLinkExtensionEditorModule>(DIRECTLINKEXTENSIONEDITOR_MODULE_NAME);
	}

	/**
	 * Checks to see if this module is loaded and ready.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(DIRECTLINKEXTENSIONEDITOR_MODULE_NAME);
	}

	virtual void OverwriteUriResolver(const TSharedRef<UE::DatasmithImporter::IUriResolver>& UriResolver)
	{
		IDirectLinkExtensionModule::Get().OverwriteUriResolver(UriResolver);
	}

	virtual UE::DatasmithImporter::IDirectLinkManager& GetManager() const override
	{
		return IDirectLinkExtensionModule::Get().GetManager();
	}

	/**
	 * Spawn a dialog window prompting the user to select one available FDirectLinkExternalSource.
	 * The list of displayed DirectLink source can be filtered.
	 * @return The selected DirectLinkExternalSource, nullptr is returned if the dialog was canceled. 
	 */
	virtual TSharedPtr<UE::DatasmithImporter::FDirectLinkExternalSource> DisplayDirectLinkSourcesDialog() = 0;
};

