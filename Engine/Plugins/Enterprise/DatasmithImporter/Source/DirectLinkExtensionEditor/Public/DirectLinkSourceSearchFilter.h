// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ContentBrowserFrontEndFilterExtension.h"

#include "DirectLinkSourceSearchFilter.generated.h"

class FFrontendFilter;
class FFrontendFilterCategory;

/**
 * Content Browser filter used to isolate assets imported via a Direct Link source.
 */
UCLASS()
class UDirectLinkSourceSearchFilter : public UContentBrowserFrontEndFilterExtension
{
public:
	GENERATED_BODY()

	// UContentBrowserFrontEndFilterExtension interface
	virtual void AddFrontEndFilterExtensions(TSharedPtr<FFrontendFilterCategory> DefaultCategory, TArray< TSharedRef<FFrontendFilter> >& InOutFilterList) const override;
	// End of UContentBrowserFrontEndFilterExtension interface
};
