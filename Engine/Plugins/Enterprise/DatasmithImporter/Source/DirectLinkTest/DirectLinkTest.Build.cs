// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class DirectLinkTest : ModuleRules
	{
		public DirectLinkTest(ReadOnlyTargetRules Target) : base(Target)
		{
			// #ue_directlink_cleanup move to QAEnterprise
			PublicDependencyModuleNames.AddRange(new string[]{
				"Core",
				"DatasmithCore",
				"DatasmithTranslator",
				"DirectLink",
			});

			PrivateDependencyModuleNames.AddRange(new string[]{
				"CoreUObject",
				"Engine",
			});
		}
	}
}
