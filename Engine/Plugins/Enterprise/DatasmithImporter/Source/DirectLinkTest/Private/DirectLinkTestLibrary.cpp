// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectLinkTestLibrary.h"

#include "DirectLinkTestLog.h"
#include "TestSceneProvider.h"

#include "DatasmithDefinitions.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithSceneXmlWriter.h"
#include "DatasmithTranslatableSource.h"
#include "DirectLink/DatasmithSceneReceiver.h"

#include "DirectLinkCommon.h"
#include "DirectLinkEndpoint.h"
#include "DirectLinkParameterStore.h"
#include "DirectLinkSceneSnapshot.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTime.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"


#define LOCTEXT_NAMESPACE "DatasmithBlueprintTestLibrary"


void DumpSceneXML(TSharedPtr<IDatasmithScene> SourceScene, FString FilePath, FString Suffix)
{
	Suffix = TEXT(".") + Suffix + TEXT(".udatasmith");
	FilePath.ReplaceInline(TEXT(".udatasmith"), *Suffix);
	TUniquePtr<FArchive> DumpFile(IFileManager::Get().CreateFileWriter(*FilePath));
	if (DumpFile && SourceScene.IsValid())
	{
		FDatasmithSceneXmlWriter DatasmithSceneXmlWriter;
		DatasmithSceneXmlWriter.Serialize(SourceScene.ToSharedRef(), *DumpFile);
	}
}

TSharedPtr<IDatasmithScene> TranslateFile(const FString& InFilePath, TSharedPtr<IDatasmithScene> InSourceScene=nullptr)
{
	FDatasmithSceneSource Source;
	Source.SetSourceFile(InFilePath);
	FDatasmithTranslatableSceneSource TranslatableSource(Source);

	if (!TranslatableSource.IsTranslatable())
	{
		UE_LOGF(LogDirectLinkTest, Error, "Datasmith adapter import error: no suitable translator found for this source. Abort import.");
		return nullptr;
	}

	TSharedRef<IDatasmithScene> SourceScene = InSourceScene.IsValid() ? InSourceScene.ToSharedRef() : FDatasmithSceneFactory::CreateScene(*Source.GetSceneName());

	if (!TranslatableSource.Translate(SourceScene))
	{
		UE_LOGF(LogDirectLinkTest, Error, "Datasmith import error: Scene translation failure. Abort import.");
		return nullptr;
	}

	return SourceScene;
}


bool UDirectLinkTestLibrary::TestParameters()
{
	bool ok = true;
	DirectLink::FParameterStore Store;
	DirectLink::TStoreKey<FString> Text;
	Store.RegisterParameter(Text, "text");

	FString In = TEXT("test string value");
	FString Out;
	ok &= ensure(Out == In);

	Text = In;
	Out = Text;
	ok &= ensure(Out == In);

	return ok;
}


////////////////////////////////////////////////////////////////////////////////////////////


namespace DirectLinkTestLibrary
{

struct FReceiverState
{
	TUniquePtr<DirectLink::FEndpoint> Endpoint;
	DirectLink::FDestinationHandle Destination;
	TSharedPtr<FTestSceneProvider> Provider;
	FString DumpXMLFilePath;
};

static FReceiverState ReceiverState;

struct FSenderState
{
	TUniquePtr<DirectLink::FEndpoint> SenderEndpoint;
	DirectLink::FSourceHandle Source;
};
static FSenderState SenderState;

}
using namespace DirectLinkTestLibrary;



bool UDirectLinkTestLibrary::StartReceiver()
{
	StopReceiver();

	ReceiverState.Endpoint = MakeUnique<DirectLink::FEndpoint>(TEXT("UDirectLinkTestLibrary-Receiver"));
	ReceiverState.Endpoint->SetVerbose();
	ReceiverState.Provider = MakeShared<FTestSceneProvider>();
	ReceiverState.Destination = ReceiverState.Endpoint->AddDestination(
		TEXT("stream-A"),
		DirectLink::EVisibility::Public,
		ReceiverState.Provider
	);

	return ReceiverState.Endpoint.IsValid();
}

bool UDirectLinkTestLibrary::StopReceiver()
{
	ReceiverState = FReceiverState();
	return true;
}

bool UDirectLinkTestLibrary::SetupReceiver()
{
	// startup
	if (!ReceiverState.Endpoint.IsValid())
	{
		StartReceiver();
	}

	return true;
}



bool UDirectLinkTestLibrary::StartSender()
{
	StopSender();

	SenderState.SenderEndpoint = MakeUnique<DirectLink::FEndpoint>(TEXT("UDirectLinkTestLibrary-Sender"));
	SenderState.Source = SenderState.SenderEndpoint->AddSource(TEXT("stream-A"), DirectLink::EVisibility::Public);
	SenderState.SenderEndpoint->SetVerbose();
	return true;
}

bool UDirectLinkTestLibrary::StopSender()
{
	SenderState = FSenderState();
	return true;
}

bool UDirectLinkTestLibrary::SetupSender()
{
	if (!SenderState.SenderEndpoint.IsValid())
	{
		StartSender();
	}

	return true;
}

bool UDirectLinkTestLibrary::SendScene(const FString& InFilePath)
{
	if (!SenderState.SenderEndpoint.IsValid())
	{
		if (!StartSender())
		{
			return false;
		}
	}

	UE_LOGF(LogDirectLinkTest, Display, "translate scene %ls...", *InFilePath);

	// load a scene
	TSharedPtr<IDatasmithScene> SourceScene = TranslateFile(InFilePath);
	DumpSceneXML(SourceScene, InFilePath, TEXT(".translated.udatasmith"));
	if (!SourceScene.IsValid())
	{
		UE_LOGF(LogDirectLinkTest, Warning, "invalid scene");
		return false;
	}

	SenderState.SenderEndpoint->SetSourceRoot(SenderState.Source, SourceScene.Get(), true);
	ReceiverState.DumpXMLFilePath = InFilePath;

	return true;
}

bool UDirectLinkTestLibrary::DumpReceivedScene()
{
	if (ReceiverState.Provider)
	{
		for (auto& KV : ReceiverState.Provider->SceneReceivers)
		{
			if (TSharedPtr<FDatasmithSceneReceiver>& DatasmithSceneReceiver = KV.Value)
			{
				if (TSharedPtr<IDatasmithScene> Scene = DatasmithSceneReceiver->GetScene())
				{
					DumpSceneXML(Scene, ReceiverState.DumpXMLFilePath, TEXT("Received"));
				}
			}
		}
	}
	return true;
}

static TArray<TUniquePtr<DirectLink::FEndpoint>> GEndpoints;


int UDirectLinkTestLibrary::MakeEndpoint(const FString& NiceName, bool bVerbose)
{
	int32 Index = GEndpoints.Add(MakeUnique<DirectLink::FEndpoint>(NiceName));
	GEndpoints.Last()->SetVerbose(bVerbose);
	return Index;
}

bool UDirectLinkTestLibrary::DeleteEndpoint(int32 EndpointId)
{
	if (GEndpoints.IsValidIndex(EndpointId) && GEndpoints[EndpointId].IsValid())
	{
		GEndpoints[EndpointId].Reset();
		return true;
	}

	return false;
}

bool UDirectLinkTestLibrary::AddPublicSource(int32 EndpointId, FString SourceName)
{
	if (GEndpoints.IsValidIndex(EndpointId) && GEndpoints[EndpointId].IsValid())
	{
		GEndpoints[EndpointId]->AddSource(SourceName, DirectLink::EVisibility::Public);
		return true;
	}

	return false;
}

bool UDirectLinkTestLibrary::AddPublicDestination(int32 EndpointId, FString DestName)
{
	if (GEndpoints.IsValidIndex(EndpointId) && GEndpoints[EndpointId].IsValid())
	{
		GEndpoints[EndpointId]->AddDestination(DestName, DirectLink::EVisibility::Public, MakeShared<FTestSceneProvider>());
		return true;
	}

	return false;
}

bool UDirectLinkTestLibrary::DeleteAllEndpoint()
{
	GEndpoints.Reset();
	return true;
}

#undef LOCTEXT_NAMESPACE
