// Copyright Epic Games, Inc. All Rights Reserved.
#include "DirectLinkTestModule.h"

#include "DirectLinkTestLog.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FDirectLinkTestModule, DirectLinkTest);

DEFINE_LOG_CATEGORY(LogDirectLinkTest)
