// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestSceneProvider.h"
#include "DirectLink/DatasmithSceneReceiver.h"


bool FTestSceneProvider::CanOpenNewConnection(const FSourceInformation& Source)
{
	static bool tmp = true;
	return tmp;
}


TSharedPtr<DirectLink::ISceneReceiver> FTestSceneProvider::GetSceneReceiver(const FSourceInformation& Source)
{
	if (const auto* ElementPtr = SceneReceivers.Find(Source.Id))
	{
		return *ElementPtr;
	}

	return SceneReceivers.Add(Source.Id, MakeShared<FDatasmithSceneReceiver>());
}

