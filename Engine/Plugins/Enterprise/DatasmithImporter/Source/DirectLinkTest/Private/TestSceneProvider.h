// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DirectLinkCommon.h"
#include "DirectLinkConnectionRequestHandler.h"


class FDatasmithSceneReceiver;

class FTestSceneProvider : public DirectLink::IConnectionRequestHandler
{
public:
	virtual bool CanOpenNewConnection(const FSourceInformation& Source) override;
	virtual TSharedPtr<DirectLink::ISceneReceiver> GetSceneReceiver(const FSourceInformation& Source) override;

	TMap<FGuid, TSharedPtr<FDatasmithSceneReceiver>> SceneReceivers;
};
