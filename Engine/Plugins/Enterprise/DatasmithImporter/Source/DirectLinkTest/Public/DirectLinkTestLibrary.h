// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DirectLinkEndpoint.h"

#include "Kismet/BlueprintFunctionLibrary.h"

#include "DirectLinkTestLibrary.generated.h"


UCLASS()
class UDirectLinkTestLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "DirectLink")
	static bool TestParameters();


	UFUNCTION(BlueprintCallable, Category = "DirectLink")
	static bool StartReceiver();

	UFUNCTION(BlueprintCallable, Category = "DirectLink")
	static bool SetupReceiver();

	UFUNCTION(BlueprintCallable, Category = "DirectLink")
	static bool StopReceiver();


	UFUNCTION(BlueprintCallable, Category = "DirectLink")
	static bool StartSender();

	UFUNCTION(BlueprintCallable, Category = "DirectLink")
	static bool SetupSender();

	UFUNCTION(BlueprintCallable, Category = "DirectLink")
	static bool StopSender();


	UFUNCTION(BlueprintCallable, Category = "DirectLink")
	static bool SendScene(const FString& InFilePath);

	UFUNCTION(BlueprintCallable, Category = "DirectLink")
	static bool DumpReceivedScene();


////////////////////////////////////////////////////////////////////
// Endpoint section
public:
	UFUNCTION(BlueprintCallable, Category = "DirectLink")
	static int MakeEndpoint(const FString& NiceName, bool bVerbose=true);

	UFUNCTION(BlueprintCallable, Category = "DirectLink")
	static bool DeleteEndpoint(int32 EndpointId);

	UFUNCTION(BlueprintCallable, Category = "DirectLink")
	static bool AddPublicSource(int32 EndpointId, FString SourceName);

	UFUNCTION(BlueprintCallable, Category = "DirectLink")
	static bool AddPublicDestination(int32 EndpointId, FString DestName);

	UFUNCTION(BlueprintCallable, Category = "DirectLink")
	static bool DeleteAllEndpoint();
////////////////////////////////////////////////////////////////////
};


