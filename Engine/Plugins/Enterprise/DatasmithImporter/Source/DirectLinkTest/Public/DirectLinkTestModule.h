// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#define DIRECTLINKTEST_MODULE_NAME TEXT("DirectLinkTest")

/**
 * TODO: Module description
 */
class FDirectLinkTestModule : public IModuleInterface
{
public:
	static inline FDirectLinkTestModule& Get()
	{
		return FModuleManager::LoadModuleChecked<FDirectLinkTestModule>(DIRECTLINKTEST_MODULE_NAME);
	}

	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(DIRECTLINKTEST_MODULE_NAME);
	}
};
