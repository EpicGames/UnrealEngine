// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ExternalSource : ModuleRules
	{
		public ExternalSource(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"DatasmithTranslator",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"DatasmithCore",
				}
			);

			if (Target.Type == TargetType.Editor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"Engine",
						"UnrealEd",
					}
				);
			}
		}
	}
}
