// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExternalSource.h"

#include "Async/Async.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithSceneSource.h"
#include "DatasmithTranslatableSource.h"
#include "DatasmithTranslator.h"
#include "DatasmithTranslatorManager.h"
#include "DatasmithUtils.h"
#include "Misc/AsyncTaskNotification.h"

#if WITH_EDITOR
#include "Editor.h"
#endif //WITH_EDITOR

#define LOCTEXT_NAMESPACE "ExternalSource"

DEFINE_LOG_CATEGORY(LogExternalSource);


namespace UE::DatasmithImporter
{
	FExternalSource::FExternalSource(const FSourceUri& InSourceUri)
		: SourceUri(InSourceUri)
	{}

	FExternalSource::~FExternalSource()
	{}

	const TSharedPtr<IDatasmithTranslator>& FExternalSource::GetAssetTranslator()
	{
		if (!AssetTranslator)
		{
			FDatasmithSceneSource SceneSource;
			SceneSource.SetSourceFile(GetFallbackFilepath());
			SceneSource.SetSceneName(GetSourceName());
			
			AssetTranslator = FDatasmithTranslatorManager::Get().SelectFirstCompatible(SceneSource);
			if (AssetTranslator)
			{
				AssetTranslator->SetSource(SceneSource);
			}
		}

		return AssetTranslator;
	}

	FString FExternalSource::GetSceneName() const
	{
		if (SceneName.IsEmpty())
		{
			return GetSourceName();
		}
		
		return SceneName;
	}

	void FExternalSource::SetSceneName(const TCHAR* InSceneName)
	{
		if (InSceneName)
		{
			SceneName = InSceneName;
		}
		else
		{
			SceneName.Reset();
		}
	}


	TSharedPtr<IDatasmithScene> FExternalSource::TryLoad()
	{
		if (!IsAvailable())
		{
			return nullptr;
		}
		else if (!IsOutOfSync())
		{
			// External source already has an up-to-date scene, no need to load again.
			return GetDatasmithScene();
		}

		const FExternalSourceCapabilities Capabilities = GetCapabilities();
		if (Capabilities.bSupportSynchronousLoading)
		{
			return Load();
		}

		if (Capabilities.bSupportAsynchronousLoading)
		{		
			return AsyncLoad().Get();
		}

		return nullptr;
	}

	TSharedPtr<IDatasmithScene> FExternalSource::Load()
	{
		if (!GetCapabilities().bSupportSynchronousLoading
			|| !IsAvailable())
		{
			return nullptr;
		}
		else if (!IsOutOfSync())
		{
			// External source already has an up-to-date scene, no need to load again.
			return GetDatasmithScene();
		}

		TSharedPtr<IDatasmithScene> LoadedScene = LoadImpl();
		if (LoadedScene)
		{
			TriggerOnExternalSourceChanged();
		}

		return LoadedScene;
	}

	TFuture<TSharedPtr<IDatasmithScene>> FExternalSource::AsyncLoad()
	{
		TPromise<TSharedPtr<IDatasmithScene>> ScenePromise;
		TFuture<TSharedPtr<IDatasmithScene>> SceneFuture = ScenePromise.GetFuture();
		
		if (!GetCapabilities().bSupportAsynchronousLoading 
			|| !IsAvailable())
		{
			ScenePromise.EmplaceValue(nullptr);
			return SceneFuture;
		}
		else if (!IsOutOfSync())
		{
			// External source already has an up-to-date scene, no need to load again.
			ScenePromise.EmplaceValue(GetDatasmithScene());
			return SceneFuture;
		}

		PendingPromiseQueue.Enqueue(MoveTemp(ScenePromise));
		
		if (!AsyncTaskNotification.IsValid())
		{
			// If we are not loading already, start the async loading.
			// The async task notification will be set to complete once TriggerOnExternalSourceLoaded() is called.
			FAsyncTaskNotificationConfig AsyncTaskConfig{ FText::Format(LOCTEXT("LoadingExternalSourceSlowTask", "Loading {0}..."), FText::FromString(GetSourceName())) };
#if WITH_EDITOR
			if (!GIsEditor || GIsPlayInEditorWorld)
			{
				AsyncTaskConfig.bIsHeadless = true;
			}
#endif //WITH_EDITOR

			AsyncTaskNotification = MakeShared<FAsyncTaskNotification>(AsyncTaskConfig);

			if (!StartAsyncLoad())
			{
				UE_LOGF(LogExternalSource, Error, "The Datasmith scene from source \"%ls\" could not be loaded.", *GetSourceName());

				// Mark task as failed.
				// We must set AsyncTaskNotification before calling StartAsyncLoadImpl() to avoid getting in a state
				// where TriggerOnExternalSourceChanged() is called before AsyncTaskNotification is set, and thus never clearing the task notification.
				AsyncTaskNotification->SetComplete(false);
				AsyncTaskNotification.Reset();

				if (FOptionalScenePromise* LastPromise = PendingPromiseQueue.Peek())
				{
					(*LastPromise)->SetValue(nullptr);
					PendingPromiseQueue.Pop();
				}
			}
		}

		// We are already in the process of async load.
		return SceneFuture;
	}

	void FExternalSource::ClearOnExternalSourceLoadedDelegates()
	{
		OnExternalSourceChanged.Clear();

		CancelAsyncLoad();
	}

	void FExternalSource::CancelAsyncLoad()
	{
		// Set value of all pending TPromise to null, to avoid creating deadlocks.
		{
			TSharedPtr<IDatasmithScene> NullScene;
			FOptionalScenePromise CurrentPromise;

			while (PendingPromiseQueue.Dequeue(CurrentPromise))
			{
				CurrentPromise->SetValue(NullScene);
			}
		}

		// Cancel the current loading notification
		if (AsyncTaskNotification.IsValid())
		{
			AsyncTaskNotification->SetComplete(false);
			AsyncTaskNotification.Reset();
		}
	}

	void FExternalSource::TriggerOnExternalSourceChanged()
	{
		if (AsyncTaskNotification.IsValid())
		{
			AsyncTaskNotification->SetComplete(true);
			AsyncTaskNotification.Reset();
		}

		TSharedPtr<IDatasmithScene> Scene = GetDatasmithScene();

		if (Scene.IsValid())
		{
			Scene->SetName(*GetSceneName());
			ValidateDatasmithVersion();
			OnExternalSourceChanged.Broadcast(AsShared());
		}

		FOptionalScenePromise CurrentPromise;
		while (PendingPromiseQueue.Dequeue(CurrentPromise))
		{
			CurrentPromise->SetValue(Scene);
		}
	}

	bool FExternalSource::TranslatorLoadScene(const TSharedRef<IDatasmithScene>& Scene)
	{
		bool bLoaded = false;
		//Unload any previously loaded scene, and load a new one.
		SceneGuard = MakeUnique<FDatasmithSceneGuard>(GetAssetTranslator(), Scene, bLoaded);
		return bLoaded;
	}

	void FExternalSource::ValidateDatasmithVersion() const
	{
		if (TSharedPtr<IDatasmithScene> Scene = GetDatasmithScene())
		{
			// Warn on potentially incompatible versions.
			float FileVersion;
			LexFromString(FileVersion, Scene->GetExporterVersion());
			FString ExporterSDKVersion = Scene->GetExporterSDKVersion();

			const bool bDisplayWarning = FileVersion > 0.f && FileVersion != FDatasmithUtils::GetDatasmithFormatVersionAsFloat();
			const bool bIsNewer = FileVersion > 0.f && FileVersion > FDatasmithUtils::GetDatasmithFormatVersionAsFloat();

			if (bDisplayWarning)
			{
				const FString EngineVersion = FDatasmithUtils::GetEnterpriseVersionAsString();
				const FString UpdateText = bIsNewer
					? FString::Printf(TEXT("For best results, install Unreal Engine version %s or later."), *EngineVersion)
					: TEXT("For best results, update your Datasmith exporter plugin if possible.");

				UE_LOGF(LogExternalSource, Warning, "The imported Datasmith scene was created with a version of a Datasmith exporter (%ls) that doesn't match your Unreal Engine version (%ls). %ls",
					*ExporterSDKVersion,
					*EngineVersion,
					*UpdateText);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE