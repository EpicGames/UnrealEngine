// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExternalSourceModule.h"

#include "UriManager.h"

class FExternalSourceModule : public IExternalSourceModule
{
public:
	virtual void StartupModule() override
	{
		UriManager = MakeUnique<UE::DatasmithImporter::FUriManager>();
	}

	virtual void ShutdownModule() override
	{
		UriManager.Reset();
	}

	virtual UE::DatasmithImporter::IUriManager& GetManager() const override
	{
		check(UriManager.IsValid());
		return *UriManager;
	}

	TUniquePtr<UE::DatasmithImporter::FUriManager> UriManager;
};

IMPLEMENT_MODULE(FExternalSourceModule, ExternalSource);