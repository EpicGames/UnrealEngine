// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceUri.h"

#include "AssetRegistry/AssetData.h"
#include "Misc/AutomationTest.h"
#include "Misc/Paths.h"

namespace UE::DatasmithImporter
{
	bool FSourceUri::IsValid() const
	{
		const int32 SchemeDelimiterIndex = Uri.Find(GetSchemeDelimiter());
		const int32 UriPathIndex = SchemeDelimiterIndex + GetSchemeDelimiter().Len();

		// Make sure the URI has at least a scheme and a path.
		return SchemeDelimiterIndex > 0 && (Uri.Len() - UriPathIndex) > 0;
	}

	FStringView FSourceUri::GetScheme() const
	{
		const int32 SchemeSeparatorLocation = Uri.Find(GetSchemeDelimiter());
		FStringView Scheme;
		
		if (SchemeSeparatorLocation != INDEX_NONE)
		{
			Scheme = FStringView(Uri).LeftChop(Uri.Len() - SchemeSeparatorLocation);
		}

		return Scheme;
	}

	const FString& FSourceUri::GetSchemeDelimiter()
	{
		// Todo - The actual delimiter according to RFC 3986 standard, should simply be ":" when there is no authority defined in the URI.
		static FString SchemeDelimiter(TEXT("://"));
		return SchemeDelimiter;
	}

	bool FSourceUri::HasScheme(const FString& InScheme) const
	{
		return Uri.StartsWith(InScheme + GetSchemeDelimiter());
	}

	FStringView FSourceUri::GetPath() const
	{
		const int32 SchemeLen = GetScheme().Len() + GetSchemeDelimiter().Len();

		if (Uri.Len() > SchemeLen)
		{
			return FStringView(Uri)
				.RightChop(SchemeLen) //Remove the scheme
				.LeftChop(GetQuery().Len()); //Remove the Query
		}
		
		return FStringView();
	}

	FStringView FSourceUri::GetQuery() const
	{
		int32 QueryStartIndex;
		if (Uri.FindChar(GetQueryStartCharacter(), QueryStartIndex))
		{
			return FStringView(Uri).RightChop(QueryStartIndex);
		}

		return FStringView();
	}

	TMap<FString, FString> FSourceUri::GetQueryMap() const
	{
		FStringView Query = GetQuery();
		TMap<FString, FString> Result;

		if (Query.Len() <= 0)
		{
			return Result;
		}

		//Remove the '?' at the start of the query string.
		Query.RightChopInline(1);

		while (Query.Len())
		{
			int32 DelimiterIndex;
			if (!Query.FindChar(GetQueryAppendPairCharacter(), DelimiterIndex))
			{
				DelimiterIndex = Query.Len();
			}

			int32 EqualIndex;
			if (Query.FindChar(GetQueryPairCharacter(), EqualIndex) && EqualIndex < DelimiterIndex)
			{
				TPair<FString, FString> CurrentQuery;
				CurrentQuery.Key = FString(Query.Left(EqualIndex));
				CurrentQuery.Value = FString(Query.Mid(EqualIndex + 1 , DelimiterIndex - (EqualIndex + 1)));

				Result.Add(MoveTemp(CurrentQuery));

				Query.RightChopInline(DelimiterIndex);
			}
		}

		return Result;
	}

	const FString& FSourceUri::GetFileScheme()
	{
		static FString Scheme(TEXT("file"));
		return Scheme;
	}

	FSourceUri FSourceUri::FromFilePath(const FString& FilePath)
	{
		// Make sure all paths are converted to absolute and normalized, otherwise URI won't be comparable.
		const FString AbsoluteFilePath = FPaths::ConvertRelativePathToFull(FilePath);

		return FSourceUri(GetFileScheme(), AbsoluteFilePath);
	}

	FSourceUri FSourceUri::FromAssetData(const FAssetData& AssetData)
	{
		const FAssetTagValueRef ValueRef = AssetData.TagsAndValues.FindTag(GetAssetDataTag());

		if (ValueRef.IsSet())
		{
			return FSourceUri(ValueRef.GetValue());
		}

		return FSourceUri();
	}

	FName FSourceUri::GetAssetDataTag()
	{
		static const FName SourceUriTag(TEXT("SourceUri"));
		return SourceUriTag;
	}

	/**
	 * Automated test to validate FSourceUri parsing.
	 */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSourceUriParsingTests, "Editor.Import.Datasmith.ExternalSource.Source URI Parsing", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FSourceUriParsingTests::RunTest(const FString& Parameters)
	{
		const FString Scheme = TEXT("FooScheme");
		const FString Path = TEXT("BarPath");

		{
			const FSourceUri InvalidUri = FSourceUri(Scheme);
			if (InvalidUri.IsValid())
			{
				AddError(TEXT("FSourceUri with only a scheme are not valid."));
				return false;
			}

			const FSourceUri ValidUri = FSourceUri(Scheme, Path);
			if (!ValidUri.IsValid())
			{
				AddError(TEXT("FSourceUri with a scheme and a path should be valid."));
				return false;
			}
			else if (ValidUri.GetScheme() != Scheme)
			{
				AddError(TEXT("GetScheme() does not return the same value passed during construction."));
				return false;
			}
			else if (ValidUri.GetPath() != Path)
			{
				AddError(TEXT("GetPath() does not return the same value passed during construction."));
				return false;
			}
		}

		{
			const TArray<TPair<FString, FString>> EmptyArray;
			const FSourceUri NoQueryUri = FSourceUri(Scheme, Path, EmptyArray);
			if (!NoQueryUri.GetQuery().IsEmpty() || !NoQueryUri.GetQueryMap().IsEmpty())
			{
				AddError(TEXT("Uri should not have any query."));
				return false;
			}

			const TPair<FString, FString> DummyPair(TEXT("FooProperty"), TEXT("BarValue"));
			const TArray<TPair<FString, FString>> ParameterValueArray { DummyPair };
			const TSet<TPair<FString, FString>> ParameterValueSet { DummyPair };
			const TMap<FString, FString> ParameterValueMap { DummyPair };

			const FSourceUri ArrayUri = FSourceUri(Scheme, Path, ParameterValueArray);
			const FSourceUri SetUri = FSourceUri(Scheme, Path, ParameterValueSet);
			const FSourceUri MapUri = FSourceUri(Scheme, Path, ParameterValueMap);
			if (ArrayUri != SetUri || SetUri != MapUri)
			{
				AddError(TEXT("Query constructor does not give same result depending on the collection used."));
				return false;
			}

			const TMap<FString, FString> QueryMap = ArrayUri.GetQueryMap();
			if (const FString* Value = QueryMap.Find(DummyPair.Key))
			{
				if (**Value != DummyPair.Value)
				{
					AddError(TEXT("GetQueryMap() could not produce valid query map."));
					return false;
				}
			}
			else
			{
				AddError(TEXT("GetQueryMap() could not produce valid query map."));
				return false;
			}
		}

		return true;
	}
}