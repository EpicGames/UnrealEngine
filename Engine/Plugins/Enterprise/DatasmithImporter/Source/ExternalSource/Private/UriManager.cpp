// Copyright Epic Games, Inc. All Rights Reserved.

#include "UriManager.h"

#include "IUriResolver.h"

namespace UE::DatasmithImporter
{
	TSharedPtr<FExternalSource> FUriManager::GetOrCreateExternalSource(const FSourceUri& Uri) const
	{
		if (const TSharedPtr<IUriResolver> Resolver = GetFirstCompatibleResolver(Uri))
		{
			return Resolver->GetOrCreateExternalSource(Uri);
		}

		return nullptr;
	}

	bool FUriManager::CanResolveUri(const FSourceUri& Uri) const
	{
		return GetFirstCompatibleResolver(Uri) != nullptr;
	}

	void FUriManager::RegisterResolver(FName InResolverName, const TSharedRef<IUriResolver>& InUrinResolver)
	{
		RegisteredResolvers.Emplace(InResolverName, InUrinResolver);
		InvalidateCache();
	}

	bool FUriManager::UnregisterResolver(FName InResolverName)
	{
		const int32 NumberOfEntryRemoved = RegisteredResolvers.RemoveAll(
			[&InResolverName](const FUriResolverRegisterInformation& RegisterInformation)
			{
				return RegisterInformation.Name == InResolverName;
			}
		);
		InvalidateCache();

		return NumberOfEntryRemoved > 0;
	}

	const TArray<FName>& FUriManager::GetSupportedSchemes() const
	{
		if (CachedSchemes.Num() == 0 && RegisteredResolvers.Num() > 0)
		{
			CachedSchemes.Reserve(RegisteredResolvers.Num());

			for (const FUriResolverRegisterInformation& RegisterInfo : RegisteredResolvers)
			{
				CachedSchemes.Add(RegisterInfo.UriResolver->GetScheme());
			}
		}

		return CachedSchemes;
	}

	TSharedPtr<IUriResolver> FUriManager::GetFirstCompatibleResolver(const FSourceUri& Uri) const
	{
		for (const FUriResolverRegisterInformation& RegisterInfo : RegisteredResolvers)
		{
			if (RegisterInfo.UriResolver->CanResolveUri(Uri))
			{
				return RegisterInfo.UriResolver;
			}
		}

		return nullptr;
	}

	void FUriManager::InvalidateCache()
	{
		CachedSchemes.Reset();
	}

#if WITH_EDITOR
	TSharedPtr<FExternalSource> FUriManager::BrowseExternalSource(const FName& UriScheme, const FSourceUri& DefaultSourceUri) const
	{
		for (const FUriResolverRegisterInformation& RegisterInfo : RegisteredResolvers)
		{
			if (RegisterInfo.UriResolver->GetScheme() == UriScheme)
			{
				return RegisterInfo.UriResolver->BrowseExternalSource(DefaultSourceUri);
			}
		}

		return nullptr;
	}
#endif //WITH_EDITOR
}