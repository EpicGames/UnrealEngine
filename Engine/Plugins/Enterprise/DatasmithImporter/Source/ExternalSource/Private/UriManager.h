// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IUriManager.h"
#include "Containers/Array.h"

namespace UE::DatasmithImporter
{
	class IUriResolver;

	struct FUriResolverRegisterInformation
	{
		FUriResolverRegisterInformation(FName InName, const TSharedRef<IUriResolver>& InUriResolver)
			: Name(InName)
			, UriResolver(InUriResolver)
		{}

		FName Name;
		TSharedRef<IUriResolver> UriResolver;
	};

	class FUriManager : public IUriManager
	{
	public:
		// IUriManager interface begin
		virtual TSharedPtr<FExternalSource> GetOrCreateExternalSource(const FSourceUri& URI) const override;
		virtual bool CanResolveUri(const FSourceUri& URI) const override;
		virtual void RegisterResolver(FName ResolverName, const TSharedRef<IUriResolver>& UriResolver) override;
		virtual bool UnregisterResolver(FName ResolverName) override;
		virtual const TArray<FName>& GetSupportedSchemes() const override;
#if WITH_EDITOR
		virtual TSharedPtr<FExternalSource> BrowseExternalSource(const FName& UriScheme, const FSourceUri& DefaultSourceUri) const override;
#endif //WITH_EDITOR
		// IUriManager interface end

	private:
		TSharedPtr<IUriResolver> GetFirstCompatibleResolver(const FSourceUri& Uri) const;

		void InvalidateCache();

		TArray<FUriResolverRegisterInformation> RegisteredResolvers;

		mutable TArray<FName> CachedSchemes;
	};
	
}