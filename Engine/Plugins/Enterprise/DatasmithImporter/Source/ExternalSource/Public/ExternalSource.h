// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SourceUri.h"

#include "Async/Future.h"
#include "Containers/Queue.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "HAL/Platform.h"
#include "Logging/LogMacros.h"
#include "Misc/SecureHash.h"
#include "Templates/SharedPointer.h"

#if WITH_EDITOR
#include "Delegates/IDelegateInstance.h"
#endif //WITH_EDITOR

class FAsyncTaskNotification;
class FDatasmithSceneGuard;
class IDatasmithScene;
class IDatasmithTranslator;

namespace UE::DatasmithImporter
{
	class FExternalSource;
}

DECLARE_LOG_CATEGORY_EXTERN(LogExternalSource, Log, All);

DECLARE_MULTICAST_DELEGATE_OneParam(OnExternalSourceChangedDelegate, const TSharedRef<UE::DatasmithImporter::FExternalSource>&);

namespace UE::DatasmithImporter
{
	struct FExternalSourceCapabilities
	{
		/**
		 * FExternalSource::Load() should only be called when this capability is enabled.
		 */
		bool bSupportSynchronousLoading = false;
		/**
		 * FExternalSource::AsyncLoad() should only be called when this capability is enabled.
		 */
		bool bSupportAsynchronousLoading = false;
	};

	/**
	 * Class allowing to load a IDatasmithScene from a source URI.
	 */
	class EXTERNALSOURCE_API FExternalSource : public TSharedFromThis<FExternalSource>
	{
	public:
		explicit FExternalSource(const FSourceUri& InSourceUri);

		virtual ~FExternalSource();

		virtual FString GetSourceName() const = 0;

		const FSourceUri& GetSourceUri() const { return SourceUri; };

		/**
		 * Return true if the resource pointed by this ExternalSource is available for being loaded.
		 */
		virtual bool IsAvailable() const = 0;

		/**
		 * Return true if the loaded scene is out-of-sync and needs to be reloaded.
		 */
		virtual bool IsOutOfSync() const = 0;

		/**
		 * Gives the hash of the source, only returns a valid hash after loading the scene.
		 */
		virtual FMD5Hash GetSourceHash() const = 0;

		virtual FExternalSourceCapabilities GetCapabilities() const = 0;

		/**
		 * Return the DatasmithScene if it is loaded, return an invalid TSharedPtr otherwise.
		 */
		virtual TSharedPtr<IDatasmithScene> GetDatasmithScene() const = 0;

		/**
		 * As long as UFactory does not offer a way to import FExternalSource directly, we must rely on using filepaths, even for source that are not on the file system.
		 */
		virtual FString GetFallbackFilepath() const = 0;
		
		/**
		 * DISCLAIMER: The translator is exposed here as a temporary workaround in order to set the import option before the load
		 *			   and load assets with bulk data (StaticMeshes/LoadLevelSequence/etc.). We should not rely on this for anything
		 *			   else as this will be removed once Import options can be set directly on FExternalSource.
		 */
		const TSharedPtr<IDatasmithTranslator>& GetAssetTranslator();

		/**
		 * Return the name of the scene that will be loaded.
		 * #ueent_todo This is only required because of the way the Datasmith reimport works. Consider removing it when we adapt the ExternalSource to interchange.
		 */
		FString GetSceneName() const;

		/**
		 * Override the name of the scene that will be loaded. If the pointer is nullptr, the override will be reset.
		 * #ueent_todo This is only required because of the way the Datasmith reimport works. Consider removing it when we adapt the ExternalSource to interchange.
		 */
		void SetSceneName(const TCHAR* SceneName);

		/**
		 * Attempt to load the FExternalSource according to its supported capabilities.
		 */
		TSharedPtr<IDatasmithScene> TryLoad();

		/**
		 * Load the external source and return a valid IDatasmithScene TSharedPtr if successful.
		 * The FExternalSource must have the FExternalSourceCapabilities::bSupportSynchronousLoading enabled to call this.
		 */
		TSharedPtr<IDatasmithScene> Load();

		/**
		 * Load the external source asynchronously, the returned TFuture gives a valid IDatasmithScene TSharedPtr if successful.
		 * The FExternalSource must have the FExternalSourceCapabilities::bSupportAsynchronousLoading enabled to call this.
		 */
		TFuture<TSharedPtr<IDatasmithScene>> AsyncLoad();

		/**
		 * True when the ExternalSource is currently loading the source asynchronously.
		 */
		bool IsAsyncLoading() const { return AsyncTaskNotification.IsValid(); }

		/**
		 * Cancel any pending AsyncLoad() operation, pending TFuture will return invalid TSharedPtr.
		 */
		void CancelAsyncLoad();

		/**
		 * Delegate called on main thread every time the loaded data is updated.
		 * Used for registering auto-reimport on assets.
		 */
		OnExternalSourceChangedDelegate OnExternalSourceChanged;

	protected:

		/**
		 * Synchronously load the DatasmithScene. Upon success, GetDatasmithScene() should return a valid scene.
		 * @return a valid SharedPtr if the operation was successful.
		 */
		virtual TSharedPtr<IDatasmithScene> LoadImpl() = 0;
		
		/**
		 * Starts an async task for loading the DatasmithScene.
		 * When the task is completed, TriggerOnExternalSourceLoaded() should be called to notify that the Scene is now available.
		 * @return true if the async loading task is running.
		 */
		virtual bool StartAsyncLoad() = 0;

		/**
		 * Interrupt async task notification and clear all registered delegates. 
		 * Used for when a source becomes unavailable.
		 */
		void ClearOnExternalSourceLoadedDelegates();

		/**
		 * Broadcast OnExternalSourceChanged delegates and set pending TPromises.
		 */
		void TriggerOnExternalSourceChanged();

		/**
		 * Loads a scene from the translator and starts a new translator load lifecycle.
		 */
		bool TranslatorLoadScene(const TSharedRef<IDatasmithScene>& Scene);

	private:

		typedef TOptional<TPromise<TSharedPtr<IDatasmithScene>>> FOptionalScenePromise;

		void ValidateDatasmithVersion() const;

		/**
		 * Queue containing the unfulfilled promises added by AsyncLoad()
		 * We can't directly make a TQueue of TPromise, as the queue internally creates nodes for tracking the head/tail,
		 * those nodes would then hold default-constructed TPromises and would assert on deletion, since deleting an unset promise is a programming error.
		 */
		TQueue<FOptionalScenePromise, EQueueMode::Mpsc> PendingPromiseQueue;

		/**
		 * Async task notification, for when we are loading the Datasmith scene asynchronously.
		 */
		TSharedPtr<FAsyncTaskNotification> AsyncTaskNotification;

		/**
		 * The URI of the source this ExternalSource is accessing.
		 */
		FSourceUri SourceUri;

		FString SceneName;

		TSharedPtr<IDatasmithTranslator> AssetTranslator;

		TUniquePtr<FDatasmithSceneGuard> SceneGuard;
	};
}