// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IUriManager.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#define EXTERNALSOURCE_MODULE_NAME TEXT("ExternalSource")

namespace UE::DatasmithImporter
{
	class FExternalSource;
	class FSourceUri;
}

class IExternalSourceModule : public IModuleInterface
{
public:
	static inline IExternalSourceModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IExternalSourceModule>(EXTERNALSOURCE_MODULE_NAME);
	}

	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(EXTERNALSOURCE_MODULE_NAME);
	}

	virtual UE::DatasmithImporter::IUriManager& GetManager() const = 0;

	static TSharedPtr<UE::DatasmithImporter::FExternalSource> GetOrCreateExternalSource(const UE::DatasmithImporter::FSourceUri& Uri)
	{
		return Get().GetManager().GetOrCreateExternalSource(Uri);
	}
};