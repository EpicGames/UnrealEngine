// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SourceUri.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

#if WITH_EDITOR
#include "EditorFramework/AssetImportData.h"
#endif //WITH_EDITOR

namespace UE::DatasmithImporter
{
	class FExternalSource;
	class IUriResolver;

	class EXTERNALSOURCE_API IUriManager
	{
	public:
		virtual ~IUriManager() = default;

		/**
		 * Using the registered UriManagers, return the FExternalSource associated to the given Uri, either by creating it or returning a cached value.
		 * Returns nullptr if the Uri is not compatible.
		 */
		virtual TSharedPtr<FExternalSource> GetOrCreateExternalSource(const FSourceUri& Uri) const = 0;

		/**
		 * Return true if there is a registered UriResolver that can generate a FExternalSource from the given Uri.
		 */
		virtual bool CanResolveUri(const FSourceUri& Uri) const = 0;

		/**
		 * Register the provided UriResolver in the manager.
		 */
		virtual void RegisterResolver(FName ResolverName, const TSharedRef<IUriResolver>& UriResolver) = 0;

		/**
		 * Unregister the UriResolver associated to the provided name.
		 */
		virtual bool UnregisterResolver(FName ResolverName) = 0;

		/**
		 * Return an array containing all the scheme supported by the registered URiResolvers.
		 */
		virtual const TArray<FName>& GetSupportedSchemes() const = 0;

#if WITH_EDITOR
		/**
		 * Open a dialog window for browsing ExternalSource based on the passed URI Scheme
		 */
		virtual TSharedPtr<FExternalSource> BrowseExternalSource(const FName& UriScheme, const FSourceUri& DefaultSourceUri = FSourceUri()) const = 0;

		template<typename ImportDataType>
		TSharedPtr<FExternalSource> TryGetExternalSourceFromImportData(const ImportDataType& ImportSourceData) const
		{
			FSourceUri SourceUri(ImportSourceData.DatasmithImportInfo.SourceUri);

			// If we have a valid URI, we should not try to read the source file path.
			if (SourceUri.IsValid())
			{
				return GetOrCreateExternalSource(SourceUri);
			}

			const FAssetImportInfo::FSourceFile* FirstFileInfo = ImportSourceData.SourceData.SourceFiles.GetData();
			if (FirstFileInfo)
			{
				SourceUri = FSourceUri::FromFilePath(FirstFileInfo->RelativeFilename);
				return GetOrCreateExternalSource(SourceUri);
			}

			return nullptr;
		}
#endif //WITH_EDITOR
	};
}
