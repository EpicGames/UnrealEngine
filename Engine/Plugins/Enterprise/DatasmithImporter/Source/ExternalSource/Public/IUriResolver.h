// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/NameTypes.h"
#include "Templates/SharedPointer.h"

namespace UE::DatasmithImporter
{
	class FExternalSource;
	class FSourceUri;

	class EXTERNALSOURCE_API IUriResolver
	{
	public:
		virtual ~IUriResolver() = default;
	
		/**
		 * Return the FExternalSource associated to the given Uri, either by creating it or returning a cached value.
		 * Returns nullptr if the Uri is not compatible.
		 */
		virtual TSharedPtr<FExternalSource> GetOrCreateExternalSource(const FSourceUri& Uri) const = 0;
		
		/**
		 * Return true if the UriResolver can generate a FExternalSource from the given Uri.
		 */
		virtual bool CanResolveUri(const FSourceUri& Uri) const = 0;

		/**
		 * Return the URI scheme handled by the UriResolver.
		 */
		virtual FName GetScheme() const = 0;

#if WITH_EDITOR
		/**
		 * Open a dialog window for browsing ExternalSources associated to this UriResolver
		 */
		virtual TSharedPtr<FExternalSource> BrowseExternalSource(const FSourceUri& DefaultUri) const = 0;
#endif //WITH_EDITOR
	};
}