// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"

struct FAssetData;

namespace UE::DatasmithImporter
{
	/**
	 * URI container class used for referencing external sources.
	 * TODO - A proper standardized version should be implemented in a Runtime/Core module to unify all URI implementations across the engine.
	 *		  For example, a URI implementation is provided in Plugins\Media\ElectraPlayer\Source\ElectraPlayerRuntime\Private\Runtime\Utilities\URI.h
	 */
	class EXTERNALSOURCE_API FSourceUri
	{
	public:
		FSourceUri() = default;

		explicit FSourceUri(const FString& InUri)
			: Uri(InUri)
		{}

		FSourceUri(const FString& InScheme, const FString& InPath)
			: Uri(InScheme + GetSchemeDelimiter() + InPath)
		{}

		template<typename TContainer>
		FSourceUri(const FString& InScheme, const FString& InPath, const TContainer& QueryContainer)
			: Uri(InScheme + GetSchemeDelimiter() + InPath + BuildQueryString(QueryContainer))
		{}

		/**
		 * Generate a FSourceUri from the given filepath.
		 */
		static FSourceUri FromFilePath(const FString& FilePath);

		/**
		 * Return the scheme used for file URIs : "file"
		 */
		static const FString& GetFileScheme();

		/**
		 * Try to get the FSourceUri registered in the AssetData. If no SourceUri is registered, an invalid FSourceUri is returned.
		 */
		static FSourceUri FromAssetData(const FAssetData& AssetData);

		/**
		 * Returns the Tag that should be used for registering the SourceUri of an asset in its corresponding FAssetData.
		 */
		static FName GetAssetDataTag();

		/**
		 * Return if the FSourceUri is a properly structure valid uri.
		 */
		bool IsValid() const;

		FStringView GetScheme() const;

		/**
		 * Check if the FSourceUri has the provided scheme.
		 * @param InScheme	The scheme to look for.
		 * @return	True if the Uri has the provided scheme.
		 */
		bool HasScheme(const FString& InScheme) const;

		/**
		 * Return the path of the FSourceUri
		 * @return	The path, for now no distinction is made between the authority and the path.
		 */
		FStringView GetPath() const;

		FStringView GetQuery() const;

		TMap<FString, FString> GetQueryMap() const;

		/**
		 * Return the FSourceUri as a string.
		 */
		const FString& ToString() const { return Uri; }

		bool operator==(const FSourceUri& Other) const
		{
			return Uri == Other.Uri;
		}

		bool operator!=(const FSourceUri& Other) const
		{
			return !(*this == Other);
		}

	private:

		template<typename TContainer>
		static FString BuildQueryString(const TContainer& QueryContainer)
		{
			static_assert(std::is_same<typename TContainer::ElementType, TPair<FString, FString>>::value, "Query container must consist of a collection of TPair<FString, FString>.");

			if (QueryContainer.Num() > 0)
			{
				TCHAR QueryDelimiter(GetQueryStartCharacter());
				FString QueryString = FString::ConstructFromPtrSize(&QueryDelimiter, 1);
				for (typename TContainer::TConstIterator It = QueryContainer.CreateConstIterator(); It;)
				{
					const TPair<FString, FString>& CurrentPair = *It;

					if (++It)
					{
						// We are not on the last pair, add '&' character to split this pair with the next.
						QueryString += FString::Printf(TEXT("%s%c%s%c"), *CurrentPair.Key, GetQueryPairCharacter(), *CurrentPair.Value, GetQueryAppendPairCharacter());
					}
					else
					{
						QueryString += FString::Printf(TEXT("%s%c%s"), *CurrentPair.Key, GetQueryPairCharacter(), *CurrentPair.Value);
					}
				}

				return QueryString;
			}

			return FString();
		}

		static const FString& GetSchemeDelimiter();

		/**
		 * URI Queries are structured like so <RestOfURI>?<KeyA><ValueA>&<KeyB>=<ValueB>
		 */
		static TCHAR GetQueryStartCharacter() { return '?'; }
		static TCHAR GetQueryPairCharacter() { return '='; }
		static TCHAR GetQueryAppendPairCharacter() { return '&'; }

		FString Uri;
	};

	inline uint32 GetTypeHash(const FSourceUri& SourceUri)
	{
		return GetTypeHash(SourceUri.ToString());
	}
}